// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DetailCustomizations : ModuleRules
{
	public DetailCustomizations(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] { "Slate", "PropertyEditor" });
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AdvancedWidgets",
				"AppFramework",
				"Core",
// 				"AudioEditor",
				"CoreUObject",
				"ApplicationCore",
				"DesktopWidgets",
				"Engine",
				"Landscape",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"EditorWidgets",
				"Kismet",
				"KismetWidgets",
				"MovieSceneCapture",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Sequencer",
				"MovieScene",
				"TimeManagement",
				"SharedSettingsWidgets",
				"ContentBrowser",
				"BlueprintGraph",
				"GraphEditor",
				"AnimGraph",
				"PropertyEditor",
				"LevelEditor",
				"DesktopPlatform",
				"ClassViewer",
				"TargetPlatform",
				"ExternalImagePicker",
				"MoviePlayer",
				"SourceControl",
				"InternationalizationSettings",
				"SourceCodeAccess",
				"RHI",
				"HardwareTargeting",
				"NavigationSystem",
				"AIModule", 
				"ConfigEditor",
				"CinematicCamera",
				"ComponentVisualizers",
				"SkeletonEditor",
				"LevelSequence",
				"AdvancedPreviewScene",
				"AudioSettingsEditor",
				"HeadMountedDisplay",
				"DataTableEditor",
				"ToolMenus",
				"PhysicsCore",
				"RenderCore",
				"ToolWidgets",
				"MaterialEditor",
				"VirtualTexturingEditor",
				"Json",
				"JsonUtilities",
				"SettingsEditor",
				"ShaderPlatformConfigEditor",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Media",
				"LandscapeEditor",
				"Layers",
				"GameProjectGeneration",
				"MeshMergeUtilities",
				"MeshReductionInterface",
				"GeometryProcessingInterfaces"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Layers",
				"GameProjectGeneration",
				"MeshMergeUtilities",
				"MeshReductionInterface",
				"GeometryProcessingInterfaces"
			}
		);
	}
}
