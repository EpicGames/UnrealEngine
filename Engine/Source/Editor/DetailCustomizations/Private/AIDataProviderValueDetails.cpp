// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIDataProviderValueDetails.h"

#include "DataProviders/AIDataProvider.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AIDataProviderValueDetails"

TSharedRef<IPropertyTypeCustomization> FAIDataProviderValueDetails::MakeInstance()
{
	return MakeShareable(new FAIDataProviderValueDetails);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FAIDataProviderValueDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	DataBindingProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAIDataProviderValue, DataBinding));
	DataFieldProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAIDataProviderValue, DataField));
	DefaultValueProperty = StructPropertyHandle->GetChildHandle(TEXT("DefaultValue"));

	DataBindingProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAIDataProviderValueDetails::OnBindingChanged));
	
	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData(StructPtrs);
	DataPtr = (StructPtrs.Num() == 1) ? reinterpret_cast<FAIDataProviderValue*>(StructPtrs[0]) : nullptr;
	OnBindingChanged();

	TSharedRef<SWidget> DefPropWidget = DefaultValueProperty->CreatePropertyValueWidget();
	DefPropWidget->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAIDataProviderValueDetails::GetDefaultValueVisibility)));

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	.MinDesiredWidth(300.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 2.0f, 5.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(this, &FAIDataProviderValueDetails::GetValueDesc)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAIDataProviderValueDetails::GetBindingDescVisibility)))
		]
		+ SHorizontalBox::Slot()
		.Padding(0.0f, 2.0f, 5.0f, 2.0f)
		[
			DefPropWidget
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FAIDataProviderValueDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (StructPropertyHandle->IsValidHandle())
	{
		StructBuilder.AddProperty(DataBindingProperty.ToSharedRef());

		StructBuilder.AddCustomRow(LOCTEXT("PropertyField", "Property"))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAIDataProviderValueDetails::GetDataFieldVisibility)))
		.NameContent()
		[
			DataFieldProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FAIDataProviderValueDetails::OnGetDataFieldContent)
			.ContentPadding(FMargin(2.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FAIDataProviderValueDetails::GetDataFieldDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];		
	}
}

void FAIDataProviderValueDetails::OnBindingChanged()
{
	MatchingProperties.Reset();
	if (DataPtr)
	{
		DataPtr->GetMatchingProperties(MatchingProperties);

		FName NameValue;
		DataFieldProperty->GetValue(NameValue);

		// If current field name is not in the list of matching names then default on the first match
		if (MatchingProperties.Num() && !MatchingProperties.Contains(NameValue))
		{
			ensureAlwaysMsgf(false, TEXT("We expect UEnvQueryNode::PostEditChangeProperty to prevent "
				"copying an incompatible provider value on top of an existing one before calling NotifyAssetUpdate and refreshing the details."));
			OnDataFieldNameChange(0);
		}

		// If current field name is set but there are no matching names then reset the binding to the default since it's invalid.
		// This could happen when copy /pasting FAIDataProviderValue based properties of different types.
		else if (MatchingProperties.IsEmpty() && !NameValue.IsNone())
		{
			ensureAlwaysMsgf(false, TEXT("We expect UEnvQueryNode::PostEditChangeProperty to prevent "
				"copying an incompatible provider value on top of an existing one before calling NotifyAssetUpdate and refreshing the details."));
			DataBindingProperty->ResetToDefault();
			DataFieldProperty->ResetToDefault();
		}
	}
}

TSharedRef<SWidget> FAIDataProviderValueDetails::OnGetDataFieldContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (int32 i = 0; i < MatchingProperties.Num(); i++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(this, &FAIDataProviderValueDetails::OnDataFieldNameChange, i));
		MenuBuilder.AddMenuEntry(FText::FromName(MatchingProperties[i]), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

void FAIDataProviderValueDetails::OnDataFieldNameChange(int32 Index)
{
	FName NameValue = MatchingProperties[Index];
	DataFieldProperty->SetValue(NameValue);
}

FText FAIDataProviderValueDetails::GetDataFieldDesc() const
{
	FName NameValue;
	DataFieldProperty->GetValue(NameValue);

	return FText::FromString(NameValue.ToString());
}

FText FAIDataProviderValueDetails::GetValueDesc() const
{
	return DataPtr ? FText::FromString(DataPtr->ToString()) : LOCTEXT("EmptyValue", "empty");
}

EVisibility FAIDataProviderValueDetails::GetDataFieldVisibility() const
{
	return (DataPtr && DataPtr->DataBinding && (MatchingProperties.Num() > 1)) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAIDataProviderValueDetails::GetBindingDescVisibility() const
{
	return (DataPtr && DataPtr->DataBinding) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAIDataProviderValueDetails::GetDefaultValueVisibility() const
{
	return (!DataPtr || !DataPtr->DataBinding) ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
