// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IPropertyHandle;
class SWidget;
struct FAIDataProviderValue;

class FAIDataProviderValueDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	TSharedPtr<IPropertyHandle> DataBindingProperty;
	TSharedPtr<IPropertyHandle> DataFieldProperty;
	TSharedPtr<IPropertyHandle> DefaultValueProperty;
	FAIDataProviderValue* DataPtr;

	TArray<FName> MatchingProperties;

	void OnBindingChanged();
	TSharedRef<SWidget> OnGetDataFieldContent();
	void OnDataFieldNameChange(int32 Index);
	FText GetDataFieldDesc() const;
	FText GetValueDesc() const;
	EVisibility GetBindingDescVisibility() const;
	EVisibility GetDataFieldVisibility() const;
	EVisibility GetDefaultValueVisibility() const;
};
