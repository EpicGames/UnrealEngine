// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorComponentDetails.h"

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/EngineBaseTypes.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "ActorComponentDetails"

TSharedRef<IDetailCustomization> FActorComponentDetails::MakeInstance()
{
	return MakeShareable( new FActorComponentDetails );
}

void FActorComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TSharedPtr<IPropertyHandle> PrimaryTickProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UActorComponent, PrimaryComponentTick));

	// Defaults only show tick properties
	if (PrimaryTickProperty->IsValidHandle() && DetailBuilder.HasClassDefaultObject())
	{
		IDetailCategoryBuilder& TickCategory = DetailBuilder.EditCategory("ComponentTick");

		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bStartWithTickEnabled)));
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickInterval)));
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bTickEvenWhenPaused)), EPropertyLocation::Advanced);
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bAllowTickOnDedicatedServer)), EPropertyLocation::Advanced);
		TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickGroup)), EPropertyLocation::Advanced);
	}

	PrimaryTickProperty->MarkHiddenByCustomization();

	TArray<TWeakObjectPtr<UObject>> WeakObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(WeakObjectsBeingCustomized);

	bool bHideReplicates = false;
	for (TWeakObjectPtr<UObject>& WeakObjectBeingCustomized : WeakObjectsBeingCustomized)
	{
		if (UObject* ObjectBeingCustomized = WeakObjectBeingCustomized.Get())
		{
			if (UActorComponent* Component = Cast<UActorComponent>(ObjectBeingCustomized))
			{
				if (!Component->GetComponentClassCanReplicate())
				{
					bHideReplicates = true;
					break;
				}
			}
			else
			{
				bHideReplicates = true;
				break;
			}
		}
	}

	if (bHideReplicates)
	{
		TSharedPtr<IPropertyHandle> ReplicatesProperty = DetailBuilder.GetProperty(UActorComponent::GetReplicatesPropertyName());
		ReplicatesProperty->MarkHiddenByCustomization();
	}
}


#undef LOCTEXT_NAMESPACE
