// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorDetails.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/Level.h"
#include "UObject/UnrealType.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "ToolMenus.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "Engine/Brush.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/Volume.h"
#include "GameFramework/WorldSettings.h"
#include "Components/BillboardComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"
#include "LayersModule.h"
#include "LevelEditor.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "EdGraphSchema_K2.h"
#include "ComponentTransformDetails.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "Engine/BrushShape.h"
#include "ActorDetailsDelegates.h"
#include "EditorCategoryUtils.h"
#include "Widgets/Input/SHyperlink.h"
#include "ObjectEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "Algo/AnyOf.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_AddDelegate.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Subsystems/EditorActorSubsystem.h"

#define LOCTEXT_NAMESPACE "ActorDetails"

FExtendActorDetails OnExtendActorDetails;

TSharedRef<IDetailCustomization> FActorDetails::MakeInstance()
{
	return MakeShared<FActorDetails>();
}

FActorDetails::~FActorDetails()
{
}

void FActorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	// Get the list of hidden categories
	TArray<FString> HideCategories;
	if (DetailLayout.GetBaseClass())
	{
		FEditorCategoryUtils::GetClassHideCategories(DetailLayout.GetBaseClass(), HideCategories);
	}

	// These details only apply when adding an instance of the actor in a level
	if (TSharedPtr<IDetailsView> DetailsView = DetailLayout.GetDetailsViewSharedPtr(); !DetailLayout.HasClassDefaultObject() && DetailsView && DetailsView->GetSelectedActorInfo().NumSelected > 0)
	{
		// Build up a list of unique blueprints in the selection set (recording the first actor in the set for each one)
		TMap<UBlueprint*, UObject*> UniqueBlueprints;

		// Per level Actor Counts
		TMap<ULevel*, int32> ActorsPerLevelCount;

		bool bHasBillboardComponent = false;
		const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout.GetSelectedObjects();
		for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
		{
			AActor* Actor = Cast<AActor>( SelectedObjects[ObjectIndex].Get() );

			if (Actor != NULL)
			{
				// Store the selected actors for use later. Its fine to do this when CustomizeDetails is called because if the selected actors changes, CustomizeDetails will be called again on a new instance
				// and our current resource would be destroyed.
				SelectedActors.Add( Actor );

				// Record the level that contains this actor and increment it's actor count
				ULevel* Level = Actor->GetLevel();
				if (Level != NULL)
				{
					int32& ActorCountForThisLevel = ActorsPerLevelCount.FindOrAdd(Level);
					++ActorCountForThisLevel;
				}

				// Add to the unique blueprint map if the actor is generated from a blueprint
				if (UBlueprint* Blueprint = Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy))
				{
					if (!UniqueBlueprints.Find(Blueprint))
					{
						UniqueBlueprints.Add(Blueprint, Actor);
					}
				}

				if (!bHasBillboardComponent)
				{
					bHasBillboardComponent = Actor->FindComponentByClass<UBillboardComponent>() != NULL;
				}
			}
		}

		if (!bHasBillboardComponent)
		{
			// Actor billboard scale is not relevant if the actor doesn't have a billboard component
			DetailLayout.HideProperty( GET_MEMBER_NAME_CHECKED(AActor, SpriteScale) );
		}

		if (!HideCategories.Contains(TEXT("Transform")))
		{
			AddTransformCategory(DetailLayout);
		}
		
		if (!HideCategories.Contains(TEXT("Actor")))
		{
			AddActorCategory(DetailLayout, ActorsPerLevelCount);
		}

		// Hide World Partition specific properties in non WP levels
		const bool bShouldDisplayWorldPartitionProperties = Algo::AnyOf(SelectedActors, [](const TWeakObjectPtr<AActor> Actor)
		{
			UWorld* World = Actor.IsValid() ? Actor->GetTypedOuter<UWorld>() : nullptr;
			return UWorld::IsPartitionedWorld(World);
		});

		if (!bShouldDisplayWorldPartitionProperties)
		{
			DetailLayout.HideProperty(DetailLayout.GetProperty(AActor::GetRuntimeGridPropertyName(), AActor::StaticClass()));
			DetailLayout.HideProperty(DetailLayout.GetProperty(AActor::GetDataLayerAssetsPropertyName(), AActor::StaticClass()));
			DetailLayout.HideProperty(DetailLayout.GetProperty(AActor::GetExternalDataLayerAssetPropertyName(), AActor::StaticClass()));
			DetailLayout.HideProperty(DetailLayout.GetProperty(AActor::GetHLODLayerPropertyName(), AActor::StaticClass()));
		}

		OnExtendActorDetails.Broadcast(DetailLayout, FGetSelectedActors::CreateSP(this, &FActorDetails::GetSelectedActors));
	}

	TSharedPtr<IPropertyHandle> PrimaryTickProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(AActor, PrimaryActorTick));

	// Defaults only show tick properties
	if (DetailLayout.HasClassDefaultObject())
	{
		if (!HideCategories.Contains(TEXT("Tick")))
		{
			// Note: the category is renamed to differentiate between 
			IDetailCategoryBuilder& TickCategory = DetailLayout.EditCategory("Tick", LOCTEXT("TickCategoryName", "Actor Tick") );

			TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bStartWithTickEnabled)));
			TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickInterval)));
			TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bTickEvenWhenPaused)), EPropertyLocation::Advanced);
			TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, bAllowTickOnDedicatedServer)), EPropertyLocation::Advanced);
			TickCategory.AddProperty(PrimaryTickProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTickFunction, TickGroup)), EPropertyLocation::Advanced);
		}
		
		if (!HideCategories.Contains(TEXT("Events")))
		{
			AddEventsCategory(DetailLayout);
		}	
	}

	PrimaryTickProperty->MarkHiddenByCustomization();
}

void FActorDetails::OnConvertActor(UClass* ChosenClass)
{
	if (ChosenClass)
	{
		// Check each selected actor's pointer.
		TArray<AActor*> SelectedActorsRaw;
		for (int32 i=0; i<SelectedActors.Num(); i++)
		{
			if (SelectedActors[i].IsValid())
			{
				SelectedActorsRaw.Add(SelectedActors[i].Get());
			}
		}

		// If there are valid pointers, convert the actors.
		if (SelectedActorsRaw.Num())
		{
			// Dismiss the menu BEFORE converting actors as it can refresh the details panel and if the menu is still open
			// it will be parented to an invalid actor details widget
			FSlateApplication::Get().DismissAllMenus();

			UEditorActorSubsystem::ConvertActors(SelectedActorsRaw, ChosenClass, TSet<FString>(), true);
		}
	}
}

class FConvertToClassFilter : public IClassViewerFilter
{
public:
	/** All classes in this set will be allowed. */
	TSet< const UClass* > AllowedClasses;

	/** All classes in this set will be disallowed. */
	TSet< const UClass* > DisallowedClasses;

	/** Allowed ChildOf relationship. */
	TSet< const UClass* > AllowedChildOfRelationship;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		EFilterReturn::Type eState = InFilterFuncs->IfInClassesSet(AllowedClasses, InClass);
		if(eState == EFilterReturn::NoItems)
		{
			eState = InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfRelationship, InClass);
		}

		// As long as it has not failed to be on an allowed list, check if it is on a disallowed list.
		if(eState == EFilterReturn::Passed)
		{
			eState = InFilterFuncs->IfInClassesSet(DisallowedClasses, InClass);

			// If it passes, it's on the disallowed list, so we do not want it.
			if(eState == EFilterReturn::Passed)
			{
				return false;
			}
			else
			{
				return true;
			}
		}

		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		EFilterReturn::Type eState = InFilterFuncs->IfInClassesSet(AllowedClasses, InUnloadedClassData);
		if(eState == EFilterReturn::NoItems)
		{
			eState = InFilterFuncs->IfInChildOfClassesSet(AllowedChildOfRelationship, InUnloadedClassData);
		}

		// As long as it has not failed to be on an allowed list, check if it is on a disallowed list.
		if(eState == EFilterReturn::Passed)
		{
			eState = InFilterFuncs->IfInClassesSet(DisallowedClasses, InUnloadedClassData);

			// If it passes, it's on the disallowed list, so we do not want it.
			if(eState == EFilterReturn::Passed)
			{
				return false;
			}
			else
			{
				return true;
			}
		}

		return false;
	}
};

UClass* FActorDetails::GetConversionRoot( UClass* InCurrentClass ) const
{
	UClass* ParentClass = InCurrentClass;

	while(ParentClass)
	{
		if( ParentClass->GetBoolMetaData(FName(TEXT("IsConversionRoot"))) )
		{
			break;
		}
		ParentClass = ParentClass->GetSuperClass();
	}

	return ParentClass;
}

void FActorDetails::CreateClassPickerConvertActorFilter(const TWeakObjectPtr<AActor> ConvertActor, class FClassViewerInitializationOptions* ClassPickerOptions)
{
	// Shouldn't ever be overwriting an already established filter
	check( ConvertActor.IsValid() )
	check( ClassPickerOptions != nullptr && ClassPickerOptions->ClassFilters.IsEmpty() );
	TSharedRef<FConvertToClassFilter> Filter = MakeShared<FConvertToClassFilter>();
	ClassPickerOptions->ClassFilters.Add(Filter);

	UClass* ConvertClass = ConvertActor->GetClass();
	UClass* RootConversionClass = GetConversionRoot(ConvertClass);

	if(RootConversionClass)
	{
		Filter->AllowedChildOfRelationship.Add(RootConversionClass);
	}

	// Never convert to the same class
	Filter->DisallowedClasses.Add(ConvertClass);

	if( ConvertActor->IsA<ABrush>() )
	{
		// Volumes cannot be converted to brushes or brush shapes or the abstract type
		Filter->DisallowedClasses.Add(ABrush::StaticClass());
		Filter->DisallowedClasses.Add(ABrushShape::StaticClass());
		Filter->DisallowedClasses.Add(AVolume::StaticClass());
	}
}


TSharedRef<SWidget> FActorDetails::OnGetConvertContent()
{
	// Build a class picker widget

	// Fill in options
	FClassViewerInitializationOptions Options;

	Options.bShowUnloadedBlueprints = true;
	Options.bIsActorsOnly = true;
	Options.bIsPlaceableOnly = true;

	// All selected actors are of the same class, so just need to use one to generate the filter
	if ( SelectedActors.Num() > 0 )
	{
		CreateClassPickerConvertActorFilter(SelectedActors.Top(), &Options);
	}

	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::ListView;

	TSharedRef<SWidget> ClassPicker = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &FActorDetails::OnConvertActor));

	return
		SNew(SBox)
		.WidthOverride(280.f)
		.MaxDesiredHeight(500.f)
		[
			ClassPicker
		];
}

EVisibility FActorDetails::GetConvertMenuVisibility() const
{
	return EVisibility::Visible;
}

TSharedRef<SWidget> FActorDetails::MakeConvertMenu( const FSelectedActorInfo& SelectedActorInfo )
{
	UClass* RootConversionClass = GetConversionRoot(SelectedActorInfo.SelectionClass);
	return
		SNew(SComboButton)
		.ContentPadding(2.f)
		.IsEnabled(RootConversionClass != NULL)
		.Visibility(this, &FActorDetails::GetConvertMenuVisibility)
		.OnGetMenuContent(this, &FActorDetails::OnGetConvertContent)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SelectAType", "Select a Type"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

}

void FActorDetails::OnNarrowSelectionSetToSpecificLevel( TWeakObjectPtr<ULevel> LevelToNarrowInto )
{
	if (ULevel* RequiredLevel = LevelToNarrowInto.Get())
	{
		// Remove any selected objects that aren't in the specified level
		TArray<AActor*> ActorsToDeselect;
		for ( TArray< TWeakObjectPtr<AActor> >::TConstIterator Iter(SelectedActors); Iter; ++Iter)
		{ 
			if( (*Iter).IsValid() )
			{
				AActor* Actor = (*Iter).Get();
				if (!Actor->IsIn(RequiredLevel))
				{
					ActorsToDeselect.Add(Actor);
				}
			}
		}

		for (TArray<AActor*>::TIterator DeselectIt(ActorsToDeselect); DeselectIt; ++DeselectIt)
		{
			AActor* Actor = *DeselectIt;
			GEditor->SelectActor(Actor, /*bSelected=*/ false, /*bNotify=*/ false);
		}

		// Tell the editor selection status was changed.
		GEditor->NoteSelectionChange();
	}
}

bool FActorDetails::IsActorValidForLevelScript() const
{
	AActor* Actor = GEditor->GetSelectedActors()->GetTop<AActor>();
	return FKismetEditorUtilities::IsActorValidForLevelScript(Actor);
}

FReply FActorDetails::FindSelectedActorsInLevelScript()
{
	GUnrealEd->FindSelectedActorsInLevelScript();
	return FReply::Handled();
};

bool FActorDetails::AreAnySelectedActorsInLevelScript() const
{
	return GUnrealEd->AreAnySelectedActorsInLevelScript();
};

/** Util to create a menu for events we can add for the selected actor */
TSharedRef<SWidget> FActorDetails::MakeEventOptionsWidgetFromSelection()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName("DetailCustomizations.EventOptions");
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		ToolMenus->RegisterMenu(MenuName);
	}

	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
	AActor* Actor = SelectedActors[0].Get();
	FKismetEditorUtilities::AddLevelScriptEventOptionsForActor(Menu, SelectedActors[0], true, true, false);
	return ToolMenus->GenerateWidget(Menu);
}


void FActorDetails::AddLayersCategory( IDetailLayoutBuilder& DetailBuilder )
{
	if( !FModuleManager::Get().IsModuleLoaded( TEXT("Layers") ) )
	{
		return;
	}

	FLayersModule& LayersModule = FModuleManager::LoadModuleChecked< FLayersModule >( TEXT("Layers") );

	const FText LayerCategory = LOCTEXT("LayersCategory", "Layers");

	DetailBuilder.EditCategory( "Layers", LayerCategory, ECategoryPriority::Uncommon )
	.AddCustomRow( FText::GetEmpty() )
	[
		LayersModule.CreateLayerCloud( SelectedActors )
	];
}

void FActorDetails::AddTransformCategory( IDetailLayoutBuilder& DetailBuilder )
{
	const FSelectedActorInfo& SelectedActorInfo = DetailBuilder.GetDetailsViewSharedPtr()->GetSelectedActorInfo();

	bool bAreBrushesSelected = SelectedActorInfo.bHaveBrush;
	bool bIsOnlyWorldPropsSelected =  SelectedActors.Num() == 1 && SelectedActors[0].IsValid() && SelectedActors[0]->IsA<AWorldSettings>();
	bool bLacksRootComponent = SelectedActors[0].IsValid() && (SelectedActors[0]->GetRootComponent()==NULL);

	// Don't show the Transform details if the only actor selected is world properties, or if they have no RootComponent
	if ( bIsOnlyWorldPropsSelected || bLacksRootComponent )
	{
		return;
	}

	TSharedRef<FComponentTransformDetails> TransformDetails = MakeShared<FComponentTransformDetails>(DetailBuilder.GetSelectedObjects(), SelectedActorInfo, DetailBuilder);

	IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory( "TransformCommon", LOCTEXT("TransformCommonCategory", "Transform"), ECategoryPriority::Transform );

	TransformCategory.AddCustomBuilder( TransformDetails );
}

void FActorDetails::AddEventsCategory(IDetailLayoutBuilder& DetailBuilder)
{
	// Get the currently selected actor, which would be the "Default__Actor" 
	const TArray<TWeakObjectPtr<UObject>>& Selected = DetailBuilder.GetSelectedObjects();
	if(Selected.IsEmpty())
	{
		return;
	}

	AActor* Actor = Cast<AActor>(Selected[0].Get());
	UBlueprint* Blueprint = Actor ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;

	if(!Actor || !Blueprint || !FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint))
	{
		return;
	}

	IDetailCategoryBuilder& EventsCategory = DetailBuilder.EditCategory("Events", FText::GetEmpty(), ECategoryPriority::Uncommon);
	static const FName HideInDetailPanelName("HideInDetailPanel");

	// Find all the Multicast delegate properties and give a binding button for them
	for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(Actor->GetClass(), EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FMulticastDelegateProperty* Property = *PropertyIt;
		
		// Only show BP assiangable, non-hidden delegates		
		if (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintAssignable) && !Property->HasMetaData(HideInDetailPanelName))
		{
			const FName EventName = Property->GetFName();
			FText EventText = Property->GetDisplayNameText();

			EventsCategory.AddCustomRow(EventText)
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
				.ToolTipText(Property->GetToolTipText())

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 5.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("GraphEditor.Event_16x"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(EventText)
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.f)
				[
					// A "Plus" button to add a binding. For dynamic delegates on the CDO, you can always
					// make a new binding, so always display the "Plus"
					SNew(SButton)
					.ContentPadding(FMargin(3.0f, 2.0f))
					.HAlign(HAlign_Center)
					.OnClicked(this, &FActorDetails::HandleAddOrViewEventForVariable, Blueprint, Property)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
					]
				]
			];
		}
	}
}

FReply FActorDetails::HandleAddOrViewEventForVariable(UBlueprint* BP, FMulticastDelegateProperty* Property)
{
	const UFunction* SignatureFunction = Property ? Property->SignatureFunction.Get() : nullptr;
	UEdGraph* EventGraph = BP ? FBlueprintEditorUtils::FindEventGraph(BP) : nullptr;

	if (EventGraph && SignatureFunction && Property)
	{
		const FVector2D SpawnPos = EventGraph->GetGoodPlaceForNewNode();

		// Adding a bound dynatic delegate from the Actor that is based off this BP will always be in a self context
		UK2Node_AddDelegate* TemplateNode = NewObject<UK2Node_AddDelegate>();
		TemplateNode->SetFromProperty(Property, /* bSelfContext */ true, Property->GetOwnerClass());

		UEdGraphNode* SpawnedDelegate = FEdGraphSchemaAction_K2AssignDelegate::AssignDelegate(TemplateNode, EventGraph, nullptr, SpawnPos, true);
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(SpawnedDelegate, false);
	}

	return FReply::Handled();
}

void FActorDetails::AddActorCategory( IDetailLayoutBuilder& DetailBuilder, const TMap<ULevel*, int32>& ActorsPerLevelCount )
{		
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );

	const FLevelEditorCommands& Commands = LevelEditor.GetLevelEditorCommands();
	TSharedRef<const FUICommandList> CommandBindings = LevelEditor.GetGlobalLevelEditorActions();

	const FSelectedActorInfo& SelectedActorInfo = DetailBuilder.GetDetailsViewSharedPtr()->GetSelectedActorInfo();
	TSharedPtr<SVerticalBox> LevelBox;

	IDetailCategoryBuilder& ActorCategory = DetailBuilder.EditCategory("Actor", FText::GetEmpty(), ECategoryPriority::Uncommon );

	if (GetSelectedActors().Num() == 1)
	{
		if (AActor* Actor = GEditor->GetSelectedActors()->GetTop<AActor>())
		{
			if (Actor->GetActorGuid().IsValid())
			{
				const FText ActorGuidText = FText::FromString(Actor->GetActorGuid().ToString());
				ActorCategory.AddCustomRow( LOCTEXT("ActorGuid", "ActorGuid") )
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ActorGuid2", "Actor Guid"))
						.ToolTipText(LOCTEXT("ActorGuid_ToolTip", "Actor Guid"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SNew(STextBlock)
							.Text(ActorGuidText)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.IsEnabled(false)
					];

				if (Actor->GetActorInstanceGuid() != Actor->GetActorGuid())
				{
					const FText ActorInstanceGuidText = FText::FromString(Actor->GetActorInstanceGuid().ToString());
					ActorCategory.AddCustomRow( LOCTEXT("ActorInstanceGuid", "ActorInstanceGuid") )
						.NameContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ActorInstanceGuid2", "Actor Instance Guid"))
							.ToolTipText(LOCTEXT("ActorInstanceGuid_ToolTip", "Actor Instance Guid"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						.ValueContent()
						[
							SNew(STextBlock)
								.Text(ActorInstanceGuidText)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.IsEnabled(false)
						];
				}
			}

			if (Actor->GetContentBundleGuid().IsValid())
			{
				UWorldPartition* WorldPartition = Actor->GetWorld() ? Actor->GetWorld()->GetWorldPartition() : nullptr;
				if (WorldPartition && WorldPartition->IsContentBundleEnabled())
				{
					const FText ActorContentBundleGuidText = FText::FromString(Actor->GetContentBundleGuid().ToString());
					ActorCategory.AddCustomRow( LOCTEXT("ContentBundleGuid", "ContentBundleGuid") )
						.NameContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ContentBundleGuid2", "Content Bundle Guid"))
							.ToolTipText(LOCTEXT("ActorContentBundleGuid_ToolTip", "Actor Content BundleGuid"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						.ValueContent()
						[
							SNew(STextBlock)
								.Text(ActorContentBundleGuidText)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.IsEnabled(false)
						];
				}
			}
		}
	};

#if 1
	// Create the info buttons per level
	for ( auto LevelIt( ActorsPerLevelCount.CreateConstIterator() ); LevelIt; ++LevelIt)
	{
		ULevel* Level = LevelIt.Key();
		int32 SelectedActorCountInLevel = LevelIt.Value();
		
		// Get a description of the level
		FText LevelDescription = FText::FromString( FPackageName::GetShortName( Level->GetOutermost()->GetFName() ) );
		if (Level == Level->OwningWorld->PersistentLevel)
		{
			LevelDescription = NSLOCTEXT("UnrealEd", "PersistentLevel", "Persistent Level");
		}

		// Create a description and tooltip for the actor count/selection hyperlink
		const FText ActorCountDescription = FText::Format( LOCTEXT("SelectedActorsInOneLevel", "{0} selected in"), FText::AsNumber( SelectedActorCountInLevel ) );

		const FText Tooltip = FText::Format( LOCTEXT("SelectedActorsHyperlinkTooltip", "Narrow the selection set to just the actors in {0}"), LevelDescription);

		// Create the row for this level
		TWeakObjectPtr<ULevel> WeakLevelPtr = Level;

		ActorCategory.AddCustomRow( LOCTEXT("SelectionFilter", "Selected") )
		.NameContent()
		[
			SNew(SHyperlink)
				.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
				.OnNavigate(this, &FActorDetails::OnNarrowSelectionSetToSpecificLevel, WeakLevelPtr)
				.Text(ActorCountDescription)
				.TextStyle(FAppStyle::Get(), "DetailsView.HyperlinkStyle")
				.ToolTipText(Tooltip)
		]
		.ValueContent()
		.MaxDesiredWidth(0.f)
		[
			SNew(STextBlock)
				.Text(LevelDescription)
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	}
#endif

	// Convert Actor Menu
	// WorldSettings should never convert to another class type
	if( SelectedActorInfo.SelectionClass != AWorldSettings::StaticClass() && SelectedActorInfo.HasConvertableAsset() )
	{
		ActorCategory.AddCustomRow( LOCTEXT("ConvertMenu", "Convert") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ConvertActor", "Convert Actor"))
			.ToolTipText(LOCTEXT("ConvertActor_ToolTip", "Convert actors to different types"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			MakeConvertMenu( SelectedActorInfo )
		];
	}
}

const TArray< TWeakObjectPtr<AActor> >& FActorDetails::GetSelectedActors() const
{
	return SelectedActors;
}

#undef LOCTEXT_NAMESPACE
