// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class IDetailLayoutBuilder;
class SWidget;
class UBlueprint;
class UClass;
class ULevel;
class UObject;
struct FSelectedActorInfo;
struct FSlateBrush;

class FActorDetails : public IDetailCustomization
{
public:
	virtual ~FActorDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
private:
	/** Delegate that returns an icon */
	DECLARE_DELEGATE_RetVal(const FSlateBrush*, FGetBrushImage);

	/** Delegate that returns a tooltip or other text field */
	DECLARE_DELEGATE_RetVal(FText, FGetText);

	/**
	 * Returns the conversion root class of the passed in class
	 *
	 * @param InCurrentClass		The class to find the conversion root of
	 *
	 * @return						Going up the class hierarchy, the first class found to have "IsConversionRoot" metadata (NULL if none is found)
	 */
	UClass* GetConversionRoot( UClass* InCurrentClass ) const;

	/**
	 * Callback from the Class Picker when a class is picked.
	 *
	 * @param ChosenClass		The class chosen to convert to
	 */
	void OnConvertActor(UClass* ChosenClass);

	/** 
	 * Creates the filters for displaying valid classes to convert to
	 *
	 * @param ConvertActor			The actor being converted
	 * @param ClassPickerOptions	Options for the Class Picker being generated, to be given a newly created filter
	 */
	void CreateClassPickerConvertActorFilter(const TWeakObjectPtr<AActor> ConvertActor, class FClassViewerInitializationOptions* ClassPickerOptions);

	/** Retrieves the content for the Convert combo button */
	TSharedRef<SWidget> OnGetConvertContent();

	/** Returns the visibility of the Convert menu */
	EVisibility GetConvertMenuVisibility() const;

	/** Create the Convert combo button */
	TSharedRef<SWidget> MakeConvertMenu( const FSelectedActorInfo& SelectedActorInfo );

	void OnNarrowSelectionSetToSpecificLevel( TWeakObjectPtr<ULevel> LevelToNarrowInto );

	bool IsActorValidForLevelScript() const;

	FReply FindSelectedActorsInLevelScript();

	bool AreAnySelectedActorsInLevelScript() const;

	/** Util to create a menu for events we can add for the selected actor */
	TSharedRef<SWidget> MakeEventOptionsWidgetFromSelection();

	void AddActorCategory( IDetailLayoutBuilder& DetailBuilder, const TMap<ULevel*, int32>& ActorsPerLevelCount );

	void AddBlueprintCategory( IDetailLayoutBuilder& DetailBuilder, const TMap<UBlueprint*, UObject*>& UniqueBlueprints );

	void AddLayersCategory( IDetailLayoutBuilder& DetailBuilder );

	void AddTransformCategory( IDetailLayoutBuilder& DetailBuilder );

	/** Display a category with all dynamic delegates on a CDO */
	void AddEventsCategory(IDetailLayoutBuilder& DetailBuilder);

	/** Handle the creation of a bound event from a dynamic delegate on a CDO */
	FReply HandleAddOrViewEventForVariable(UBlueprint* BP, class FMulticastDelegateProperty* Property);

	const TArray< TWeakObjectPtr<AActor> >& GetSelectedActors() const;

	// Functions for actor loading strategy details
private:

	/** Bring up the menu for user to select the path to create blueprint at */
	FReply OnPickBlueprintPathClicked(bool bHavest);

	/** Bring up the menu for user to select the path to create blueprint at */
	void   OnSelectBlueprintPath(const FString& );

	/** The path the user has selected to create a blueprint at*/
	FString PathForActorBlueprint;

	/** List of currently selected actors*/
	TArray< TWeakObjectPtr<AActor> > SelectedActors;
};
