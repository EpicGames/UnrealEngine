// Copyright Epic Games, Inc. All Rights Reserved.

#include "AmbientSoundDetails.h"

#include "Components/AudioComponent.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Sound/AmbientSound.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundNodeDelay.h"
#include "Sound/SoundNodeLooping.h"
#include "Sound/SoundNodeMixer.h"
#include "Sound/SoundNodeRandom.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class USoundBase;


#define LOCTEXT_NAMESPACE "AmbientSoundDetails"

TSharedRef<IDetailCustomization> FAmbientSoundDetails::MakeInstance()
{
	return MakeShareable( new FAmbientSoundDetails );
}

void FAmbientSoundDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();

	for( int32 ObjectIndex = 0; !AmbientSound.IsValid() && ObjectIndex < SelectedObjects.Num(); ++ObjectIndex )
	{
		const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[ObjectIndex];
		if ( CurrentObject.IsValid() )
		{
			AmbientSound = Cast<AAmbientSound>(CurrentObject.Get());
		}
	}

	DetailBuilder.EditCategory( "Sound", FText::GetEmpty(), ECategoryPriority::Important )
		.AddCustomRow( FText::GetEmpty() )
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding( 0.f, 2.0f, 0.f, 0.f)
			.FillHeight(1.0f)
			.VAlign( VAlign_Center )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.OnClicked( this, &FAmbientSoundDetails::OnEditSoundCueClicked )
						.IsEnabled( this, &FAmbientSoundDetails::IsEditSoundCueEnabled )
						.Text( LOCTEXT("EditAsset", "Edit") )
						.ToolTipText( LOCTEXT("EditAssetToolTip", "Edit this sound cue") )
					]
				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						// Add a menu for displaying all textures 
						SNew( SComboButton )
						.OnGetMenuContent( this, &FAmbientSoundDetails::OnGetSoundCueTemplates )
						.VAlign( VAlign_Center )
						.ContentPadding(2.f)
						.ButtonContent()
						[
							SNew( STextBlock )
							.ToolTipText( LOCTEXT("NewSoundCueToolTip", "Create a new sound cue with the desired template") )
							.Text( LOCTEXT("NewSoundCue", "New") )
						]
					]
				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.OnClicked( this, &FAmbientSoundDetails::OnPlaySoundClicked )
						.Text( LOCTEXT("PlaySoundCue", "Play") )
					]
				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.OnClicked( this, &FAmbientSoundDetails::OnStopSoundClicked )
						.Text( LOCTEXT("StopSoundCue", "Stop") )
					]
			]
		];

	DetailBuilder.EditCategory("Attenuation", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Modulation", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
}

bool FAmbientSoundDetails::IsEditSoundCueEnabled() const
{
	if (AmbientSound.IsValid())
	{
		// Only sound cues are editable
		USoundCue* SoundCue = Cast<USoundCue>(AmbientSound.Get()->GetAudioComponent()->Sound);
		return (SoundCue != NULL);
	}
	return false;
}

FReply FAmbientSoundDetails::OnEditSoundCueClicked()
{
	if( AmbientSound.IsValid() )
	{
		USoundCue* SoundCue = Cast<USoundCue>(AmbientSound.Get()->GetAudioComponent()->Sound);
		if (SoundCue)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SoundCue);
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> FAmbientSoundDetails::OnGetSoundCueTemplates()
{
	FMenuBuilder MenuBuilder( true, NULL );

	FUIAction EmptyAction( FExecuteAction::CreateSP( this, &FAmbientSoundDetails::CreateNewSoundCue, SOUNDCUE_LAYOUT_EMPTY ) );
	MenuBuilder.AddMenuEntry( LOCTEXT("SoundCue_EmptyLayout", "Blank"), LOCTEXT("SoundCue_EmptyLayout_Tooltip", "Create an empty SoundCue"), FSlateIcon(), EmptyAction );

	FUIAction MixerAction( FExecuteAction::CreateSP( this, &FAmbientSoundDetails::CreateNewSoundCue, SOUNDCUE_LAYOUT_MIXER ) );
	MenuBuilder.AddMenuEntry( LOCTEXT("SoundCue_MixerLayout", "Mixer"), LOCTEXT("SoundCue_MixerLayout_Tooltip", "Create a SoundCue with a Mixer"), FSlateIcon(), MixerAction );

	FUIAction RandomLoopAction( FExecuteAction::CreateSP( this, &FAmbientSoundDetails::CreateNewSoundCue, SOUNDCUE_LAYOUT_RANDOM_LOOP ) );
	MenuBuilder.AddMenuEntry( LOCTEXT("SoundCue_RandomLoopLayout", "Random Loop"), LOCTEXT("SoundCue_RandomLoopLayout_Tooltip", "Create a SoundCue with a Loop and a Random node"), FSlateIcon(), RandomLoopAction );

	FUIAction RandomLoopWithDelayAction( FExecuteAction::CreateSP( this, &FAmbientSoundDetails::CreateNewSoundCue, SOUNDCUE_LAYOUT_RANDOM_LOOP_WITH_DELAY ) );
	MenuBuilder.AddMenuEntry( LOCTEXT("SoundCue_RandomLoopWithDelayLayout", "Random Loop with Delay"), LOCTEXT("SoundCue_RandomLoopWithDelayLayout_Tooltip", "Create a SoundCue with a Loop, a Delay, and a Random node"), FSlateIcon(), RandomLoopWithDelayAction );

	return MenuBuilder.MakeWidget();
}

void FAmbientSoundDetails::CreateNewSoundCue( ESoundCueLayouts Layout )
{

	if( AmbientSound.IsValid() )
	{
		OnStopSoundClicked();

		AAmbientSound* AS = AmbientSound.Get();

		// First if the existing SoundCue is a child of the AmbientSound rename it off to oblivion so we can have a good name
		USoundCue* SoundCue = Cast<USoundCue>(AmbientSound->GetAudioComponent()->Sound);
		if (SoundCue && SoundCue->GetOuter() == AS)
		{
			SoundCue->Rename(*MakeUniqueObjectName( GetTransientPackage(), USoundCue::StaticClass() ).ToString(), GetTransientPackage(), REN_DontCreateRedirectors);
		}

		int32 NodeColumn = 0;
		USoundNode* PrevNode = NULL;

		SoundCue = NewObject<USoundCue>(AS, FName(*AS->GetInternalSoundCueName()));
		AS->GetAudioComponent()->Sound = SoundCue;
		AS->GetAudioComponent()->PostEditChange();

		if (Layout == SOUNDCUE_LAYOUT_RANDOM_LOOP || Layout == SOUNDCUE_LAYOUT_RANDOM_LOOP_WITH_DELAY)
		{
			USoundNodeLooping* LoopingNode = SoundCue->ConstructSoundNode<USoundNodeLooping>();
			LoopingNode->CreateStartingConnectors();
			LoopingNode->PlaceNode(NodeColumn++, 0, 1);
			
			if (!PrevNode)
			{
				SoundCue->FirstNode = LoopingNode;
			}
			else
			{
				check(PrevNode->ChildNodes.Num() > 0);
				PrevNode->ChildNodes[0] = LoopingNode;
			}
			PrevNode = LoopingNode;

			if (Layout == SOUNDCUE_LAYOUT_RANDOM_LOOP_WITH_DELAY)
			{
				USoundNodeDelay* DelayNode = SoundCue->ConstructSoundNode<USoundNodeDelay>();
				DelayNode->CreateStartingConnectors();
				DelayNode->PlaceNode(NodeColumn++,0,1);

				if (!PrevNode)
				{
					SoundCue->FirstNode = DelayNode;
				}
				else
				{
					check(PrevNode->ChildNodes.Num() > 0);
					PrevNode->ChildNodes[0] = DelayNode;
				}
				PrevNode = DelayNode;
			}

			USoundNodeRandom* RandomNode = SoundCue->ConstructSoundNode<USoundNodeRandom>();
			RandomNode->CreateStartingConnectors();
			RandomNode->PlaceNode(NodeColumn++,0,1);

			if (!PrevNode)
			{
				SoundCue->FirstNode = RandomNode;
			}
			else
			{
				check(PrevNode->ChildNodes.Num() > 0);
				PrevNode->ChildNodes[0] = RandomNode;
			}
			PrevNode = RandomNode;
		}
		else if (Layout == SOUNDCUE_LAYOUT_MIXER)
		{
			USoundNodeMixer* MixerNode = SoundCue->ConstructSoundNode<USoundNodeMixer>();
			MixerNode->CreateStartingConnectors();
			MixerNode->PlaceNode(NodeColumn++,0,1);

			if (!PrevNode)
			{
				SoundCue->FirstNode = MixerNode;
			}
			else
			{
				check(PrevNode->ChildNodes.Num() > 0);
				PrevNode->ChildNodes[0] = MixerNode;
			}
			PrevNode = MixerNode;
		}

		if (PrevNode)
		{
			const int32 ChildCount = PrevNode->ChildNodes.Num();
			for (int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
			{
				USoundNodeWavePlayer* PlayerNode = SoundCue->ConstructSoundNode<USoundNodeWavePlayer>();
				PlayerNode->PlaceNode(NodeColumn,ChildIndex,ChildCount);

				if (Layout == SOUNDCUE_LAYOUT_MIXER)
				{
					PlayerNode->bLooping = true;
				}

				PrevNode->ChildNodes[ChildIndex] = PlayerNode;
			}
		}

		SoundCue->LinkGraphNodesFromSoundNodes();

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SoundCue);
	}
}


FReply FAmbientSoundDetails::OnPlaySoundClicked()
{
	if( AmbientSound.IsValid() )
	{
		USoundBase* Sound = AmbientSound.Get()->GetAudioComponent()->Sound;
		if (Sound)
		{
			GEditor->PlayPreviewSound(Sound);
		}
	}

	return FReply::Handled();
}

FReply FAmbientSoundDetails::OnStopSoundClicked()
{
	GEditor->ResetPreviewAudioComponent();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
