// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class SWidget;

class FAmbientSoundDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	enum ESoundCueLayouts
	{
		SOUNDCUE_LAYOUT_EMPTY,
		SOUNDCUE_LAYOUT_MIXER,
		SOUNDCUE_LAYOUT_RANDOM_LOOP,
		SOUNDCUE_LAYOUT_RANDOM_LOOP_WITH_DELAY
	};

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	bool IsEditSoundCueEnabled() const;

	FReply OnEditSoundCueClicked();
	FReply OnPlaySoundClicked();
	FReply OnStopSoundClicked();

	TSharedRef<SWidget> OnGetSoundCueTemplates();

	void CreateNewSoundCue( ESoundCueLayouts Layout );

	TWeakObjectPtr<class AAmbientSound> AmbientSound;
};
