// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimMontageSegmentDetails.h"

#include "AnimPreviewInstance.h"
#include "Animation/AnimCompositeBase.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimationAsset.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/EditorAnimSegment.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "Components/SceneComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorComponents.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "EngineDefines.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SScrubControlPanel.h"
#include "SceneInterface.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "ShowFlags.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Field.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Viewports.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "AnimMontageSegmentDetails"

/////////////////////////////////////////////////////////////////////////
FAnimationSegmentViewportClient::FAnimationSegmentViewportClient(FAdvancedPreviewScene& InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget)
	: FEditorViewportClient(nullptr, &InPreviewScene, InEditorViewportWidget)
{
	SetViewMode(VMI_Lit);

	// Always composite editor objects after post processing in the editor
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.DisableAdvancedFeatures();
	
	UpdateLighting();

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(70, 70, 70);
	DrawHelper.GridColorMajor = FColor(40, 40, 40);
	DrawHelper.GridColorMinor =  FColor(20, 20, 20);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
}


void FAnimationSegmentViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);
	
	// Tick the preview scene world.
	if (PreviewScene)
	{
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
	}
}

void FAnimationSegmentViewportClient::UpdateLighting()
{
	const USkeletalMeshEditorSettings* Options = GetDefault<USkeletalMeshEditorSettings>();

	PreviewScene->SetLightDirection(Options->AnimPreviewLightingDirection);
	PreviewScene->SetLightColor(Options->AnimPreviewDirectionalColor);
	PreviewScene->SetLightBrightness(Options->AnimPreviewLightBrightness);
}


FSceneInterface* FAnimationSegmentViewportClient::GetScene() const
{
	return PreviewScene->GetScene();
}


/////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////


TSharedRef<IDetailCustomization> FAnimMontageSegmentDetails::MakeInstance()
{
	return MakeShareable( new FAnimMontageSegmentDetails );
}

void FAnimMontageSegmentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& SegmentCategory = DetailBuilder.EditCategory("Animation Segment", LOCTEXT("AnimationSegmentCategoryTitle", "Animation Segment") );

	AnimSegmentHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UEditorAnimSegment, AnimSegment));
	TSharedPtr<IPropertyHandle> TargetPropertyHandle = AnimSegmentHandle->GetChildHandle("AnimReference");
	FProperty* TargetProperty = TargetPropertyHandle->GetProperty();

	const FObjectPropertyBase* ObjectProperty = CastFieldChecked<const FObjectPropertyBase>(TargetProperty);

	IDetailPropertyRow& PropertyRow = SegmentCategory.AddProperty(TargetPropertyHandle);

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
	PropertyRow.OverrideResetToDefault(FResetToDefaultOverride::Hide());

	SAssignNew(ValueWidget, SObjectPropertyEntryBox)
		.PropertyHandle(TargetPropertyHandle)
		.AllowedClass(ObjectProperty->PropertyClass)
		.AllowClear(false)
		.OnObjectChanged(this, &FAnimMontageSegmentDetails::SetAnimationAsset)
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(this, &FAnimMontageSegmentDetails::OnShouldFilterAnimAsset));

	PropertyRow.CustomWidget()
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			ValueWidget.ToSharedRef()
		];

	TWeakPtr<FAnimMontageSegmentDetails> WeakDetails = TWeakPtr<FAnimMontageSegmentDetails>(StaticCastSharedRef<FAnimMontageSegmentDetails>(AsShared()));
	if (AnimSegmentHandle.IsValid())
	{
		AnimStartTimeProperty = AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimStartTime));

		IDetailPropertyRow& StartPropertyRow = SegmentCategory.AddProperty(AnimStartTimeProperty);
		StartPropertyRow.CustomWidget()
		.NameContent()
		[
			AnimStartTimeProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Font(DetailBuilder.GetDetailFont())
			.AllowSpin(true)
			.MinSliderValue(0.f)
			.MinValue(0.f)
			.MaxSliderValue(this, &FAnimMontageSegmentDetails::GetAnimationAssetPlayLength)
			.MaxValue(this, &FAnimMontageSegmentDetails::GetAnimationAssetPlayLength)
			.Value(this, &FAnimMontageSegmentDetails::GetStartTime)
			.OnValueChanged(this, &FAnimMontageSegmentDetails::OnStartTimeChanged, ETextCommit::Default, true)
			.OnValueCommitted(this, &FAnimMontageSegmentDetails::OnStartTimeChanged, false)
		];

		FResetToDefaultOverride Handler = FResetToDefaultOverride::Create
		(
			FIsResetToDefaultVisible::CreateLambda([this](TSharedPtr<IPropertyHandle> InAnimEndTimeProperty)
			{
				float Value = 0.f;
				if (InAnimEndTimeProperty->GetValue(Value) == FPropertyAccess::Success)
				{
					return !FMath::IsNearlyEqual(Value, GetAnimationAssetPlayLength().Get(Value));
				}

				return false;
			}),
			FResetToDefaultHandler::CreateLambda([this](TSharedPtr<IPropertyHandle> InAnimEndTimeProperty)
			{			
				InAnimEndTimeProperty->SetValue(GetAnimationAssetPlayLength().Get(0.f));
			})
		);
		
		AnimEndTimeProperty = AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimEndTime));
		SegmentCategory.AddProperty(AnimEndTimeProperty).CustomWidget()
		.NameContent()
		[
			AnimEndTimeProperty->CreatePropertyNameWidget()
		]
		.OverrideResetToDefault(Handler)
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Font(DetailBuilder.GetDetailFont())
			.AllowSpin(true)
			.MinSliderValue(0.f)
			.MinValue(0.f)
			.MaxSliderValue(this, &FAnimMontageSegmentDetails::GetAnimationAssetPlayLength)
			.MaxValue(this, &FAnimMontageSegmentDetails::GetAnimationAssetPlayLength)
			.Value(this, &FAnimMontageSegmentDetails::GetEndTime)
			.OnValueChanged(this, &FAnimMontageSegmentDetails::OnEndTimeChanged, ETextCommit::Default, true)
			.OnValueCommitted(this, &FAnimMontageSegmentDetails::OnEndTimeChanged, false)
		];

		const TSharedPtr<IPropertyHandle> AnimPlayRateProperty = AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, AnimPlayRate));
		
		IDetailPropertyRow& PlayRatePropertyRow = SegmentCategory.AddProperty(AnimPlayRateProperty);

		PlayRatePropertyRow.CustomWidget()
		.NameContent()
		[
			AnimPlayRateProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Font(DetailBuilder.GetDetailFont())
			.AllowSpin(true)
			.MinSliderValue(-1.f)
			.MinValue(-32.f)
			.MaxSliderValue(1.f)
			.MaxValue(32.f)
			.Value(this, &FAnimMontageSegmentDetails::GetPlayRate)
			.OnValueChanged_Lambda([AnimPlayRateProperty](float InValue)
			{
				if(AnimPlayRateProperty.IsValid() && !FMath::IsNearlyZero(InValue))
				{
					AnimPlayRateProperty->SetValue(InValue, EPropertyValueSetFlags::InteractiveChange);
				}
			})
			.OnValueCommitted_Lambda([AnimPlayRateProperty](float InValue, ETextCommit::Type InCommitType)
			{
                if(AnimPlayRateProperty.IsValid() && !FMath::IsNearlyZero(InValue))
                {
					AnimPlayRateProperty->SetValue(InValue);
                }
			})
		];

		const TSharedPtr<IPropertyHandle> LoopingCountProperty = AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, LoopingCount));
		
		IDetailPropertyRow& LoopingPropertyRow = SegmentCategory.AddProperty(LoopingCountProperty);

		LoopingPropertyRow.CustomWidget()
		.NameContent()
		[
			LoopingCountProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<int32>)
			.Font(DetailBuilder.GetDetailFont())
			.AllowSpin(true)
			.MinSliderValue(1)
			.MinValue(1)
			.MaxSliderValue(4)
			.MaxValue(32)
			.Value_Lambda([WeakDetails]
			{
				if (const TSharedPtr<FAnimMontageSegmentDetails> SegmentDetails = WeakDetails.Pin())
				{
					if(const FAnimSegment* AnimSegmentPtr = SegmentDetails->GetAnimationSegment())
					{
						return AnimSegmentPtr->LoopingCount;
					}
				}
				return 0;
			})
			.OnValueChanged_Lambda([LoopingCountProperty](int32 InValue)
			{
				if(LoopingCountProperty.IsValid() && InValue != 0)
				{
					LoopingCountProperty->SetValue(InValue, EPropertyValueSetFlags::InteractiveChange);
				}
			})
			.OnValueCommitted_Lambda([LoopingCountProperty](int32 InValue, ETextCommit::Type InCommitType)
			{
				if(LoopingCountProperty.IsValid() && InValue != 0)
				{
					LoopingCountProperty->SetValue(InValue);
				}
			})
		];
	}

	SegmentCategory.AddProperty(AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, StartPos)));

	const UAnimSequenceBase* AnimRef = GetAnimationAsset();
	const USkeleton* Skeleton = AnimRef ? AnimRef->GetSkeleton() : nullptr;

	SegmentCategory.AddCustomRow(FText::GetEmpty(), false)
	[
		SNew(SAnimationSegmentViewport)
		.AnimRef(this, &FAnimMontageSegmentDetails::GetAnimationAsset)
		.StartTime(this, &FAnimMontageSegmentDetails::GetStartTime)
		.EndTime(this, &FAnimMontageSegmentDetails::GetEndTime)
		.PlayRate(this, &FAnimMontageSegmentDetails::GetPlayRate)
		.OnStartTimeChanged_Lambda([WeakDetails](float InValue, bool bInteractive)
		{
			if (const TSharedPtr<FAnimMontageSegmentDetails> SegmentDetails = WeakDetails.Pin())
			{
				SegmentDetails->OnStartTimeChanged(InValue, ETextCommit::Default, bInteractive);
			}
		})
		.OnEndTimeChanged_Lambda([WeakDetails](float InValue, bool bInteractive)
		{
			if (const TSharedPtr<FAnimMontageSegmentDetails> SegmentDetails = WeakDetails.Pin())
			{
				SegmentDetails->OnEndTimeChanged(InValue, ETextCommit::Default, bInteractive);
			}
		})
	];
	
	const TSharedPtr<IPropertyHandle> PlayLengthHandle = AnimSegmentHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimSegment, CachedPlayLength));
	PlayLengthHandle->MarkHiddenByCustomization();
	IDetailCategoryBuilder& WarningCategory = DetailBuilder.EditCategory(TEXT("Warning"), LOCTEXT("WarningCategoryDisplayName", "Warning"), ECategoryPriority::Transform);
	WarningCategory.AddCustomRow(LOCTEXT("WarningCategoryDisplayName", "Warning"))
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(FMargin(0.f, 4.f))
		[
			SNew(SWarningOrErrorBox)
			.MessageStyle(EMessageStyle::Warning)
			.Message_Lambda([this]() -> FText
			{
				const FAnimSegment* Segment = GetAnimationSegment();
				static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
					.SetMinimumFractionalDigits(2)
					.SetMaximumFractionalDigits(2);
				
				return FText::Format(LOCTEXT("AnimSegmentLengthMismatchWarning", "Referenced Animation length has changed and mismatches with set Segment length. Animation length: {0}. Segment length: {1}."), FText::AsNumber(Segment->GetAnimReference()->GetPlayLength(), &FormatOptions), FText::AsNumber(Segment->CachedPlayLength, &FormatOptions));
			})
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked_Lambda([this, PlayLengthHandle]()
					{						
						if (PlayLengthHandle.IsValid())
						{
							if (const FAnimSegment* AnimSegment = GetAnimationSegment())
							{									
								if(const IAnimationDataModel* DataModel = AnimSegment->GetAnimReference()->GetDataModel())
								{
									const float NewPlayLength = static_cast<float>(DataModel->GetPlayLength());
									PlayLengthHandle->SetValue(NewPlayLength);
								}
							}
						}
							
						return FReply::Handled();
					})
					.TextStyle(FAppStyle::Get(), "NormalText")
					.Text(LOCTEXT("KeepSegmentLengthButtonText", "Keep Segment Length"))
					.ToolTipText(LOCTEXT("KeepSegmentLengthButtonToolTip", "This will update the cached Animation Asset length with its current value while keeping the Segment Length the same."))
					.HAlign(EHorizontalAlignment::HAlign_Center)
				]
				+SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked_Lambda([this]()
					{						
						if (AnimEndTimeProperty.IsValid())
						{
							if (const FAnimSegment* AnimSegment = GetAnimationSegment())
							{
								if(const IAnimationDataModel* DataModel = AnimSegment->GetAnimReference()->GetDataModel())
								{
									const float NewPlayLength = static_cast<float>(DataModel->GetPlayLength());
									AnimEndTimeProperty->SetValue(NewPlayLength);
								}
							}
						}
							
						return FReply::Handled();
					})
					.TextStyle(FAppStyle::Get(), "NormalText")
					.Text(LOCTEXT("SyncWithAnimLengthButtonText", "Sync with Animation Length"))
					.ToolTipText(LOCTEXT("SyncWithAnimLengthButtonToolTip", "This will update the Segment Length to match the Animation Asset length its current value."))
					.HAlign(EHorizontalAlignment::HAlign_Center)
				]
			]
		]
	]
	.Visibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility
	{
		if (GetAnimationSegment() && GetAnimationSegment()->IsPlayLengthOutOfDate())
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	}));	
}

bool FAnimMontageSegmentDetails::OnShouldFilterAnimAsset(const FAssetData& AssetData) const
{
	return AssetData.GetClass() == UAnimMontage::StaticClass();
}

const UAnimSequenceBase* FAnimMontageSegmentDetails::GetAnimationAsset() const
{
	if (const FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		return AnimSegment->GetAnimReference().Get();
	}

	return nullptr;
}

void FAnimMontageSegmentDetails::SetAnimationAsset(const FAssetData& InAssetData)
{
	if (FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(InAssetData.GetAsset());
		AnimSegment->SetAnimReference(AnimSequenceBase);
	}
}

void FAnimMontageSegmentDetails::OnStartTimeChanged(float InValue, ETextCommit::Type InCommitType, bool bInteractive)
{
	if (FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		const float ValueToSet = FMath::Min(AnimSegment->AnimEndTime, InValue);
		AnimStartTimeProperty->SetValue(ValueToSet, bInteractive ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags);			
	}
}

TOptional<float> FAnimMontageSegmentDetails::GetStartTime() const
{
	TOptional<float> TimeValue;
	
	if (const FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		TimeValue = AnimSegment->AnimStartTime;
	}

	return TimeValue;
}

void FAnimMontageSegmentDetails::OnEndTimeChanged(float InValue, ETextCommit::Type InCommitType, bool bInteractive)
{
	if (FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		const float ValueToSet = FMath::Max(AnimSegment->AnimStartTime, InValue);
		AnimEndTimeProperty->SetValue(ValueToSet, bInteractive ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags);		
	}
}

TOptional<float> FAnimMontageSegmentDetails::GetEndTime() const
{
	TOptional<float> TimeValue;
	if (const FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		TimeValue = AnimSegment->AnimEndTime;
	}

	return TimeValue;
}

TOptional<float> FAnimMontageSegmentDetails::GetPlayRate() const
{
	TOptional<float> Value;
	if (const FAnimSegment* AnimSegment = GetAnimationSegment())
	{
		Value = AnimSegment->AnimPlayRate;
	}

	return Value;
}

FAnimSegment* FAnimMontageSegmentDetails::GetAnimationSegment() const
{
	if (AnimSegmentHandle.IsValid())
	{
		void* Data;
		const FPropertyAccess::Result Result = AnimSegmentHandle->GetValueData(Data);
		if (Result == FPropertyAccess::MultipleValues)
		{
			return nullptr;
		}

		if (Result == FPropertyAccess::Success)
		{
			FAnimSegment* AnimSegment = reinterpret_cast<FAnimSegment*>(Data);
			return AnimSegment;
		}
	}

	return nullptr;
}

bool FAnimMontageSegmentDetails::CanEditSegmentProperties() const
{
	return true;
}

TOptional<float> FAnimMontageSegmentDetails::GetAnimationAssetPlayLength() const
{
	TOptional<float> Value;
	
	if (const UAnimSequenceBase* SequenceBase = GetAnimationAsset())
	{
		Value = SequenceBase->GetPlayLength();
	}

	return Value;
}

/////////////////////////////////////////////////
////////////////////////////////////////////////

SAnimationSegmentViewport::~SAnimationSegmentViewport()
{
	// clean up components
	if (PreviewComponent)
	{
		for (int32 I=PreviewComponent->GetAttachChildren().Num()-1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			// PreviewComponet will be cleaned up by PreviewScene, 
			// but if anything is attached, it won't be cleaned up, 
			// so we'll need to clean them up manually
			CleanupComponent(PreviewComponent->GetAttachChildren()[I]);
		}
		check(PreviewComponent->GetAttachChildren().Num() == 0);
	}
}

void SAnimationSegmentViewport::CleanupComponent(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 I = Component->GetAttachChildren().Num() - 1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			CleanupComponent(Component->GetAttachChildren()[I]);
		}
		check(Component->GetAttachChildren().Num() == 0);

		Component->DestroyComponent();
	}
}

SAnimationSegmentViewport::SAnimationSegmentViewport()
	: CurrentAnimSequenceBase(nullptr), AdvancedPreviewScene(FPreviewScene::ConstructionValues())
{
}

void SAnimationSegmentViewport::Construct(const FArguments& InArgs)
{
	AnimationRefAttribute = InArgs._AnimRef;
	StartTimeAttribute = InArgs._StartTime;
	EndTimeAttribute = InArgs._EndTime;
	PlayRateAttribute = InArgs._PlayRate;
	OnStartTimeChanged = InArgs._OnStartTimeChanged;
	OnEndTimeChanged = InArgs._OnEndTimeChanged;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SAssignNew(Description, STextBlock)
			.Text(LOCTEXT("DefaultViewportLabel", "Default View"))
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]

		+SVerticalBox::Slot()
		.FillHeight(1)
		.HAlign(HAlign_Center)
		[
			SNew( SBorder )
			.HAlign(HAlign_Center)
			[
				SAssignNew( ViewportWidget, SViewport )
				.EnableGammaCorrection( false )
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[			
			SNew(SAnimationSegmentScrubPanel)
			.ViewInputMin(this, &SAnimationSegmentViewport::GetViewMinInput)
			.ViewInputMax(this, &SAnimationSegmentViewport::GetViewMaxInput)
			.PreviewInstance(this, &SAnimationSegmentViewport::GetPreviewInstance)
			.DraggableBars(this, &SAnimationSegmentViewport::GetBars)
			.OnBarDrag(this, &SAnimationSegmentViewport::OnBarDrag, true)
			.OnBarCommit(this, &SAnimationSegmentViewport::OnBarDrag, false)
			.bAllowZoom(true)
		]
	];
	

	// Create a viewport client
	LevelViewportClient	= MakeShareable( new FAnimationSegmentViewportClient(AdvancedPreviewScene) );

	LevelViewportClient->ViewportType = LVT_Perspective;
	LevelViewportClient->bSetListenerPosition = false;
	LevelViewportClient->SetViewLocation( EditorViewportDefs::DefaultPerspectiveViewLocation );
	LevelViewportClient->SetViewRotation( EditorViewportDefs::DefaultPerspectiveViewRotation );	

	SceneViewport = FSceneViewport::Create(LevelViewportClient, ViewportWidget);
	LevelViewportClient->SetRealtime( true );
	LevelViewportClient->VisibilityDelegate.BindSP( this, &SAnimationSegmentViewport::IsVisible );
	LevelViewportClient->SetViewMode( VMI_Lit );
	
	PreviewComponent = NewObject<UDebugSkelMeshComponent>();
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	AdvancedPreviewScene.AddComponent(PreviewComponent, FTransform::Identity);

	InitSkeleton();
}

void SAnimationSegmentViewport::InitSkeleton()
{
	UAnimSequenceBase* AnimSequenceBase = const_cast<UAnimSequenceBase*>(AnimationRefAttribute.Get());
	if (PreviewComponent && AnimSequenceBase && AnimSequenceBase != CurrentAnimSequenceBase)
	{
		USkeleton* Skeleton = AnimSequenceBase->GetSkeleton();
		USkeletalMesh* PreviewMesh = Skeleton ? Skeleton->GetAssetPreviewMesh(AnimSequenceBase) : nullptr;
		if (Skeleton && PreviewMesh)
		{
			UAnimSingleNodeInstance* Preview = PreviewComponent->PreviewInstance;
			if((Preview == nullptr || Preview->GetCurrentAsset() != AnimSequenceBase) || (PreviewComponent->GetSkeletalMeshAsset() != PreviewMesh))
			{
				const float PlayRate = PlayRateAttribute.Get().Get(1.f);

				PreviewComponent->SetSkeletalMesh(PreviewMesh);
				PreviewComponent->EnablePreview(true, AnimSequenceBase);
				PreviewComponent->PreviewInstance->SetLooping(true);
				PreviewComponent->SetPlayRate(PlayRate);

				//Place the camera at a good viewer position
				const FVector NewPosition = LevelViewportClient->GetViewLocation().GetSafeNormal();
				LevelViewportClient->SetViewLocation(NewPosition * (PreviewMesh->GetImportedBounds().SphereRadius*1.5f));

				CurrentAnimSequenceBase = AnimSequenceBase;
			}
		}
	}
}

void SAnimationSegmentViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	class UDebugSkelMeshComponent* Component = PreviewComponent;
	const FString TargetSkeletonName = CurrentAnimSequenceBase && CurrentAnimSequenceBase->GetSkeleton() ? CurrentAnimSequenceBase->GetSkeleton()->GetName() : TEXT("None");
	if (Component != nullptr)
	{
		// Reinit the skeleton if the anim ref has changed
		InitSkeleton();

		const float Start = StartTimeAttribute.Get().Get(0.f);
		const float End = EndTimeAttribute.Get().Get(0.f);
		const float PlayRate = PlayRateAttribute.Get().Get(0.f);

		if (Component->PreviewInstance && (Component->PreviewInstance->GetCurrentTime() > End || Component->PreviewInstance->GetCurrentTime() < Start))
		{
			const float NewStart = PlayRate > 0.f ? Start : End;
			Component->PreviewInstance->SetPosition(NewStart, false);
		}

		Component->SetPlayRate(PlayRate);

		if (Component->IsPreviewOn())
		{
			Description->SetText(FText::Format( LOCTEXT("Previewing", "Previewing {0}"), FText::FromString(Component->GetPreviewText()) ));
		}
		else if (Component->AnimClass)
		{
			Description->SetText(FText::Format( LOCTEXT("Previewing", "Previewing {0}"), FText::FromString(Component->AnimClass->GetName()) ));
		}
		else if (Component->GetSkeletalMeshAsset() == NULL)
		{
			Description->SetText(FText::Format( LOCTEXT("NoMeshFound", "No skeletal mesh found for skeleton '{0}'"), FText::FromString(TargetSkeletonName) ));
		}
		else
		{
			Description->SetText(LOCTEXT("Default", "Default"));
		}
	}
	else
	{
		Description->SetText(FText::Format( LOCTEXT("NoMeshFound", "No skeletal mesh found for skeleton '{0}'"), FText::FromString(TargetSkeletonName) ));
	}
}

void SAnimationSegmentViewport::RefreshViewport()
{
}

bool SAnimationSegmentViewport::IsVisible() const
{
	return ViewportWidget.IsValid();
}

float SAnimationSegmentViewport::GetViewMinInput() const 
{ 
	if (PreviewComponent != NULL)
	{
		if (PreviewComponent->PreviewInstance != NULL)
		{
			return 0.0f;
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return FMath::Max(static_cast<float>(PreviewComponent->GetAnimInstance()->LifeTimer - 30.0), 0.0f);
		}
	}

	return 0.f; 
}

float SAnimationSegmentViewport::GetViewMaxInput() const 
{ 
	if (PreviewComponent != NULL)
	{
		if (PreviewComponent->PreviewInstance != NULL)
		{
			return PreviewComponent->PreviewInstance->GetLength();
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return static_cast<float>(PreviewComponent->GetAnimInstance()->LifeTimer);
		}
	}

	return 0.f;
}

TArray<float> SAnimationSegmentViewport::GetBars() const
{
	const float Start = StartTimeAttribute.Get().Get(0.f);
	const float End = EndTimeAttribute.Get().Get(0.f);

	TArray<float> Bars;
	Bars.Add(Start);
	Bars.Add(End);

	return Bars;
}

void SAnimationSegmentViewport::OnBarDrag(int32 Index, float Position, bool bInteractive)
{
	if(Index==0)
	{
		OnStartTimeChanged.ExecuteIfBound(Position, bInteractive);
	}
	else if(Index==1)
	{
		OnEndTimeChanged.ExecuteIfBound(Position, bInteractive);
	}
}

UAnimSingleNodeInstance* SAnimationSegmentViewport::GetPreviewInstance() const
{
	return PreviewComponent ? PreviewComponent->PreviewInstance : NULL;
}

/////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////
/////////////////////////////////////////////////////////////////////////
void SAnimationSegmentScrubPanel::Construct( const SAnimationSegmentScrubPanel::FArguments& InArgs )
{
	bSliderBeingDragged = false;
	PreviewInstance = InArgs._PreviewInstance;
	LockedSequence = InArgs._LockedSequence;

	this->ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill) 
			.VAlign(VAlign_Center)
			.FillWidth(1)
			.Padding( FMargin( 3.0f, 0.0f) )
			[
				SAssignNew(ScrubControlPanel, SScrubControlPanel)
				.IsEnabled(true)
				.Value(this, &SAnimationSegmentScrubPanel::GetScrubValue)
				.NumOfKeys(this, &SAnimationSegmentScrubPanel::GetNumOfFrames)
				.SequenceLength(this, &SAnimationSegmentScrubPanel::GetSequenceLength)
				.OnValueChanged(this, &SAnimationSegmentScrubPanel::OnValueChanged)
				.OnBeginSliderMovement(this, &SAnimationSegmentScrubPanel::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &SAnimationSegmentScrubPanel::OnEndSliderMovement)
				.OnClickedForwardPlay(this, &SAnimationSegmentScrubPanel::OnClick_Forward)
				.OnGetPlaybackMode(this, &SAnimationSegmentScrubPanel::GetPlaybackMode)
				.ViewInputMin(InArgs._ViewInputMin)
				.ViewInputMax(InArgs._ViewInputMax)
				.bAllowZoom(InArgs._bAllowZoom)
				.IsRealtimeStreamingMode(this, &SAnimationSegmentScrubPanel::IsRealtimeStreamingMode)
				.DraggableBars(InArgs._DraggableBars)
				.OnBarDrag(InArgs._OnBarDrag)
				.OnBarCommit(InArgs._OnBarCommit)
				.OnTickPlayback(InArgs._OnTickPlayback)
			]
		];

}

SAnimationSegmentScrubPanel::~SAnimationSegmentScrubPanel()
{
	
}

FReply SAnimationSegmentScrubPanel::OnClick_Forward()
{
	UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance();
	if (PreviewInst)
	{
		bool bIsReverse = PreviewInst->IsReverse();
		bool bIsPlaying = PreviewInst->IsPlaying();
		// if current bIsReverse and bIsPlaying, we'd like to just turn off reverse
		if (bIsReverse && bIsPlaying)
		{
			PreviewInst->SetReverse(false);
		}
		// already playing, simply pause
		else if (bIsPlaying) 
		{
			PreviewInst->SetPlaying(false);
		}
		// if not playing, play forward
		else 
		{
			PreviewInst->SetReverse(false);
			PreviewInst->SetPlaying(true);
		}
	}

	return FReply::Handled();
}

EPlaybackMode::Type SAnimationSegmentScrubPanel::GetPlaybackMode() const
{
	UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance();
	if (PreviewInst && PreviewInst->IsPlaying())
	{
		return PreviewInst->IsReverse() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
	}
	return EPlaybackMode::Stopped;
}

bool SAnimationSegmentScrubPanel::IsRealtimeStreamingMode() const
{
	return GetPreviewInstance() == NULL;
}

void SAnimationSegmentScrubPanel::OnValueChanged(float NewValue)
{
	if (UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance())
	{
		PreviewInst->SetPosition(NewValue);
	}
}

// make sure viewport is freshes
void SAnimationSegmentScrubPanel::OnBeginSliderMovement()
{
	bSliderBeingDragged = true;

	if (UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance())
	{
		PreviewInst->SetPlaying(false);
	}
}

void SAnimationSegmentScrubPanel::OnEndSliderMovement(float NewValue)
{
	bSliderBeingDragged = false;
}

void SAnimationSegmentScrubPanel::AnimChanged(UAnimationAsset* AnimAsset)
{
}

uint32 SAnimationSegmentScrubPanel::GetNumOfFrames() const
{
	if (DoesSyncViewport())
	{
		UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance();
		float Length = PreviewInst->GetLength();
		// if anim sequence, use correct number of keys
		int32 NumKeys = (int32) (Length/0.0333f); 
		if (PreviewInst->GetCurrentAsset() && PreviewInst->GetCurrentAsset()->IsA(UAnimSequenceBase::StaticClass()))
		{
			NumKeys = CastChecked<UAnimSequenceBase>(PreviewInst->GetCurrentAsset())->GetNumberOfSampledKeys();
		}
		return NumKeys;
	}
	else if (LockedSequence)
	{
		return LockedSequence->GetNumberOfSampledKeys();
	}
	return 1;
}

float SAnimationSegmentScrubPanel::GetSequenceLength() const
{
	if (DoesSyncViewport())
	{
		return GetPreviewInstance()->GetLength();
	}
	else if (LockedSequence)
	{
		return LockedSequence->GetPlayLength();
	}
	return 0.f;
}

bool SAnimationSegmentScrubPanel::DoesSyncViewport() const
{
	UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance();

	return (( LockedSequence==NULL && PreviewInst ) || ( LockedSequence && PreviewInst && PreviewInst->GetCurrentAsset() == LockedSequence ));
}

class UAnimSingleNodeInstance* SAnimationSegmentScrubPanel::GetPreviewInstance() const
{
	return PreviewInstance.Get();
}

float SAnimationSegmentScrubPanel::GetScrubValue() const
{
	if (DoesSyncViewport())
	{
		UAnimSingleNodeInstance* PreviewInst = GetPreviewInstance();
		if (PreviewInst)
		{
			return PreviewInst->GetCurrentTime();
		}
	}
	return 0.f;
}

void SAnimationSegmentScrubPanel::ReplaceLockedSequence(class UAnimSequenceBase* NewLockedSequence)
{
	LockedSequence = NewLockedSequence;
}

#undef LOCTEXT_NAMESPACE
