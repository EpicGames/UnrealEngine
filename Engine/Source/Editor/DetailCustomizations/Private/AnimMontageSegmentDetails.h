// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EditorViewportClient.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "ITransportControl.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PreviewScene.h"
#include "SScrubWidget.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FSceneInterface;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditorViewport;
class STextBlock;
class SViewport;
class UAnimSequenceBase;
class UAnimationAsset;
class USceneComponent;
struct FAnimSegment;
struct FAssetData;
struct FGeometry;

class FAnimMontageSegmentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
protected:
	bool OnShouldFilterAnimAsset(const FAssetData& AssetData) const;
	
	const UAnimSequenceBase* GetAnimationAsset() const;
	FAnimSegment* GetAnimationSegment() const;
	bool CanEditSegmentProperties() const;

	void SetAnimationAsset(const FAssetData& InAssetData);
	void OnStartTimeChanged(float InValue, ETextCommit::Type InCommitType, bool bInteractive = false);	
	void OnEndTimeChanged(float InValue, ETextCommit::Type InCommitType, bool bInteractive = false);

	TOptional<float> GetStartTime() const;
	TOptional<float> GetEndTime() const;	
	TOptional<float> GetAnimationAssetPlayLength() const;
	TOptional<float> GetPlayRate() const;

protected:
	TSharedPtr<IPropertyHandle> AnimSegmentHandle;
	TSharedPtr<IPropertyHandle> AnimationAssetHandle;
	TSharedPtr<IPropertyHandle> AnimStartTimeProperty;
	TSharedPtr<IPropertyHandle> AnimEndTimeProperty;
};


///////////////////////////////////////////////////
class SAnimationSegmentViewport : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnValueChanged, float, bool);
	
	SLATE_BEGIN_ARGS( SAnimationSegmentViewport )
	{}
		SLATE_ATTRIBUTE(const UAnimSequenceBase*, AnimRef)
		SLATE_ATTRIBUTE(TOptional<float>, StartTime)
		SLATE_ATTRIBUTE(TOptional<float>, EndTime)
		SLATE_ATTRIBUTE(TOptional<float>, PlayRate)
		SLATE_EVENT(FOnValueChanged, OnStartTimeChanged)
		SLATE_EVENT(FOnValueChanged, OnEndTimeChanged)
	SLATE_END_ARGS()
	
public:
	SAnimationSegmentViewport();
	virtual ~SAnimationSegmentViewport();

	void Construct(const FArguments& InArgs);

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	void RefreshViewport();

private:
	void InitSkeleton();

	TSharedPtr<FEditorViewportClient> LevelViewportClient;

	TAttribute<const UAnimSequenceBase*> AnimationRefAttribute;	
	TAttribute<TOptional<float>> StartTimeAttribute;
	TAttribute<TOptional<float>> EndTimeAttribute;
	TAttribute<TOptional<float>> PlayRateAttribute;
	FOnValueChanged OnStartTimeChanged;
	FOnValueChanged OnEndTimeChanged;
	
	/** Slate viewport for rendering and I/O */
	//TSharedPtr<class FSceneViewport> Viewport;
	TSharedPtr<SViewport> ViewportWidget;

	TSharedPtr<class FSceneViewport> SceneViewport;

	TObjectPtr<UAnimSequenceBase> CurrentAnimSequenceBase;
	FAdvancedPreviewScene AdvancedPreviewScene;
	class UDebugSkelMeshComponent* PreviewComponent;

	TSharedPtr<STextBlock> Description;

	void CleanupComponent(USceneComponent* Component);
	bool IsVisible() const;

public:

	/** Get Min/Max Input of value **/
	float GetViewMinInput() const;
	float GetViewMaxInput() const;
	
	class UAnimSingleNodeInstance*	GetPreviewInstance() const;

	/** Optional, additional values to draw on the timeline **/
	TArray<float> GetBars() const;
	void OnBarDrag(int32 index, float newPos, bool bInteractive);
	

};

///////////////////////////////////////////////////
/** This is a slimmed down version of SAnimationScrubPanel and has no ties to persona. It would be best to have these inherit from a more
  *	generic base class so some functionality could be shared.
  */
class SAnimationSegmentScrubPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAnimationSegmentScrubPanel)
		: _LockedSequence()
	{}
		/** If you'd like to lock to one asset for this scrub control, give this**/
		SLATE_ARGUMENT(UAnimSequenceBase*, LockedSequence)
		SLATE_ATTRIBUTE(class UAnimSingleNodeInstance*, PreviewInstance)
		/** View Input range **/
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_ARGUMENT( bool, bAllowZoom )
		SLATE_ATTRIBUTE( TArray<float>, DraggableBars )
		SLATE_EVENT( FOnScrubBarDrag, OnBarDrag )
		SLATE_EVENT( FOnScrubBarCommit, OnBarCommit )
		SLATE_EVENT( FOnTickPlayback, OnTickPlayback )
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs );
	virtual ~SAnimationSegmentScrubPanel();

	void ReplaceLockedSequence(class UAnimSequenceBase * NewLockedSequence);
protected:

	bool bSliderBeingDragged;	
	FReply OnClick_Forward();
	

	TAttribute<class UAnimSingleNodeInstance*>	PreviewInstance;
	
	void AnimChanged(UAnimationAsset * AnimAsset);

	void OnValueChanged(float NewValue);
	// make sure viewport is freshes
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue);

	
	EPlaybackMode::Type GetPlaybackMode() const;
	bool IsRealtimeStreamingMode() const;

	float GetScrubValue() const;
	class UAnimSingleNodeInstance* GetPreviewInstance() const;

	TSharedPtr<class SScrubControlPanel> ScrubControlPanel;
	class UAnimSequenceBase* LockedSequence;

	/** Do I need to sync with viewport? **/
	bool DoesSyncViewport() const;
	uint32 GetNumOfFrames() const;
	float GetSequenceLength() const;
};

// FAnimationSegmentViewportClient
class FAnimationSegmentViewportClient : public FEditorViewportClient
{
public:
	FAnimationSegmentViewportClient(FAdvancedPreviewScene& InPreviewScene, const TWeakPtr<SEditorViewport>& InEditorViewportWidget = nullptr);

	// FlEditorViewportClient interface
	virtual FSceneInterface* GetScene() const override;
	virtual FLinearColor GetBackgroundColor() const override { return FLinearColor::Black; }
	virtual void Tick(float DeltaSeconds) override;
	// End of FEditorViewportClient

	void UpdateLighting();
};
