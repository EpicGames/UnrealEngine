// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceDetails.h"

#include "AnimMontageSegmentDetails.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimEnums.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimTypes.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorViewportClient.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "PreviewScene.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "SceneInterface.h"
#include "Slate/SceneViewport.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Viewports.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "CompressedAnimationDataNodeBuilder.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE	"AnimSequenceDetails"

// default name for retarget source
#define DEFAULT_RETARGET_SOURCE_NAME	TEXT("Default")

TSharedRef<IDetailCustomization> FAnimSequenceDetails::MakeInstance()
{
	return MakeShareable(new FAnimSequenceDetails);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FAnimSequenceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	/////////////////////////////////////////////////////////////////////////////////
	// Animation
	/////////////////////////////////////////////////////////////////////////////////
	IDetailCategoryBuilder& AnimationCategory = DetailBuilder.EditCategory("Animation");

	// *** Retarget source handler ***
	RetargetSourceNameHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSource));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAssetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSourceAsset));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// first create profile combo list
	RetargetSourceComboList.Empty();
	// first one is default one
	RetargetSourceComboList.Add( MakeShareable( new FString (DEFAULT_RETARGET_SOURCE_NAME) ));

	// find skeleton
	TSharedPtr<IPropertyHandle> SkeletonHanlder = DetailBuilder.GetProperty(TEXT("Skeleton"));
	FName CurrentPoseName;
	ensure (RetargetSourceNameHandler->GetValue(CurrentPoseName) != FPropertyAccess::Fail);
	
	// Check if we use only one skeleton
	USkeleton* Skeleton = NULL;
	SelectedAnimSequences.Reset();
	TArray< TWeakObjectPtr<UObject> > SelectedObjectsList = DetailBuilder.GetSelectedObjects();
	for (auto SelectionIt = SelectedObjectsList.CreateIterator(); SelectionIt; ++SelectionIt)
	{
		if (UAnimSequence* TestAnimSequence = Cast<UAnimSequence>(SelectionIt->Get()))
		{
			SelectedAnimSequences.Add(TestAnimSequence);
		}
	}

	// do it in separate loop since before it only cared AnimSequence 
	for (auto& It : SelectedAnimSequences)
	{
		// we should only have one selected anim sequence
		if(Skeleton && Skeleton != It->GetSkeleton())
		{
			// Multiple different skeletons
			Skeleton = NULL;
			break;
		}
		Skeleton = It->GetSkeleton();
	}

	// set target skeleton. It can be null
	TargetSkeleton = Skeleton;

	// find what is initial selection is
	TSharedPtr<FString> InitialSelected;
	if (TargetSkeleton.IsValid())
	{
		RegisterRetargetSourceChanged();

		// Add each retarget source
		TArray<FName> RetargetSources;
		TargetSkeleton->GetRetargetSources(RetargetSources);
		
		// go through profile and see if it has mine
		for (FName& RetargetSource : RetargetSources)
		{
			RetargetSourceComboList.Add( MakeShareable( new FString ( RetargetSource.ToString() )));

			if (RetargetSource == CurrentPoseName) 
			{
				InitialSelected = RetargetSourceComboList.Last();
			}
		}
	}

	// add widget for editing retarget source
	AnimationCategory
	.AddCustomRow(RetargetSourceNameHandler->GetPropertyDisplayName())
	.RowTag(RetargetSourceNameHandler->GetProperty()->GetFName())
	.NameContent()
	[
		RetargetSourceNameHandler->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(RetargetSourceComboBox, SSearchableComboBox)
		.OptionsSource(&RetargetSourceComboList)
		.OnGenerateWidget(this, &FAnimSequenceDetails::MakeRetargetSourceComboWidget)
		.OnSelectionChanged(this, &FAnimSequenceDetails::OnRetargetSourceChanged)
		.OnComboBoxOpening(this, &FAnimSequenceDetails::OnRetargetSourceComboOpening)
		.InitiallySelectedItem(InitialSelected)
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.ContentPadding(0.f)
		.Content()
		[
			SNew( STextBlock )
			.Text(this, &FAnimSequenceDetails::GetRetargetSourceComboBoxContent)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.ToolTipText(this, &FAnimSequenceDetails::GetRetargetSourceComboBoxToolTip)
		]	
	];

	AnimationCategory
	.AddCustomRow(RetargetSourceAssetHandle->GetPropertyDisplayName())
	.RowTag(RetargetSourceAssetHandle->GetProperty()->GetFName())
	.NameContent()
	[
		RetargetSourceAssetHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			RetargetSourceAssetHandle->CreatePropertyValueWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(SButton)
			.Text(LOCTEXT("UpdateRetargetSourceAssetDataButton", "Update"))
			.ToolTipText(LOCTEXT("UpdateRetargetSourceAssetDataButtonToolTip", "Updates retargeting data for RetargetSourceAsset. This is updated automatically at save, but you can click here to update without saving."))
			.Visibility(this, &FAnimSequenceDetails::UpdateRetargetSourceAssetDataVisibility)
			.OnClicked(this, &FAnimSequenceDetails::UpdateRetargetSourceAssetData)
		]
	];

	DetailBuilder.HideProperty(RetargetSourceNameHandler);
	DetailBuilder.HideProperty(RetargetSourceAssetHandle);

	// *** Animation Track Names ***
	TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences = DetailBuilder.GetSelectedObjectsOfType<UAnimSequence>();
	AnimationTrackNamesList.Empty();
	if (!AnimSequences.IsEmpty())
	{
		for (const TWeakObjectPtr<UAnimSequence>& AnimSequenceWeak : AnimSequences)
		{
			if(AnimSequenceWeak.IsValid())
			{ 
				if (AnimSequenceWeak->IsDataModelValid())
				{
					AnimSequenceWeak->GetDataModelInterface()->GetBoneTrackNames(AnimationTrackNamesList);
				}
			}
		}
	}

	if (AnimationTrackNamesList.Num())
	{
		const FName AnimTrackNamesGroupName("AnimationTrackNames");

		const FText TrackNameDisplayText = LOCTEXT("AnimTrackNamesLabel", "Animation Track Names");
		IDetailGroup& AnimTrackNamesGroup = AnimationCategory.AddGroup(AnimTrackNamesGroupName, TrackNameDisplayText);
		AnimTrackNamesGroup.HeaderRow()
		.NameContent()
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ContentPadding(FMargin(0,2,0,2))
			.OnClicked_Lambda( [&AnimTrackNamesGroup]() { AnimTrackNamesGroup.ToggleExpansion(!AnimTrackNamesGroup.GetExpansionState()); return FReply::Handled(); } )				
			.ForegroundColor( FSlateColor::UseForeground() )
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(TrackNameDisplayText)
			]
		]
		.ValueWidget
		[
			SNew(STextBlock)
			.Text_Lambda([NumTracks = AnimationTrackNamesList.Num()]()
			{
				return FText::Format(LOCTEXT("NumTracksFormat", "Number of Tracks: {0}"), FText::AsNumber(NumTracks));
			})
			.Font(DetailBuilder.GetDetailFont())
		];
		for (int32 Index = 0; Index < AnimationTrackNamesList.Num(); Index++)
		{
			FDetailWidgetRow& PropRow = AnimTrackNamesGroup.AddWidgetRow();
			PropRow.OverrideResetToDefault(FResetToDefaultOverride::Hide());
			PropRow.WholeRowWidget
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
							.Text_Lambda([this, Index]()
							{
								if (AnimationTrackNamesList.IsValidIndex(Index))
								{
									return FText::FromName(AnimationTrackNamesList[Index]);
								}
						
								return FText::GetEmpty();
							})
							.Font(DetailBuilder.GetDetailFont())
					]
			];
		}
	}

	// *** Compressed Animation Data information ***
	if (AnimSequences.Num() == 1)
	{
		IDetailCategoryBuilder& CompressionSettingsCategory = DetailBuilder.EditCategory("Compression");
		UAnimSequence* AnimSequence = AnimSequences[0].Get();
		CompressionSettingsCategory.AddCustomBuilder(MakeShared<FCompressedAnimationDataNodeBuilder>(AnimSequence));
	}

	/////////////////////////////////////////////////////////////////////////////
	// Additive settings category
	/////////////////////////////////////////////////////////////////////////////
	// now customize to combo box
	IDetailCategoryBuilder& AdditiveSettingsCategory = DetailBuilder.EditCategory("AdditiveSettings");

	// hide all properties for additive anim and replace them with custom additive settings setup
	AdditiveAnimTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
	RefPoseTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RefPoseType));
	RefPoseSeqHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RefPoseSeq));
	RefFrameIndexHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RefFrameIndex));

	CreateOverridenProperty(DetailBuilder, AdditiveSettingsCategory, AdditiveAnimTypeHandle, TAttribute<EVisibility>( EVisibility::Visible ));
	CreateOverridenProperty(DetailBuilder, AdditiveSettingsCategory, RefPoseTypeHandle, TAttribute<EVisibility>( this, &FAnimSequenceDetails::ShouldShowRefPoseType ));
	
	DetailBuilder.HideProperty(RefPoseSeqHandle);

	AdditiveSettingsCategory.AddCustomRow(RefPoseSeqHandle->GetPropertyDisplayName())
	.Visibility(TAttribute<EVisibility>( this, &FAnimSequenceDetails::ShouldShowRefAnimInfo ))
	.NameContent()
	[
		RefPoseSeqHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			RefPoseSeqHandle->CreatePropertyValueWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SAnimationRefPoseViewport)
			.Skeleton(TargetSkeleton.Get())
			.AnimRefPropertyHandle(RefPoseSeqHandle)
			.RefPoseTypeHandle(RefPoseTypeHandle)
			.RefFrameIndexPropertyHandle(RefFrameIndexHandle)
		]
	];

	CreateOverridenProperty(DetailBuilder, AdditiveSettingsCategory, RefFrameIndexHandle, TAttribute<EVisibility>( this, &FAnimSequenceDetails::ShouldShowRefFrameIndex ));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FAnimSequenceDetails::GenerateAnimationTrackNameArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout)
{
	IDetailPropertyRow& PropRow = ChildrenBuilder.AddProperty(PropertyHandle);
	PropRow.ShowPropertyButtons(false);
	PropRow.OverrideResetToDefault(FResetToDefaultOverride::Hide());

	FDetailWidgetRow& WidgetRow = PropRow.CustomWidget(true);

	WidgetRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];

	WidgetRow.ValueContent()
	[
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(5, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SEditableTextBox)
					.Text_Lambda([this, ArrayIndex]()
					{
						if (AnimationTrackNamesList.IsValidIndex(ArrayIndex))
						{
							return FText::FromName(AnimationTrackNamesList[ArrayIndex]);
						}
						
						return FText::GetEmpty();
					})
					.IsReadOnly(true)
			]
	];
}

void FAnimSequenceDetails::CreateOverridenProperty(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& AdditiveSettingsCategory, TSharedPtr<IPropertyHandle> PropertyHandle, TAttribute<EVisibility> VisibilityAttribute)
{
	DetailBuilder.HideProperty(PropertyHandle);
	
	AdditiveSettingsCategory.AddCustomRow(PropertyHandle->GetPropertyDisplayName())
	.Visibility(VisibilityAttribute)
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget()
	];
}

EVisibility FAnimSequenceDetails::ShouldShowRefPoseType() const
{
	uint8 AdditiveAnimType = AAT_None; 
	AdditiveAnimTypeHandle->GetValue(AdditiveAnimType);
	return AdditiveAnimType != AAT_None? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FAnimSequenceDetails::ShouldShowRefAnimInfo() const
{
	uint8 AdditiveAnimType = AAT_None; 
	uint8 RefPoseType = ABPT_None; 
	AdditiveAnimTypeHandle->GetValue(AdditiveAnimType);
	RefPoseTypeHandle->GetValue(RefPoseType);
	return TargetSkeleton.IsValid() && AdditiveAnimType != AAT_None && (RefPoseType == ABPT_AnimScaled || RefPoseType == ABPT_AnimFrame)? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAnimSequenceDetails::ShouldShowRefFrameIndex() const
{
	uint8 AdditiveAnimType = AAT_None; 
	uint8 RefPoseType = ABPT_None; 
	AdditiveAnimTypeHandle->GetValue(AdditiveAnimType);
	RefPoseTypeHandle->GetValue(RefPoseType);
	return TargetSkeleton.IsValid() && AdditiveAnimType != AAT_None && (RefPoseType == ABPT_AnimFrame || RefPoseType == ABPT_LocalAnimFrame)? EVisibility::Visible : EVisibility::Collapsed;
}


TSharedRef<SWidget> FAnimSequenceDetails::MakeRetargetSourceComboWidget( TSharedPtr<FString> InItem )
{
	return SNew(STextBlock) .Text( FText::FromString(*InItem) ) .Font( IDetailLayoutBuilder::GetDetailFont() );
}

void FAnimSequenceDetails::DelegateRetargetSourceChanged()
{
	if (TargetSkeleton.IsValid())
	{
		// first create profile combo list
		RetargetSourceComboList.Empty();
		// first one is default one
		RetargetSourceComboList.Add( MakeShareable( new FString (DEFAULT_RETARGET_SOURCE_NAME) ));	

		// go through profile and see if it has mine
		for (auto Iter = TargetSkeleton->AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
		{
			RetargetSourceComboList.Add( MakeShareable( new FString ( Iter.Key().ToString() )));
		}

		RetargetSourceComboBox->RefreshOptions();
	}
}

void FAnimSequenceDetails::RegisterRetargetSourceChanged()
{
	if (TargetSkeleton.IsValid() && !OnDelegateRetargetSourceChanged.IsBound())
	{
		OnDelegateRetargetSourceChanged = USkeleton::FOnRetargetSourceChanged::CreateSP( this, &FAnimSequenceDetails::DelegateRetargetSourceChanged );
		OnDelegateRetargetSourceChangedDelegateHandle = TargetSkeleton->RegisterOnRetargetSourceChanged(OnDelegateRetargetSourceChanged);
	}
}

FAnimSequenceDetails::~FAnimSequenceDetails()
{
	if (TargetSkeleton.IsValid() && OnDelegateRetargetSourceChanged.IsBound())
	{
		TargetSkeleton->UnregisterOnRetargetSourceChanged(OnDelegateRetargetSourceChangedDelegateHandle);
	}
}

void FAnimSequenceDetails::OnRetargetSourceComboOpening()
{
	FName RetargetSourceName;
	if (RetargetSourceNameHandler->GetValue(RetargetSourceName) != FPropertyAccess::Result::MultipleValues)
	{
		TSharedPtr<FString> ComboStringPtr = GetRetargetSourceString(RetargetSourceName);
		if( ComboStringPtr.IsValid() )
		{
			RetargetSourceComboBox->SetSelectedItem(ComboStringPtr);
		}
	}
}

void FAnimSequenceDetails::OnRetargetSourceChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo  )
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();

		if (NewValue == DEFAULT_RETARGET_SOURCE_NAME)
		{
			NewValue = TEXT("");
		}
		// set profile set up
		ensure ( RetargetSourceNameHandler->SetValue(NewValue) ==  FPropertyAccess::Result::Success );
	}
}

FText FAnimSequenceDetails::GetRetargetSourceComboBoxContent() const
{
	FName RetargetSourceName;
	if (RetargetSourceNameHandler->GetValue(RetargetSourceName) == FPropertyAccess::Result::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*GetRetargetSourceString(RetargetSourceName).Get());
}

FText FAnimSequenceDetails::GetRetargetSourceComboBoxToolTip() const
{
	return LOCTEXT("RetargetSourceComboToolTip", "When retargeting, this pose will be used as a base of animation");
}

TSharedPtr<FString> FAnimSequenceDetails::GetRetargetSourceString(FName RetargetSourceName) const
{
	FString RetargetSourceString = RetargetSourceName.ToString();

	// go through profile and see if it has mine
	for (int32 Index=1; Index<RetargetSourceComboList.Num(); ++Index)
	{
		if (RetargetSourceString == *RetargetSourceComboList[Index])
		{
			return RetargetSourceComboList[Index];
		}
	}

	return RetargetSourceComboList[0];
}

EVisibility FAnimSequenceDetails::UpdateRetargetSourceAssetDataVisibility() const
{
	for (const TWeakObjectPtr<UAnimSequence>& WeakAnimSequence : SelectedAnimSequences)
	{
		if (UAnimSequence* AnimSequence = WeakAnimSequence.Get())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (!AnimSequence->RetargetSourceAsset.IsNull())
			{
				return EVisibility::Visible;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	return EVisibility::Collapsed;
}

FReply FAnimSequenceDetails::UpdateRetargetSourceAssetData()
{
	RetargetSourceAssetHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
	return FReply::Handled();
}

////////////////////////////////////////////////
// based on SAnimationSegmentViewport
SAnimationRefPoseViewport::~SAnimationRefPoseViewport()
{
	// clean up components
	if (PreviewComponent)
	{
		for (int32 I=PreviewComponent->GetAttachChildren().Num()-1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			// PreviewComponet will be cleaned up by PreviewScene, 
			// but if anything is attached, it won't be cleaned up, 
			// so we'll need to clean them up manually
			CleanupComponent(PreviewComponent->GetAttachChildren()[I]);
		}
		check(PreviewComponent->GetAttachChildren().Num() == 0);
	}
}

void SAnimationRefPoseViewport::CleanupComponent(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 I = Component->GetAttachChildren().Num() - 1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			CleanupComponent(Component->GetAttachChildren()[I]);
		}
		check(Component->GetAttachChildren().Num() == 0);

		Component->DestroyComponent();
	}
}

SAnimationRefPoseViewport::SAnimationRefPoseViewport()
	: PreviewScene(FPreviewScene::ConstructionValues())
	, PreviewComponent(nullptr)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAnimationRefPoseViewport::Construct(const FArguments& InArgs)
{
	TargetSkeleton = InArgs._Skeleton;
	AnimRefPropertyHandle = InArgs._AnimRefPropertyHandle;
	RefPoseTypeHandle = InArgs._RefPoseTypeHandle;
	RefFrameIndexPropertyHandle = InArgs._RefFrameIndexPropertyHandle;

	// Create the preview component
	PreviewComponent = NewObject<UDebugSkelMeshComponent>();
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	PreviewScene.AddComponent( PreviewComponent, FTransform::Identity );

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(Description, STextBlock)
			.Text(LOCTEXT("DefaultViewportLabel", "Default View"))
			.AutoWrapText(true)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			[
				SAssignNew(ViewportWidget, SViewport)
				.EnableGammaCorrection(false)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SAnimationSegmentScrubPanel)
			.ViewInputMin(this, &SAnimationRefPoseViewport::GetViewMinInput)
			.ViewInputMax(this, &SAnimationRefPoseViewport::GetViewMaxInput)
			.PreviewInstance(this, &SAnimationRefPoseViewport::GetPreviewInstance)
			.DraggableBars(this, &SAnimationRefPoseViewport::GetBars)
			.OnBarDrag(this, &SAnimationRefPoseViewport::OnBarDrag)
			.OnTickPlayback(this, &SAnimationRefPoseViewport::OnTickPreview)
			.bAllowZoom(true)
		]
	];

	// Create the viewport
	LevelViewportClient = MakeShareable( new FAnimationSegmentViewportClient( PreviewScene ) );

	LevelViewportClient->ViewportType = LVT_Perspective;
	LevelViewportClient->bSetListenerPosition = false;
	LevelViewportClient->SetViewLocation( EditorViewportDefs::DefaultPerspectiveViewLocation );
	LevelViewportClient->SetViewRotation( EditorViewportDefs::DefaultPerspectiveViewRotation );

	SceneViewport = FSceneViewport::Create(LevelViewportClient, ViewportWidget);
	LevelViewportClient->SetRealtime( true );
	LevelViewportClient->VisibilityDelegate.BindSP( this, &SAnimationRefPoseViewport::IsVisible );
	LevelViewportClient->SetViewMode( VMI_Lit );

	InitSkeleton();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAnimationRefPoseViewport::InitSkeleton()
{
	UObject *Object = nullptr;
	AnimRefPropertyHandle->GetValue(Object);
	PreviewAnimationSequence = Cast<UAnimSequence>(Object);
	const USkeleton* Skeleton = nullptr;
	if(PreviewAnimationSequence != nullptr)
	{
		Skeleton = PreviewAnimationSequence->GetSkeleton();
	}

	// if skeleton doesn't match with target skeleton, this is error, we can't support it
	if (PreviewComponent && Skeleton && (Skeleton == TargetSkeleton))
	{	
		UAnimSingleNodeInstance* PreviewAnimInstance = PreviewComponent->PreviewInstance;
		USkeletalMesh* PreviewSkeletalMesh = [Skeleton, this]() -> USkeletalMesh*
		{
			// Try preview mesh on Anim Sequence
			USkeletalMesh* Mesh = PreviewAnimationSequence->GetPreviewMesh(); 
			
			// Otherwise try skeleton preview mesh
			if (Mesh == nullptr)
			{
				Mesh = Skeleton->GetPreviewMesh();
			}

			// Last resort try to find a _any_ compatible mesh for the skeleton
			if (Mesh == nullptr)
			{
				Mesh = Skeleton->FindCompatibleMesh();
			}

			return Mesh;
		}();
		
		const bool bInvalidPreviewInstance = PreviewAnimInstance == nullptr || PreviewAnimInstance->GetCurrentAsset() != PreviewAnimationSequence;
		const bool bPreviewMeshMismatch = PreviewComponent->GetSkeletalMeshAsset() != PreviewSkeletalMesh;
		if(bInvalidPreviewInstance || bPreviewMeshMismatch)
		{
			PreviewComponent->SetSkeletalMesh(PreviewSkeletalMesh);
			PreviewComponent->EnablePreview(true, PreviewAnimationSequence);
			PreviewComponent->PreviewInstance->SetLooping(true);

			//Place the camera at a good viewer position
			FVector NewPosition = LevelViewportClient->GetViewLocation();
			NewPosition.Normalize();
			if(PreviewSkeletalMesh)
			{
				NewPosition *= (PreviewSkeletalMesh->GetImportedBounds().SphereRadius*1.5f);
			}
			LevelViewportClient->SetViewLocation( NewPosition );
		}
	}
}

void SAnimationRefPoseViewport::OnTickPreview( double InCurrentTime, float InDeltaTime )
{
	LevelViewportClient->Invalidate();
}

void SAnimationRefPoseViewport::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	class UDebugSkelMeshComponent* Component = PreviewComponent;

	FString TargetSkeletonName = TargetSkeleton ? TargetSkeleton->GetName() : FName( NAME_None ).ToString();

	if ( Component != NULL )
	{
		// Reinit the skeleton if the anim ref has changed
		InitSkeleton();

		if ( Component->IsPreviewOn() && PreviewAnimationSequence != NULL )
		{
			if ( PreviewComponent != NULL && PreviewComponent->PreviewInstance != NULL )
			{
				uint8 RefPoseType;
				RefPoseTypeHandle->GetValue( RefPoseType );
				if ( RefPoseType == ABPT_AnimFrame )
				{
					int RefFrameIndex;
					RefFrameIndexPropertyHandle->GetValue( RefFrameIndex );
					float Fraction = ( PreviewAnimationSequence->GetNumberOfSampledKeys() > 0 ) ? FMath::Clamp<float>( (float)RefFrameIndex / (float)PreviewAnimationSequence->GetNumberOfSampledKeys(), 0.f, 1.f ) : 0.f;
					float RefTime = PreviewAnimationSequence->GetPlayLength() * Fraction;
					PreviewComponent->PreviewInstance->SetPosition( RefTime, false );
					PreviewComponent->PreviewInstance->SetPlaying( false );
					LevelViewportClient->Invalidate();
				}
			}

			Description->SetText( FText::Format( LOCTEXT( "Previewing", "Previewing {0}" ), FText::FromString( Component->GetPreviewText() ) ) );
		}
		else if ( Component->AnimClass )
		{
			Description->SetText( FText::Format( LOCTEXT( "Previewing", "Previewing {0}" ), FText::FromString( Component->AnimClass->GetName() ) ) );
		}
		else if ( PreviewAnimationSequence && !PreviewAnimationSequence->GetSkeleton()->IsCompatibleForEditor(TargetSkeleton) )
		{
			Description->SetText( FText::Format( LOCTEXT( "IncorrectSkeleton", "The preview asset is incompatible with the skeleton '{0}'" ), FText::FromString( TargetSkeletonName ) ) );
		}
		else if ( Component->GetSkeletalMeshAsset() == NULL )
		{
			Description->SetText( FText::Format( LOCTEXT( "NoMeshFound", "No skeletal mesh found for skeleton '{0}'" ), FText::FromString( TargetSkeletonName ) ) );
		}
		else
		{
			Description->SetText( FText::Format( LOCTEXT( "SelectAnimation", "Select animation that works for skeleton '{0}'" ), FText::FromString( TargetSkeletonName ) ) );
		}

		Component->GetScene()->GetWorld()->Tick( LEVELTICK_All, InDeltaTime );
	}
	else
	{
		Description->SetText( FText::Format( LOCTEXT( "NoMeshFound", "No skeletal mesh found for skeleton '{0}'" ), FText::FromString( TargetSkeletonName ) ) );
	}
}

void SAnimationRefPoseViewport::RefreshViewport()
{
}

bool SAnimationRefPoseViewport::IsVisible() const
{
	return ViewportWidget.IsValid();
}

float SAnimationRefPoseViewport::GetViewMinInput() const 
{ 
	if (PreviewComponent != NULL)
	{
		if (PreviewComponent->PreviewInstance != NULL)
		{
			return 0.0f;
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return FMath::Max<float>((float)(PreviewComponent->GetAnimInstance()->LifeTimer - 30.0), 0.0f);
		}
	}

	return 0.f; 
}

float SAnimationRefPoseViewport::GetViewMaxInput() const 
{ 
	if (PreviewComponent != NULL)
	{
		if (PreviewComponent->PreviewInstance != NULL)
		{
			return PreviewComponent->PreviewInstance->GetLength();
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return static_cast<float>(PreviewComponent->GetAnimInstance()->LifeTimer);
		}
	}

	return 0.f;
}

TArray<float> SAnimationRefPoseViewport::GetBars() const
{
	TArray<float> Bars;
	if (PreviewAnimationSequence)
	{
		int32 RefFrameIndex;
		RefFrameIndexPropertyHandle->GetValue(RefFrameIndex);
		float Fraction = (PreviewAnimationSequence->GetNumberOfSampledKeys() > 0)? FMath::Clamp((float)RefFrameIndex/(float)PreviewAnimationSequence->GetNumberOfSampledKeys(), 0.f, 1.f) : 0.f;
		Bars.Add(static_cast<float>(PreviewAnimationSequence->GetPlayLength() * Fraction));
	}
	else
	{
		Bars.Add(0.0f);
	}
	return Bars;
}

void SAnimationRefPoseViewport::OnBarDrag(int32 Index, float Position)
{
	if (PreviewAnimationSequence)
	{
		int RefFrameIndex = FMath::Clamp(PreviewAnimationSequence->GetPlayLength() > 0.0 ? (int)(Position * (float)PreviewAnimationSequence->GetNumberOfSampledKeys() / static_cast<float>(PreviewAnimationSequence->GetPlayLength()) + 0.5f) : static_cast<int>(0), static_cast<int>(0), static_cast<int>(PreviewAnimationSequence->GetNumberOfSampledKeys() - 1));
		RefFrameIndexPropertyHandle->SetValue(RefFrameIndex);
	}
}

UAnimSingleNodeInstance* SAnimationRefPoseViewport::GetPreviewInstance() const
{
	return PreviewComponent ? PreviewComponent->PreviewInstance : NULL;
}
#undef LOCTEXT_NAMESPACE
