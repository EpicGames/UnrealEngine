// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"
#include "Animation/Skeleton.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FEditorViewportClient;
class FString;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class STextBlock;
class SViewport;
class SWidget;
class UAnimSequence;
class USceneComponent;
struct FGeometry;

class FAnimSequenceDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	/** Create an override for the supplide Property */
	void CreateOverridenProperty(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& AdditiveSettingsCategory, TSharedPtr<IPropertyHandle> PropertyHandle, TAttribute<EVisibility> VisibilityAttribute);

	/** Functions to control property visibility */
	EVisibility ShouldShowRefPoseType() const;
	EVisibility ShouldShowRefAnimInfo() const;
	EVisibility ShouldShowRefFrameIndex() const;

	virtual ~FAnimSequenceDetails();

private:
	
	TWeakObjectPtr<USkeleton> TargetSkeleton;

	// additive setting handler
	TSharedPtr<IPropertyHandle> AdditiveAnimTypeHandle;
	TSharedPtr<IPropertyHandle> RefPoseTypeHandle;
	TSharedPtr<IPropertyHandle> RefPoseSeqHandle;
	TSharedPtr<IPropertyHandle> RefFrameIndexHandle;

	// retarget source handler
	TSharedPtr<IPropertyHandle> RetargetSourceNameHandler;
	TSharedPtr<IPropertyHandle> RetargetSourceAssetHandle;

	TSharedPtr<class SSearchableComboBox> RetargetSourceComboBox;
	TArray< TSharedPtr< FString > > RetargetSourceComboList;

	TSharedRef<SWidget> MakeRetargetSourceComboWidget( TSharedPtr<FString> InItem );
	void OnRetargetSourceChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo  );
	FText GetRetargetSourceComboBoxContent() const;
	FText GetRetargetSourceComboBoxToolTip() const;
	void OnRetargetSourceComboOpening();
	TSharedPtr<FString> GetRetargetSourceString(FName RetargetSourceName) const;
	EVisibility UpdateRetargetSourceAssetDataVisibility() const;
	FReply UpdateRetargetSourceAssetData();

	USkeleton::FOnRetargetSourceChanged OnDelegateRetargetSourceChanged;
	FDelegateHandle OnDelegateRetargetSourceChangedDelegateHandle;
	void RegisterRetargetSourceChanged();
	void DelegateRetargetSourceChanged();

	// *** Animation Track Names ***
	TSharedPtr<IPropertyHandle> AnimationTrackNamesHandle;
	TArray<FName> AnimationTrackNamesList;
	void GenerateAnimationTrackNameArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout);

	// button handler for Apply Compression
	// cache all anim sequences that are selected
	// but I need to know them before compress
	TArray< TWeakObjectPtr<UAnimSequence> > SelectedAnimSequences;
};

///////////////////////////////////////////////////
class SAnimationRefPoseViewport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimationRefPoseViewport )
	{}

		SLATE_ARGUMENT( USkeleton*, Skeleton )
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, AnimRefPropertyHandle )
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, RefPoseTypeHandle )
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, RefFrameIndexPropertyHandle )

	SLATE_END_ARGS()

public:
	SAnimationRefPoseViewport();
	virtual ~SAnimationRefPoseViewport();

	void Construct(const FArguments& InArgs);

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	void RefreshViewport();

private:

	/** Called to tick the preview during playback */
	void OnTickPreview( double InCurrentTime, float InDeltaTime );

	void InitSkeleton();

	TSharedPtr<FEditorViewportClient> LevelViewportClient;

	TSharedPtr<IPropertyHandle> AnimRefPropertyHandle;
	TSharedPtr<IPropertyHandle> RefPoseTypeHandle;
	TSharedPtr<IPropertyHandle> RefFrameIndexPropertyHandle;

	/** Slate viewport for rendering and I/O */
	//TSharedPtr<class FSceneViewport> Viewport;
	TSharedPtr<SViewport> ViewportWidget;

	TSharedPtr<class FSceneViewport> SceneViewport;

	/** Skeleton */
	USkeleton* TargetSkeleton;
	UAnimSequence* PreviewAnimationSequence;

	FAdvancedPreviewScene PreviewScene;
	class FFXSystemInterface* FXSystem;

	TSharedPtr<STextBlock> Description;

	class UDebugSkelMeshComponent* PreviewComponent;

	void CleanupComponent(USceneComponent* Component);
	bool IsVisible() const;

public:
	/** Get Min/Max Input of value **/
	float GetViewMinInput() const;
	float GetViewMaxInput() const;

	class UAnimSingleNodeInstance*	GetPreviewInstance() const;

	/** Optional, additional values to draw on the timeline **/
	TArray<float> GetBars() const;
	void OnBarDrag(int32 index, float newPos);
};
