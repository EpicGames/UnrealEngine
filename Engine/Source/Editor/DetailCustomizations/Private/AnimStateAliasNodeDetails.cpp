// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimStateAliasNodeDetails.h"

#include "AnimStateAliasNode.h"
#include "AnimStateNode.h"
#include "AnimStateNodeBase.h"
#include "AnimationStateMachineGraph.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class UBlueprint;

#define LOCTEXT_NAMESPACE "FAnimStateAliasNodeDetails"

/////////////////////////////////////////////////////////////////////////


TSharedRef<IDetailCustomization> FAnimStateAliasNodeDetails::MakeInstance()
{
	return MakeShareable( new FAnimStateAliasNodeDetails);
}

void FAnimStateAliasNodeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Get a handle to the node we're viewing
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[0];
	UAnimStateAliasNode* StateAliasNode = nullptr;
	if (CurrentObject.IsValid())
	{
		StateAliasNodeWeak = StateAliasNode = Cast<UAnimStateAliasNode>(CurrentObject.Get());
	}

	if (StateAliasNode == nullptr)
	{
		return;
	}

	GenerateStatePickerDetails(*StateAliasNode, DetailBuilder);
}

void FAnimStateAliasNodeDetails::GetReferenceableStates(const UAnimStateAliasNode& OwningNode, TSet<TWeakObjectPtr<UAnimStateNodeBase>>& OutStates) const
{
	UAnimationStateMachineGraph* StateMachine = CastChecked<UAnimationStateMachineGraph>(OwningNode.GetOuter());

	TArray<UAnimStateNode*> Nodes;
	StateMachine->GetNodesOfClass<UAnimStateNode>(Nodes);
	for (auto NodeIt = Nodes.CreateIterator(); NodeIt; ++NodeIt)
	{
		auto Node = *NodeIt;
		OutStates.Add(Node);
	}

	OutStates.Sort([](TWeakObjectPtr<UAnimStateNodeBase> InA, TWeakObjectPtr<UAnimStateNodeBase> InB)
	{
		return InA.Get()->GetStateName() < InB.Get()->GetStateName();
	});
}

bool FAnimStateAliasNodeDetails::IsGlobalAlias() const
{
	if (UAnimStateAliasNode* StateAliasNode = StateAliasNodeWeak.Get())
	{
		return StateAliasNode->bGlobalAlias;
	}
	
	return false;
}

void FAnimStateAliasNodeDetails::OnPropertyAliasAllStatesCheckboxChanged(ECheckBoxState NewState)
{
	if (UAnimStateAliasNode* StateAliasNode = StateAliasNodeWeak.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("Undo_SelectAllAliasState", "Select all state alias"));

		StateAliasNode->Modify();
		if (NewState == ECheckBoxState::Checked)
		{
			StateAliasNode->GetAliasedStates() = ReferenceableStates;
		}
		else
		{
			StateAliasNode->GetAliasedStates().Reset();
		}

		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(StateAliasNode))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

ECheckBoxState FAnimStateAliasNodeDetails::AreAllStatesAliased() const
{
	if (const UAnimStateAliasNode* StateAliasNode = StateAliasNodeWeak.Get())
	{
		const int32 NumAlisedStates = StateAliasNode->GetAliasedStates().Num();
		const int32 NumStates = ReferenceableStates.Num();

		if (NumAlisedStates == 0)
		{
			return ECheckBoxState::Unchecked;
		}
		else if(NumAlisedStates == NumStates)
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Undetermined;
}

void FAnimStateAliasNodeDetails::OnPropertyIsStateAliasedCheckboxChanged(ECheckBoxState NewState, const TWeakObjectPtr<UAnimStateNodeBase> StateNodeWeak)
{

	if (UAnimStateNodeBase* StateNode = StateNodeWeak.Get())
	{
		if (UAnimStateAliasNode* StateAliasNode = StateAliasNodeWeak.Get())
		{
			FScopedTransaction Transaction(LOCTEXT("Undo_AliasState", "Select state alias"));

			StateAliasNode->Modify();
			if (NewState == ECheckBoxState::Checked)
			{
				StateAliasNode->GetAliasedStates().FindOrAdd(StateNodeWeak);
			}
			else
			{
				StateAliasNode->GetAliasedStates().Remove(StateNodeWeak);
			}

			if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(StateAliasNode))
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}
}

ECheckBoxState FAnimStateAliasNodeDetails::IsStateAliased(const TWeakObjectPtr<UAnimStateNodeBase> StateNodeWeak) const
{
	if (const UAnimStateNodeBase* StateNode = StateNodeWeak.Get())
	{
		if (const UAnimStateAliasNode* StateAliasNode = StateAliasNodeWeak.Get())
		{
			if (StateAliasNode->GetAliasedStates().Find(StateNodeWeak))
			{
				return ECheckBoxState::Checked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

void FAnimStateAliasNodeDetails::GenerateStatePickerDetails(UAnimStateAliasNode& AliasNode, IDetailLayoutBuilder& DetailBuilder)
{
	ReferenceableStates.Reset();
	GetReferenceableStates(AliasNode, ReferenceableStates);

	if (ReferenceableStates.Num() > 0)
	{
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("State Alias")));
		CategoryBuilder.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateAliasNode, bGlobalAlias));

		FDetailWidgetRow& HeaderWidgetRow = CategoryBuilder.AddCustomRow(LOCTEXT("SelectAll", "Select All"));

		HeaderWidgetRow.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("StateName", "Name"))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			];

		HeaderWidgetRow.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectAllStatesPropertyValue", "Select All"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &FAnimStateAliasNodeDetails::AreAllStatesAliased)
					.OnCheckStateChanged(this, &FAnimStateAliasNodeDetails::OnPropertyAliasAllStatesCheckboxChanged)
					.IsEnabled_Lambda([this]() -> bool 
						{
							return !IsGlobalAlias();
						})
				]
			];

		for (auto StateIt = ReferenceableStates.CreateConstIterator(); StateIt; ++StateIt)
		{
			const TWeakObjectPtr<UAnimStateNodeBase>& StateNodeWeak = *StateIt;
			if (const UAnimStateNodeBase* StateNode = StateNodeWeak.Get())
			{
				FString StateName = StateNode->GetStateName();
				FText StateText = FText::FromString(StateName);

				FDetailWidgetRow& PropertyWidgetRow = CategoryBuilder.AddCustomRow(StateText);

				PropertyWidgetRow.NameContent()
					[
						SNew(STextBlock)
						.Text(StateText)
						.ToolTipText(StateText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					];

				PropertyWidgetRow.ValueContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(SCheckBox)
							.IsChecked(this, &FAnimStateAliasNodeDetails::IsStateAliased, StateNodeWeak)
							.OnCheckStateChanged(this, &FAnimStateAliasNodeDetails::OnPropertyIsStateAliasedCheckboxChanged, StateNodeWeak)
							.IsEnabled_Lambda([this]() -> bool 
								{
								return !IsGlobalAlias();
								})
						]
					];
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
