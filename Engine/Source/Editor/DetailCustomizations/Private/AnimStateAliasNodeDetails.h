// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "IDetailCustomization.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class UAnimStateAliasNode;
class UAnimStateNodeBase;

class FAnimStateAliasNodeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	void GetReferenceableStates(const UAnimStateAliasNode& OwningNode, TSet<TWeakObjectPtr<UAnimStateNodeBase>>& OutStates) const;

	bool IsGlobalAlias() const;
	void OnPropertyAliasAllStatesCheckboxChanged(ECheckBoxState NewState);
	ECheckBoxState AreAllStatesAliased() const;
	void OnPropertyIsStateAliasedCheckboxChanged(ECheckBoxState NewState, const TWeakObjectPtr<UAnimStateNodeBase> StateNodeWeak);
	ECheckBoxState IsStateAliased(const TWeakObjectPtr<UAnimStateNodeBase> StateNodeWeak) const;

	void GenerateStatePickerDetails(UAnimStateAliasNode& AliasNode, IDetailLayoutBuilder& DetailBuilder);

	TWeakObjectPtr<UAnimStateAliasNode> StateAliasNodeWeak;
	TSet<TWeakObjectPtr<UAnimStateNodeBase>> ReferenceableStates;
};

