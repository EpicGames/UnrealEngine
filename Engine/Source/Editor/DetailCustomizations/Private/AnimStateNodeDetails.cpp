// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimStateNodeDetails.h"

#include "AnimGraphNode_StateResult.h"
#include "AnimStateNode.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/STextComboBox.h"

class IDetailCustomization;

#define LOCTEXT_NAMESPACE "FAnimStateNodeDetails"

/////////////////////////////////////////////////////////////////////////


TSharedRef<IDetailCustomization> FAnimStateNodeDetails::MakeInstance()
{
	return MakeShareable( new FAnimStateNodeDetails );
}

void FAnimStateNodeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Add uproperty info from the following nodes.
	TArray<TWeakObjectPtr<UAnimStateNode>> StateNodes = DetailBuilder.GetObjectsOfTypeBeingCustomized<UAnimStateNode>();
	TArray<UObject*> StateResultNodes;
	for (TWeakObjectPtr<UAnimStateNode> WeakStateNode : StateNodes)
	{
		UAnimStateNode* StateNode = WeakStateNode.Get();
		if (StateNode == nullptr)
		{
			continue;
		}

		UAnimGraphNode_StateResult* StateResultNode = StateNode->GetResultNodeInsideState();
		if (StateResultNode == nullptr)
		{
			continue;
		}

		StateResultNodes.Add(StateResultNode);
	}
	
	// Customize the animation state events (as they need a specific widget)
	{
		IDetailCategoryBuilder& AnimationStateCategory = DetailBuilder.EditCategory("Animation State", LOCTEXT("AnimationState", "Animation State"));

		IDetailPropertyRow* Row = AnimationStateCategory.AddExternalObjects(StateResultNodes, EPropertyLocation::Default, FAddPropertyParams().HideRootObjectNode(true));
		if (Row != nullptr)
		{
			Row->ShouldAutoExpand(true);
		}

		IDetailGroup& NotificationsGroup = AnimationStateCategory.AddGroup("Notifications", LOCTEXT("NotificationsGroupName", "Notifications"), false, true);
	
		AddAnimationStateEventField(AnimationStateCategory, NotificationsGroup, TEXT("StateEntered"));
		AddAnimationStateEventField(AnimationStateCategory, NotificationsGroup, TEXT("StateLeft"));
		AddAnimationStateEventField(AnimationStateCategory, NotificationsGroup, TEXT("StateFullyBlended"));
		
		DetailBuilder.HideProperty("StateEntered");
		DetailBuilder.HideProperty("StateLeft");
		DetailBuilder.HideProperty("StateFullyBlended");	
	}
}

void FAnimStateNodeDetails::AddAnimationStateEventField(const IDetailCategoryBuilder& AnimationStateCategory, IDetailGroup& AnimationStateGroup, const FString& TransitionName)
{
	TSharedPtr<IPropertyHandle> NameProperty = GetTransitionEventProperty(AnimationStateCategory, TransitionName);
	IDetailPropertyRow& Row = AnimationStateGroup.AddPropertyRow(NameProperty.ToSharedRef());
	FText InfoText = LOCTEXT("AnimStateEventInfoTooltip", "These events are deferred and executed after the animation graph has been updated. Non-thread-safe code can safely be executed here. For thread-safe code that must run during the animation graph update, use the Anim Node function versions instead.");
	
	BuildTransitionEventRow(Row, NameProperty, InfoText);
}

#undef LOCTEXT_NAMESPACE
