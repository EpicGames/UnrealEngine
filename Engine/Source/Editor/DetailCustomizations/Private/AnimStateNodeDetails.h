// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTransitionNodeDetails.h"
#include "Templates/SharedPointer.h"

class IDetailGroup;
class IDetailCustomization;
class IDetailLayoutBuilder;
class STextComboBox;

class FAnimStateNodeDetails : public FAnimTransitionNodeDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	static void AddAnimationStateEventField(const IDetailCategoryBuilder& AnimationStateCategory, IDetailGroup& AnimationStateGroup, const FString& TransitionName);
};

