// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTrailNodeDetails.h"

#include "AnimGraphNode_Trail.h"
#include "BoneControllers/AnimNode_Trail.h"
#include "Curves/CurveFloat.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SCurveEditor.h"
#include "Templates/Casts.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UObject;

#define LOCTEXT_NAMESPACE "FAnimTrailNodeDetails"

/////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FAnimTrailNodeDetails::MakeInstance()
{
	return MakeShareable( new FAnimTrailNodeDetails );
}

void FAnimTrailNodeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{	
	TArray<TWeakObjectPtr<UObject> > SelectedObjects;	//the objects we're showing details for
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);

	TSharedPtr<IPropertyHandle> TrailRelaxCurveHandle = DetailBuilder.GetProperty("Node.TrailRelaxationSpeed");

	//we only do fancy customization if we have one vehicle component selected
	if(SelectedObjects.Num() != 1)
	{
		return;
	}
	else if(UAnimGraphNode_Trail* InGraphNode = Cast<UAnimGraphNode_Trail>(SelectedObjects[0].Get()))
	{
		TrailRelaxCurveEditor = FTrailRelaxCurveEditor(InGraphNode, TrailRelaxCurveHandle);
	}
	else
	{
		return;
	}

	//Trail Relax curve
	IDetailCategoryBuilder& TrailCategory = DetailBuilder.EditCategory("Trail");

	DetailBuilder.HideProperty(TrailRelaxCurveHandle);

	TrailCategory.AddProperty(TrailRelaxCurveHandle).CustomWidget()
	.NameContent()
		[
			TrailRelaxCurveHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(125.f * 3.f)
		[
			SAssignNew(TrailRelaxCurveWidget, SCurveEditor)
			.ViewMinInput(0.f)
			.ViewMaxInput(70000.f)
			.ViewMinOutput(0.f)
			.ViewMaxOutput(1.f)
			.TimelineLength(7000.f)
			.HideUI(false)
			.DesiredSize(FVector2D(512, 128))
			.ZoomToFitVertical(true)
			.ZoomToFitHorizontal(true)
		];

	TrailRelaxCurveWidget->SetCurveOwner(&TrailRelaxCurveEditor);
}

void FAnimTrailNodeDetails::PendingDelete()
{
	if(TrailRelaxCurveWidget.IsValid())
	{
		TrailRelaxCurveWidget->SetCurveOwner(nullptr);
	}
}

TArray<FRichCurveEditInfoConst> FAnimTrailNodeDetails::FTrailRelaxCurveEditor::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(&GraphNodeOwner->Node.TrailRelaxationSpeed.EditorCurveData);

	return Curves;
}

void FAnimTrailNodeDetails::FTrailRelaxCurveEditor::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(&GraphNodeOwner->Node.TrailRelaxationSpeed.EditorCurveData);
}

TArray<FRichCurveEditInfo> FAnimTrailNodeDetails::FTrailRelaxCurveEditor::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(&GraphNodeOwner->Node.TrailRelaxationSpeed.EditorCurveData);

	return Curves;
}

void FAnimTrailNodeDetails::FTrailRelaxCurveEditor::ModifyOwner()
{
	if(GraphNodeOwner)
	{
		GraphNodeOwner->Modify();
		if (TrailRelaxCurveHandle.IsValid())
		{
			TrailRelaxCurveHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

TArray<const UObject*> FAnimTrailNodeDetails::FTrailRelaxCurveEditor::GetOwners() const
{
	TArray<const UObject*> Owners;
	if (GraphNodeOwner)
	{
		Owners.Add(GraphNodeOwner);
	}

	return Owners;
}

void FAnimTrailNodeDetails::FTrailRelaxCurveEditor::MakeTransactional()
{
	if(GraphNodeOwner)
	{
		GraphNodeOwner->SetFlags(GraphNodeOwner->GetFlags() | RF_Transactional);
	}
}

bool FAnimTrailNodeDetails::FTrailRelaxCurveEditor::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return CurveInfo.CurveToEdit == &GraphNodeOwner->Node.TrailRelaxationSpeed.EditorCurveData;
}

FAnimTrailNodeDetails::FTrailRelaxCurveEditor::FTrailRelaxCurveEditor(UAnimGraphNode_Trail * InGraphNode, TSharedPtr<IPropertyHandle> InTrailRelaxCurveHandle)
{
	GraphNodeOwner = InGraphNode;
	TrailRelaxCurveHandle = InTrailRelaxCurveHandle;
}

#undef LOCTEXT_NAMESPACE
