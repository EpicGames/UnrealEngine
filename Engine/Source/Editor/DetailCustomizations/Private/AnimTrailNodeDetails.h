// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/RichCurve.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UAnimGraphNode_Trail;
class UObject;

class FAnimTrailNodeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	virtual void PendingDelete() override;

	struct FTrailRelaxCurveEditor: public FCurveOwnerInterface
	{
		FTrailRelaxCurveEditor(UAnimGraphNode_Trail* InGraphNode = nullptr, TSharedPtr<IPropertyHandle> InTrailRelaxCurveHandle = TSharedPtr<IPropertyHandle>());
		/** FCurveOwnerInterface interface */
		UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
		virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
		virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
		virtual TArray<FRichCurveEditInfo> GetCurves() override;
		virtual void ModifyOwner() override;
		virtual TArray<const UObject*> GetOwners() const override;
		virtual void MakeTransactional() override;
		virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override { }
		virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;

	private:
		UAnimGraphNode_Trail* GraphNodeOwner;
		TSharedPtr<IPropertyHandle> TrailRelaxCurveHandle;

	} TrailRelaxCurveEditor;

private:
	TSharedPtr<class SCurveEditor> TrailRelaxCurveWidget;
};

