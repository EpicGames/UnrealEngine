// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTransitionNodeDetails.h"

#include "AnimGraphNode_TransitionResult.h"
#include "AnimStateConduitNode.h"
#include "AnimStateNodeBase.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimStateMachineTypes.h"
#include "AnimationTransitionGraph.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SKismetLinearExpression.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FAnimStateNodeDetails"

/////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FAnimTransitionNodeDetails::MakeInstance()
{
	return MakeShareable( new FAnimTransitionNodeDetails );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FAnimTransitionNodeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Get a handle to the node we're viewing
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
	bool bTransitionToConduit = false;
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[ObjectIndex];
		if (CurrentObject.IsValid())
		{
			if (UAnimStateTransitionNode* TransitionNodePtr = Cast<UAnimStateTransitionNode>(CurrentObject.Get()))
			{
				if(!TransitionNode.IsValid())
				{
					TransitionNode = TransitionNodePtr;
				}

				UAnimStateNodeBase* NextState = TransitionNodePtr->GetNextState();
				if((NextState != NULL) && (NextState->IsA<UAnimStateConduitNode>()))
				{
					bTransitionToConduit = true;
				}
			}
		}
	}

	UAnimStateTransitionNode* TransNode = TransitionNode.Get();
	IDetailCategoryBuilder& TransitionCategory = DetailBuilder.EditCategory("Transition", LOCTEXT("TransitionCategoryTitle", "Transition") );

	if (bTransitionToConduit)
	{
		// Transitions to conduits are just shorthand for some other real transition;
		// All of the blend related settings are ignored, so hide them.
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, Bidirectional));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CrossfadeDuration));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendMode));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CustomBlendCurve));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, LogicType));
		DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendProfileWrapper));
	}
	else
	{
		TransitionCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, PriorityOrder)).DisplayName(LOCTEXT("PriorityOrderLabel", "Priority Order"));
		TransitionCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, Bidirectional)).DisplayName(LOCTEXT("BidirectionalLabel", "Bidirectional"));

		TSharedPtr<IPropertyHandle> LogicTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, LogicType));
		TransitionCategory.AddProperty(LogicTypeHandle)
			.DisplayName(LOCTEXT("BlendLogicLabel", "Blend Logic") )
			.CustomWidget()
			.NameContent()
			[
				LogicTypeHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(300.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					LogicTypeHandle->CreatePropertyValueWidget()
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.FillWidth(1.0f)
				.Padding(3.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.OnClicked(this, &FAnimTransitionNodeDetails::OnClickEditBlendGraph)
					.Visibility( this, &FAnimTransitionNodeDetails::GetBlendGraphButtonVisibility, SelectedObjects.Num() > 1)
					.Text(LOCTEXT("EditBlendGraph", "Edit Blend Graph"))
					.TextStyle(&FAppStyle::Get(), TEXT("TinyText"))
				]
			];

		if (TransitionNode != NULL && SelectedObjects.Num() == 1)
		{
			// The sharing option for the rule
			TransitionCategory.AddCustomRow( LOCTEXT("TransitionRuleSharingLabel", "Transition Rule Sharing") )
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TransitionRuleSharingLabel", "Transition Rule Sharing"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MaxDesiredWidth(300.0f)
			[
				GetWidgetForInlineShareMenu(
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([TransNode]() { return FText::FromString(TransNode->SharedRulesName); })),
					TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([TransNode]() { return TransNode->bSharedRules; })),
					FOnClicked::CreateSP(this, &FAnimTransitionNodeDetails::OnPromoteToSharedClick, true),
					FOnClicked::CreateSP(this, &FAnimTransitionNodeDetails::OnUnshareClick, true), 
					FOnGetContent::CreateSP(this, &FAnimTransitionNodeDetails::OnGetShareableNodesMenu, true))
			];

			// Show the rule itself
			UEdGraphPin* CanExecPin = NULL;
			if (UAnimationTransitionGraph* TransGraph = Cast<UAnimationTransitionGraph>(TransNode->BoundGraph))
			{
				if (UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode())
				{
					CanExecPin = ResultNode->FindPin(TEXT("bCanEnterTransition"));
				}
			}

			if (TransNode->bAutomaticRuleBasedOnSequencePlayerInState)
			{
				if (CanExecPin != nullptr && CanExecPin->LinkedTo.Num() > 0)
				{
					TransitionCategory.AddCustomRow(LOCTEXT("AnimGraphNodeDetailsAutomaticRule_RowWarning", "Automatic Rule"))
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimGraphNodeDetailsAutomaticRule_Warning", "Warning : Automatic Rule Based Transition will override graph exit rule."))
							.ColorAndOpacity(FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor"))
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
						]
					];
				}
				else
				{
					TransitionCategory.AddCustomRow(LOCTEXT("AnimGraphNodeDetailsAutomaticRule_Row", "Automatic Rule"))
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimGraphNodeDetailsAutomaticRule", "Automatic Rule Based Transition"))
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
						]
					];
				}
			}
			else
			{
				// indicate if a native transition rule applies to this
				UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(TransitionNode.Get());
				if(Blueprint && Blueprint->ParentClass)
				{
					UAnimInstance* AnimInstance = CastChecked<UAnimInstance>(Blueprint->ParentClass->GetDefaultObject());
					if(AnimInstance)
					{
						UEdGraph* ParentGraph = TransitionNode->GetGraph();
						UAnimStateNodeBase* PrevState = TransitionNode->GetPreviousState();
						UAnimStateNodeBase* NextState = TransitionNode->GetNextState();
						if(PrevState != nullptr && NextState != nullptr && ParentGraph != nullptr)
						{
							FName FunctionName;
							if(AnimInstance->HasNativeTransitionBinding(ParentGraph->GetFName(), FName(*PrevState->GetStateName()), FName(*NextState->GetStateName()), FunctionName))
							{
								TransitionCategory.AddCustomRow( LOCTEXT("NativeBindingPresent_Filter", "Transition has native binding") )
								[
									SNew(STextBlock)
									.Text(FText::Format(LOCTEXT("NativeBindingPresent", "Transition has native binding to {0}()"), FText::FromName(FunctionName)))
									.Font( IDetailLayoutBuilder::GetDetailFontBold() )
								];
							}
						}
					}
				}

				TransitionCategory.AddCustomRow( CanExecPin ? CanExecPin->PinFriendlyName : FText::GetEmpty() )
				[
					SNew(SKismetLinearExpression, CanExecPin)
				];
			}
		}

		//////////////////////////////////////////////////////////////////////////

		IDetailCategoryBuilder& CrossfadeCategory = DetailBuilder.EditCategory("BlendSettings", LOCTEXT("BlendSettingsCategoryTitle", "Blend Settings") );
		if (TransitionNode != NULL && SelectedObjects.Num() == 1)
		{
			// The sharing option for the crossfade settings
			CrossfadeCategory.AddCustomRow( LOCTEXT("TransitionCrossfadeSharingLabel", "Transition Crossfade Sharing") )
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TransitionCrossfadeSharingLabel", "Transition Crossfade Sharing"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MaxDesiredWidth(300.0f)
			[
				GetWidgetForInlineShareMenu(
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([TransNode]() { return FText::FromString(TransNode->SharedCrossfadeName); })),
					TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([TransNode]() { return TransNode->bSharedCrossfade; })),
					FOnClicked::CreateSP(this, &FAnimTransitionNodeDetails::OnPromoteToSharedClick, false), 
					FOnClicked::CreateSP(this, &FAnimTransitionNodeDetails::OnUnshareClick, false), 
					FOnGetContent::CreateSP(this, &FAnimTransitionNodeDetails::OnGetShareableNodesMenu, false))
			];
		}

		//@TODO: Gate editing these on shared non-authoritative ones
		CrossfadeCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CrossfadeDuration)).DisplayName( LOCTEXT("DurationLabel", "Duration") );
		CrossfadeCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendMode)).DisplayName(LOCTEXT("ModeLabel", "Mode"));
		CrossfadeCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, CustomBlendCurve)).DisplayName(LOCTEXT("CurveLabel", "Custom Blend Curve"));
		CrossfadeCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, BlendProfileWrapper)).DisplayName(LOCTEXT("BlendProfileLabel", "Blend Profile"));

		//////////////////////////////////////////////////////////////////////////

		IDetailCategoryBuilder& NotificationCategory = DetailBuilder.EditCategory("Notifications", LOCTEXT("NotificationsCategoryTitle", "Notifications") );
		
		AddTransitionEventField(NotificationCategory, TEXT("TransitionStart"));
		AddTransitionEventField(NotificationCategory, TEXT("TransitionEnd"));
		AddTransitionEventField(NotificationCategory, TEXT("TransitionInterrupt"));
	}

	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, TransitionStart));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, TransitionEnd));
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UAnimStateTransitionNode, TransitionInterrupt));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

/** RuleShare = true if we are sharing the rules of this transition (else we are implied to be sharing the crossfade settings) */
FReply FAnimTransitionNodeDetails::OnPromoteToSharedClick(bool RuleShare)
{
	TSharedPtr< SWindow > Parent = FSlateApplication::Get().GetActiveTopLevelWindow();
	if ( Parent.IsValid() )
	{
		// Show dialog to enter new track name
		TSharedRef<STextEntryPopup> TextEntry =
			SNew(STextEntryPopup)
			.Label( LOCTEXT("PromoteAnimTransitionNodeToSharedLabel", "Shared Transition Name") )
			.OnTextCommitted(this, &FAnimTransitionNodeDetails::PromoteToShared, RuleShare);

		// Show dialog to enter new event name
		FSlateApplication::Get().PushMenu(
			Parent.ToSharedRef(),
			FWidgetPath(),
			TextEntry,
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
			);
		TextEntryWidget = TextEntry;
	}

	return FReply::Handled();
}

void FAnimTransitionNodeDetails::PromoteToShared(const FText& NewTransitionName, ETextCommit::Type CommitInfo, bool bRuleShare)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (UAnimStateTransitionNode* TransNode = TransitionNode.Get())
		{
			if (bRuleShare)
			{
				TransNode->MakeRulesShareable(NewTransitionName.ToString());
				AssignUniqueColorsToAllSharedNodes(TransNode->GetGraph());
			}
			else
			{
				TransNode->MakeCrossfadeShareable(NewTransitionName.ToString());
			}
		}
	}

	FSlateApplication::Get().DismissAllMenus();
}

FReply FAnimTransitionNodeDetails::OnUnshareClick(bool bUnshareRule)
{
	if (UAnimStateTransitionNode* TransNode = TransitionNode.Get())
	{
		if (bUnshareRule)
		{
			TransNode->UnshareRules();
		}
		else
		{
			TransNode->UnshareCrossade();
		}
	}

	return FReply::Handled();
}

TSharedRef<SWidget> FAnimTransitionNodeDetails::OnGetShareableNodesMenu(bool bShareRules)
{
	FMenuBuilder MenuBuilder(true, NULL);

	FText SectionText;

	if (bShareRules)
	{
		SectionText = LOCTEXT("PickSharedAnimTransition", "Shared Transition Rules");
	}
	else
	{
		SectionText = LOCTEXT("PickSharedAnimCrossfadeSettings", "Shared Settings");
	}

	MenuBuilder.BeginSection("AnimTransitionSharableNodes", SectionText);

	if (UAnimStateTransitionNode* RawTransitionNode = TransitionNode.Get())
	{
		const UEdGraph* CurrentGraph = RawTransitionNode->GetGraph();

		// Collect all unique shared transitions and group them by their name.
		TMultiMap<FString, UAnimStateTransitionNode*> SharedTransitions;
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(CurrentGraph))
		{
			TArray<UAnimStateNodeBase*> StateNodes;
			FBlueprintEditorUtils::GetAllNodesOfClassEx<UAnimStateNodeBase>(Blueprint, StateNodes);

			for (UAnimStateNodeBase* StateNodeBase : StateNodes)
			{
				if (UAnimStateTransitionNode* GraphTransNode = Cast<UAnimStateTransitionNode>(StateNodeBase))
				{
					if (bShareRules && !GraphTransNode->SharedRulesName.IsEmpty())
					{
						SharedTransitions.Add(GraphTransNode->SharedRulesName, GraphTransNode);
					}

					if (!bShareRules && !GraphTransNode->SharedCrossfadeName.IsEmpty())
					{
						SharedTransitions.Add(GraphTransNode->SharedCrossfadeName, GraphTransNode);
					}
				}
			}
		}

		// Get the unique shared transition names
		TSet<FString> SharedTransitionKeys;
		SharedTransitions.GetKeys(SharedTransitionKeys);

		// Iterate through the unique shared transition names and list all the places where they are referenced in the tooltip.
		TArray<UAnimStateTransitionNode*> UsedIn;
		for (const FString& Key : SharedTransitionKeys)
		{
			UsedIn.Reset();
			SharedTransitions.MultiFind(Key, UsedIn, /*bMaintainOrder=*/true);
			if (UsedIn.IsEmpty())
			{
				continue;
			}

			FTextBuilder ToolTipBuilder;
			ToolTipBuilder.AppendLine(LOCTEXT("AnimTransitionUsedBy", "Used by:"));
			for (const UAnimStateTransitionNode* UsedInTransitionNode : UsedIn)
			{
				ToolTipBuilder.AppendLine(UsedInTransitionNode->GetGraph()->GetName());
			}

			FUIAction Action = FUIAction( FExecuteAction::CreateSP(this, &FAnimTransitionNodeDetails::BecomeSharedWith, UsedIn[0], bShareRules));
			MenuBuilder.AddMenuEntry( FText::FromString(Key), ToolTipBuilder.ToText(), FSlateIcon(), Action);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTransitionNodeDetails::BecomeSharedWith(UAnimStateTransitionNode* NewNode, bool bShareRules)
{
	if (UAnimStateTransitionNode* TransNode = TransitionNode.Get())
	{
		if (bShareRules)
		{
			TransNode->UseSharedRules(NewNode);
		}
		else
		{
			TransNode->UseSharedCrossfade(NewNode);
		}
	}
}

void FAnimTransitionNodeDetails::AssignUniqueColorsToAllSharedNodes(UEdGraph* CurrentGraph)
{
	TArray<UEdGraph*> SourceList;
	for (int32 idx=0; idx < CurrentGraph->Nodes.Num(); idx++)
	{
		if (UAnimStateTransitionNode* Node = Cast<UAnimStateTransitionNode>(CurrentGraph->Nodes[idx]))
		{
			if (Node->bSharedRules)
			{
				int32 colorIdx = SourceList.AddUnique(Node->BoundGraph)+1;

				FLinearColor SharedColor;
				SharedColor.R = (colorIdx & 1 ? 1.0f : 0.15f);
				SharedColor.G = (colorIdx & 2 ? 1.0f : 0.15f);
				SharedColor.B = (colorIdx & 4 ? 1.0f : 0.15f);
				SharedColor.A = 1.0f;

				// Storing this on the UAnimStateTransitionNode really bugs me. But its a pain to iterate over all the widget nodes at once
				// and we may want the shared color to be customizable in the details view
				Node->SharedColor = SharedColor;
			}
		}
	}
}

FReply FAnimTransitionNodeDetails::OnClickEditBlendGraph()
{
	if (UAnimStateTransitionNode* TransitionNodePtr = TransitionNode.Get())
	{
		if (TransitionNodePtr->CustomTransitionGraph != NULL)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(TransitionNodePtr->CustomTransitionGraph);
		}
	}

	return FReply::Handled();
}

EVisibility FAnimTransitionNodeDetails::GetBlendGraphButtonVisibility(bool bMultiSelect) const
{
	if(!bMultiSelect)
	{
		if (UAnimStateTransitionNode* TransitionNodePtr = TransitionNode.Get())
		{
			if (TransitionNodePtr->LogicType == ETransitionLogicType::TLT_Custom)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Hidden;
}

TSharedPtr<IPropertyHandle> FAnimTransitionNodeDetails::GetTransitionEventProperty(const IDetailCategoryBuilder& TransitionCategory, const FString& TransitionName)
{
	TSharedPtr<IPropertyHandle> NameProperty = TransitionCategory.GetParentLayout().GetProperty(*(TransitionName + TEXT(".NotifyName")));
	return NameProperty;
}

void FAnimTransitionNodeDetails::AddTransitionEventField(IDetailCategoryBuilder& TransitionCategory, const FString& TransitionName)
{
	TSharedPtr<IPropertyHandle> Property = GetTransitionEventProperty(TransitionCategory, TransitionName);
	BuildTransitionEventRow(TransitionCategory.AddProperty(Property), Property);
}

void FAnimTransitionNodeDetails::BuildTransitionEventRow(IDetailPropertyRow& Row, const TSharedPtr<IPropertyHandle>& NameProperty, const FText& InfoText)
{
	Row.CustomWidget()
	.NameContent()
	[
		SNew(SHorizontalBox)
		.ToolTipText(InfoText)
		.Visibility(EVisibility::Visible)
			
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			NameProperty->GetParentHandle()->CreatePropertyNameWidget()
		]

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(8.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SImage)
			.Visibility_Lambda([bShouldDisplay = !InfoText.IsEmpty()]()
			{
				return bShouldDisplay ? EVisibility::Visible : EVisibility::Hidden;
			})
			.Image(FAppStyle::GetBrush("Icons.Info"))
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]
	]
	.ValueContent()
	[
		NameProperty->CreatePropertyValueWidget()
	];
}

TSharedRef<SWidget> FAnimTransitionNodeDetails::GetWidgetForInlineShareMenu(const TAttribute<FText>& InSharedNameText, const TAttribute<bool>& bInIsCurrentlyShared, FOnClicked PromoteClick, FOnClicked DemoteClick, FOnGetContent GetContentMenu)
{
	return
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SComboButton )
			.ContentPadding(FMargin(4.0f, 2.0f))
			.ToolTipText(LOCTEXT("UseSharedAnimationTransition_ToolTip", "Use Shared Transition"))
			.OnGetMenuContent( GetContentMenu )
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda( [bInIsCurrentlyShared, InSharedNameText]() { return bInIsCurrentlyShared.Get() ? InSharedNameText.Get() : LOCTEXT("SharedTransition", "Use Shared"); } )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked_Lambda([bInIsCurrentlyShared, DemoteClick, PromoteClick]() { return bInIsCurrentlyShared.Get() ? DemoteClick.Execute() : PromoteClick.Execute(); } )
			.Text_Lambda([bInIsCurrentlyShared](){ return bInIsCurrentlyShared.Get() ? LOCTEXT("UnshareLabel", "Unshare") : LOCTEXT("ShareLabel", "Promote To Shared"); } )
			.TextStyle(&FAppStyle::Get(), TEXT("TinyText"))
		];
}

void FAnimTransitionNodeDetails::OnBlendProfileChanged(UBlendProfile* NewProfile, TSharedPtr<IPropertyHandle> ProfileProperty)
{
	if(ProfileProperty.IsValid())
	{
		ProfileProperty->SetValue((const UObject*&)NewProfile);
	}
}

#undef LOCTEXT_NAMESPACE
