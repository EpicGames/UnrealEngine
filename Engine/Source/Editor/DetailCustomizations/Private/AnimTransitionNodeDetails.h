// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Framework/SlateDelegates.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailPropertyRow;
class FText;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class STextEntryPopup;
class SWidget;
class UAnimStateTransitionNode;
class UBlendProfile;
class UEdGraph;

class FAnimTransitionNodeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

protected:
	static TSharedPtr<IPropertyHandle> GetTransitionEventProperty(const IDetailCategoryBuilder& TransitionCategory, const FString& TransitionName);
	static void AddTransitionEventField(IDetailCategoryBuilder& TransitionCategory, const FString& TransitionName);
	static void BuildTransitionEventRow(IDetailPropertyRow& Row, const TSharedPtr<IPropertyHandle>& NameProperty, const FText& InfoText = FText::GetEmpty());

	FReply OnClickEditBlendGraph();
	EVisibility GetBlendGraphButtonVisibility(bool bMultiSelect) const;

	TSharedRef<SWidget> GetWidgetForInlineShareMenu(const TAttribute<FText>& InSharedNameText, const TAttribute<bool>& bInIsCurrentlyShared, FOnClicked PromoteClick, FOnClicked DemoteClick, FOnGetContent GetContentMenu);

	FReply OnPromoteToSharedClick(bool RuleShare);
	void PromoteToShared(const FText& NewTransitionName, ETextCommit::Type CommitInfo, bool bRuleShare);
	FReply OnUnshareClick(bool bUnshareRule);
	TSharedRef<SWidget> OnGetShareableNodesMenu(bool bShareRules);
	void BecomeSharedWith(UAnimStateTransitionNode* NewNode, bool bShareRules);
	void AssignUniqueColorsToAllSharedNodes(UEdGraph* CurrentGraph);

	void OnBlendProfileChanged(UBlendProfile* NewProfile, TSharedPtr<IPropertyHandle> ProfileProperty);

private:
	TWeakObjectPtr<UAnimStateTransitionNode> TransitionNode;
	TSharedPtr<STextEntryPopup> TextEntryWidget;
};

