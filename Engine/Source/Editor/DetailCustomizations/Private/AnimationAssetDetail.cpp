// Copyright Epic Games, Inc. All Rights Reservekd.

#include "Animation/AnimationAsset.h"
#include "Animation/PoseAsset.h"
#include "Animation/Skeleton.h"
#include "AnimationAssetDetails.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailCustomization;
class UObject;
struct FAssetData;

#define LOCTEXT_NAMESPACE	"AnimationAssetDetails"


TSharedRef<IDetailCustomization> FAnimationAssetDetails::MakeInstance()
{
	return MakeShareable(new FAnimationAssetDetails);
}

void FAnimationAssetDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjectsList = DetailBuilder.GetSelectedObjects();

	for (auto SelectionIt = SelectedObjectsList.CreateConstIterator(); SelectionIt; ++SelectionIt)
	{
		if (UAnimationAsset* TestAsset = Cast<UAnimationAsset>(SelectionIt->Get()))
		{
			if (TargetSkeleton.IsValid() && !TestAsset->GetSkeleton()->IsCompatibleForEditor(TargetSkeleton.Get()))
			{
				TargetSkeleton = nullptr;
				break;
			}
			else
			{
				TargetSkeleton = TestAsset->GetSkeleton();
			}
		}
	}

	/////////////////////////////////
	// Source Animation filter for skeleton
	PreviewPoseAssetHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimationAsset, PreviewPoseAsset));

	// add widget for editing source animation 
	IDetailCategoryBuilder& AnimationCategory = DetailBuilder.EditCategory("Animation");
	AnimationCategory
	.AddCustomRow(PreviewPoseAssetHandler->GetPropertyDisplayName())
	.RowTag(PreviewPoseAssetHandler->GetProperty()->GetFName())
	.NameContent()
	[
		PreviewPoseAssetHandler->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.f)
	[
		SNew(SObjectPropertyEntryBox)
		.AllowedClass(UPoseAsset::StaticClass())
		.OnShouldFilterAsset(this, &FAnimationAssetDetails::ShouldFilterAsset)
		.PropertyHandle(PreviewPoseAssetHandler)
	];

	DetailBuilder.HideProperty(PreviewPoseAssetHandler);
}

void FAnimationAssetDetails::OnPreviewPoseAssetChanged(const FAssetData& AssetData)
{
	ensureAlways(PreviewPoseAssetHandler->SetValue(AssetData) == FPropertyAccess::Result::Success);;
}

bool FAnimationAssetDetails::ShouldFilterAsset(const FAssetData& AssetData)
{
	if (TargetSkeleton.IsValid())
	{
		return !TargetSkeleton->IsCompatibleForEditor(AssetData);
	}

	return true;
}


#undef LOCTEXT_NAMESPACE
