// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "IDetailCustomization.h"

struct FAssetData;
class IDetailLayoutBuilder;
class IPropertyHandle;
class USkeleton;

class FAnimationAssetDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
private:
	TWeakObjectPtr<USkeleton> TargetSkeleton;

	// property handler
	TSharedPtr<IPropertyHandle> PreviewPoseAssetHandler;

	// replacing source animation
	void OnPreviewPoseAssetChanged(const FAssetData& AssetData);
	bool ShouldFilterAsset(const FAssetData& AssetData);
};

