// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetImportDataCustomization.h"

#include "Containers/UnrealString.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorFramework/AssetImportData.h"
#include "EditorReimportHandler.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AssetImportDataCustomization"

TSharedRef<IPropertyTypeCustomization> FAssetImportDataCustomization::MakeInstance()
{
	return MakeShareable( new FAssetImportDataCustomization );
}

void FAssetImportDataCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils )
{
}

void FAssetImportDataCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
{
	PropertyHandle = InPropertyHandle;

	auto Font = IDetailLayoutBuilder::GetDetailFont();

	FAssetImportInfo* Info = GetEditStruct();
	if (Info)
	{
		const FText SourceFileText = LOCTEXT("SourceFile", "Source File");
		int32 NumSourceFiles = Info->SourceFiles.Num() ? Info->SourceFiles.Num() : 1;

		for (int32 Index = 0; Index < NumSourceFiles; ++Index)
		{
			FText SourceFileLabel = SourceFileText;
			if (Info->SourceFiles.IsValidIndex(Index) && Info->SourceFiles[Index].DisplayLabelName.Len() > 0)
			{
				SourceFileLabel = FText::FromString(SourceFileText.ToString() + TEXT(" (") + Info->SourceFiles[Index].DisplayLabelName + TEXT(")"));
			}

			ChildBuilder.AddCustomRow(SourceFileLabel)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(SourceFileLabel)
				.Font(Font)
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			.MaxDesiredWidth(TOptional<float>())
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SEditableText)
					.IsReadOnly(true)
					.Text(this, &FAssetImportDataCustomization::GetFilenameText, Index)
					.ToolTipText(this, &FAssetImportDataCustomization::GetFilenameText, Index)
					.Font(Font)
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &FAssetImportDataCustomization::OnChangePathClicked, Index)
					.ToolTipText(LOCTEXT("ChangePath_Tooltip", "Browse for a new source file path"))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("...", "..."))
						.Font(Font)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &FAssetImportDataCustomization::OnClearPathClicked, Index)
					.ToolTipText(LOCTEXT("ClearPath_Tooltip", "Clear this source file information from the asset"))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Cross"))
					]
				]
			];

			ChildBuilder.AddCustomRow(SourceFileText)
				.ValueContent()
				.HAlign(HAlign_Fill)
				.MaxDesiredWidth(TOptional<float>())
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &FAssetImportDataCustomization::GetTimestampText, Index)
						.Font(Font)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.IsEnabled(this, &FAssetImportDataCustomization::IsPropagateFromAbovePathEnable, Index)
						.OnClicked(this, &FAssetImportDataCustomization::OnPropagateFromAbovePathClicked, Index)
						.ToolTipText(LOCTEXT("PropagateFromAbovePath_Tooltip", "Use the above source path to set this path."))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("ArrowDown"))
						]
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.IsEnabled(this, &FAssetImportDataCustomization::IsPropagateFromBelowPathEnable, Index)
						.OnClicked(this, &FAssetImportDataCustomization::OnPropagateFromBelowPathClicked, Index)
						.ToolTipText(LOCTEXT("PropagateFromBelowPath_Tooltip", "Use the below source path to set this path."))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("ArrowUp"))
						]
					]
				];
		}
	}
	else
	{
		ChildBuilder.AddCustomRow(FText::GetEmpty())
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ClearAllSourceData", "Clear all the source file information from the asset."))
				.Font(Font)
			]
			.ValueContent()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FAssetImportDataCustomization::OnClearAllPathsClicked)
				.ToolTipText(LOCTEXT("ClearAllSourceData_Tooltip", "Clear all the source file information from the asset."))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Cross"))
				]
			];
	}
}

FText FAssetImportDataCustomization::GetFilenameText(int32 Index) const
{
	FAssetImportInfo* Info = GetEditStruct();
	if (Info)
	{
		if (Info->SourceFiles.IsValidIndex(Index))
		{
			return FText::FromString(Info->SourceFiles[Index].RelativeFilename);
		}
	}
	return LOCTEXT("NoFilenameFound", "No Source Path Set");
}

FText FAssetImportDataCustomization::GetTimestampText(int32 Index) const
{
	FAssetImportInfo* Info = GetEditStruct();
	if (Info)
	{
		if (Info->SourceFiles.IsValidIndex(Index))
		{
			return FText::FromString(Info->SourceFiles[Index].Timestamp.ToString());
		}
	}
	return FText();
}


FAssetImportInfo* FAssetImportDataCustomization::GetEditStruct() const
{
	TArray<FAssetImportInfo*> AssetImportInfo;

	if(PropertyHandle->IsValidHandle())
	{
		PropertyHandle->AccessRawData(reinterpret_cast<TArray<void*>&>(AssetImportInfo));
	}

	if (AssetImportInfo.Num() == 1)
	{
		return AssetImportInfo[0];
	}
	return nullptr;
}

TArray<FAssetImportInfo*> FAssetImportDataCustomization::GetEditStructs() const
{
	TArray<FAssetImportInfo*> AssetImportInfo;

	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->AccessRawData(reinterpret_cast<TArray<void*>&>(AssetImportInfo));
	}

	return AssetImportInfo;
}

UAssetImportData* FAssetImportDataCustomization::GetOuterClass() const
{
	static TArray<UObject*> OuterObjects;
	OuterObjects.Reset();

	PropertyHandle->GetOuterObjects(OuterObjects);

	return OuterObjects.Num() ? Cast<UAssetImportData>(OuterObjects[0]) : nullptr;
}

TArray<UAssetImportData*> FAssetImportDataCustomization::GetAllAssetImportData() const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	TArray<UAssetImportData*> AllAssetImportData;
	for (UObject* Data : OuterObjects)
	{
		AllAssetImportData.Add(Cast<UAssetImportData>(Data));
	}

	return AllAssetImportData;
}

class FImportDataSourceFileTransactionScope
{
public:
	FImportDataSourceFileTransactionScope(FText TransactionName, UAssetImportData* InImportData)
	{
		check(InImportData);
		ImportData = InImportData;
		FScopedTransaction Transaction(TransactionName);

		bIsTransactionnal = (ImportData->GetFlags() & RF_Transactional) > 0;
		if (!bIsTransactionnal)
		{
			ImportData->SetFlags(RF_Transactional);
		}

		ImportData->Modify();
	}
	
	~FImportDataSourceFileTransactionScope()
	{
		if (!bIsTransactionnal)
		{
			ImportData->ClearFlags(RF_Transactional);
		}
		ImportData->MarkPackageDirty();
	}
private:
	bool bIsTransactionnal;
	UAssetImportData* ImportData;
};

FReply FAssetImportDataCustomization::OnChangePathClicked(int32 Index) const
{
	UAssetImportData* ImportData = GetOuterClass();
	UObject* Obj = ImportData ? ImportData->GetOuter() : nullptr;

	FAssetImportInfo* Info = GetEditStruct();

	if (!Obj)
	{
		return FReply::Handled();
	}

	TArray<FString> OpenFilenames;
	FReimportManager::Instance()->GetNewReimportPath(Obj, OpenFilenames);
	if (OpenFilenames.Num() == 1 && !OpenFilenames[0].IsEmpty())
	{
		FImportDataSourceFileTransactionScope TransactionScope(LOCTEXT("SourceReimportChangePath", "Change source file path"), ImportData);
		if (!Info || !Info->SourceFiles.IsValidIndex(Index))
		{
			ImportData->UpdateFilenameOnly(FPaths::ConvertRelativePathToFull(OpenFilenames[0]));
		}
		else
		{
			ImportData->UpdateFilenameOnly(FPaths::ConvertRelativePathToFull(OpenFilenames[0]), Index);
		}
	}
	return FReply::Handled();
}

FReply FAssetImportDataCustomization::OnClearPathClicked(int32 Index) const
{
	UAssetImportData* ImportData = GetOuterClass();
	if (ImportData && ImportData->SourceData.SourceFiles.IsValidIndex(Index))
	{
		FImportDataSourceFileTransactionScope TransactionScope(LOCTEXT("SourceReimportClearPath", "Clear Source file path"), ImportData);
		//Clear the filename, Hash and timestamp. Leave the Display label.
		ImportData->SourceData.SourceFiles[Index].RelativeFilename = FString();
		ImportData->SourceData.SourceFiles[Index].FileHash = FMD5Hash();
		ImportData->SourceData.SourceFiles[Index].Timestamp = 0;
	}

	return FReply::Handled();
}

FReply FAssetImportDataCustomization::OnClearAllPathsClicked() const
{
	TArray<UAssetImportData*> ImportDatas = GetAllAssetImportData();

	FScopedTransaction Transaction(LOCTEXT("ClearAllSourceFilePaths", "Clear all source file paths"));
	for (UAssetImportData* ImportData : ImportDatas)
	{
		FImportDataSourceFileTransactionScope TransactionScope(LOCTEXT("SourceReimportClearPath", "Clear Source file path"), ImportData);
		
		//Clear the filename, Hash and timestamp. Leave the Display label.
		for (FAssetImportInfo::FSourceFile& SourceFile : ImportData->SourceData.SourceFiles)
		{
			SourceFile.RelativeFilename = FString();
			SourceFile.FileHash = FMD5Hash();
			SourceFile.Timestamp = 0;
		}
	}

	return FReply::Handled();
}

bool FAssetImportDataCustomization::IsPropagateFromAbovePathEnable(int32 Index) const
{
	UAssetImportData* ImportData = GetOuterClass();
	return (ImportData && ImportData->SourceData.SourceFiles.IsValidIndex(Index) && ImportData->SourceData.SourceFiles.IsValidIndex(Index - 1));
}

bool FAssetImportDataCustomization::IsPropagateFromBelowPathEnable(int32 Index) const
{
	UAssetImportData* ImportData = GetOuterClass();
	return (ImportData && ImportData->SourceData.SourceFiles.IsValidIndex(Index) && ImportData->SourceData.SourceFiles.IsValidIndex(Index + 1));
}

void FAssetImportDataCustomization::PropagatePath(int32 SrcIndex, int32 DstIndex) const
{
	UAssetImportData* ImportData = GetOuterClass();
	if (ImportData && ImportData->SourceData.SourceFiles.IsValidIndex(DstIndex) && ImportData->SourceData.SourceFiles.IsValidIndex(SrcIndex))
	{
		FImportDataSourceFileTransactionScope TransactionScope(LOCTEXT("SourceReimportPropagateFromAbove", "Propagate source file path"), ImportData);
		FString OriginalLabel = ImportData->SourceData.SourceFiles[DstIndex].DisplayLabelName;
		ImportData->SourceData.SourceFiles[DstIndex] = ImportData->SourceData.SourceFiles[SrcIndex];
		ImportData->SourceData.SourceFiles[DstIndex].DisplayLabelName = OriginalLabel;
	}
}

FReply FAssetImportDataCustomization::OnPropagateFromAbovePathClicked(int32 Index) const
{
	PropagatePath(Index - 1, Index);
	return FReply::Handled();
}


FReply FAssetImportDataCustomization::OnPropagateFromBelowPathClicked(int32 Index) const
{
	PropagatePath(Index + 1, Index);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
