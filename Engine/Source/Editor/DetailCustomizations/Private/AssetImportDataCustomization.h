// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class UAssetImportData;
struct FAssetImportInfo;

class FAssetImportDataCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;

private:

	/** Handle the user wanting to change a specific source path */
	FReply OnChangePathClicked(int32 Index) const;

	/** Handle the user requesting that the specified index be cleared */
	FReply OnClearPathClicked(int32 Index) const;

	FReply OnClearAllPathsClicked() const;

	/** Handle the user requesting that the Path use at Index - 1 will be replace the path at Index */
	FReply OnPropagateFromAbovePathClicked(int32 Index) const;
	bool IsPropagateFromAbovePathEnable(int32 Index) const;
	
	/** Handle the user requesting that the Path use at Index + 1 will be replace the path at Index */
	FReply OnPropagateFromBelowPathClicked(int32 Index) const;
	bool IsPropagateFromBelowPathEnable(int32 Index) const;

	/** Access the struct we are editing - returns null if we have more than one. */
	FAssetImportInfo* GetEditStruct() const;

	/** Access all the structs we are editing */
	TArray<FAssetImportInfo*> GetEditStructs() const;

	/** Access the outer class that contains this struct */
	UAssetImportData* GetOuterClass() const;

	TArray<UAssetImportData*> GetAllAssetImportData() const;

	/** Get text for the UI */
	FText GetFilenameText(int32 Index) const;
	FText GetTimestampText(int32 Index) const;

private:

	void PropagatePath(int32 SrcIndex, int32 DstIndex) const;
	/** Property handle of the property we're editing */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
