// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class SSettingsEditorCheckoutNotice;
class UAssetViewerSettings;

class FAssetViewerSettingsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface
protected:
	// Callbacks for customized SEditableTextBox
	FText OnGetProfileName() const;
	void OnProfileNameChanged(const FText& InNewText);
	void OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
	
	// Check whether or not the given profile name is valid by cross-referencing it with existing names
	const bool IsProfileNameValid(const FString& NewName);
	bool CanSetSharedProfile() const;
	EVisibility ShowFileWatcherWidget() const;
	FString SharedProfileConfigFilePath() const;
private:
	// Customized name edit text box used for the profile name
	TSharedPtr<SEditableTextBox> NameEditTextBox;
	// Cached data
	TSharedPtr<IPropertyHandle> NameProperty;
	/** Watcher widget for the default config file (checks file status / SCC state). */
	TSharedPtr<SSettingsEditorCheckoutNotice> FileWatcherWidget;

	int32 ProfileIndex;
	UAssetViewerSettings* ViewerSettings;
	bool bValidProfileName;
};
