// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttenuationSettingsCustomizations.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Attenuation.h"
#include "Fonts/SlateFontInfo.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundAttenuation.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AttenuationSettings"

namespace AttenuationSettingsUtils
{
	bool GetValue(TSharedPtr<IPropertyHandle> InProp)
	{
		if (InProp.IsValid())
		{
			bool Val;
			InProp->GetValue(Val);
			return Val;
		}
		return true;
	}

	void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
	{
		int32 AttenuationSortIndexBase = 0;
		if (const IDetailCategoryBuilder* AttenuationBuilder = AllCategoryMap.FindRef("Attenuation"))
		{
			AttenuationSortIndexBase = AttenuationBuilder->GetSortOrder();
		}

		static const TArray<FName> OrderedCategories = {
			FName("AttenuationDistance"),
			FName("AttenuationSpatialization"),
			FName("AttenuationAirAbsorption"),
			FName("AttenuationListenerFocus"),
			FName("AttenuationOcclusion"),
			FName("AttenuationSubmixSend"),
			FName("AttenuationReverbSend"),
			FName("AttenuationPriority"),
			FName("AttenuationPluginSettings"),
			FName("AttenuationSourceDataOverride"),
			FName("AttenuationAudioLink"),
		};
		static const int32 FirstNonAttenuationIndexOffsetFromBase = OrderedCategories.Num() + 1;
		
		for (const TPair<FName, IDetailCategoryBuilder*>& Pair : AllCategoryMap)
		{
			const FName CategoryName = Pair.Key;

			// Early out if attenuation category
			if (CategoryName == "Attenuation")
			{
				continue;
			}

			int32 CategoryNewSortIndex;
			const int32 CategoryIndex = OrderedCategories.Find(CategoryName);
			if (CategoryIndex != INDEX_NONE)
			{
				CategoryNewSortIndex = AttenuationSortIndexBase + CategoryIndex + 1;
			}
			else
			{
				// Add space to any other categories that might be interfering with space for attenuation-related categories
				const int32 UnrelatedCategorySortIndex = Pair.Value->GetSortOrder();
				if (AttenuationSortIndexBase <= UnrelatedCategorySortIndex)
				{
					CategoryNewSortIndex = UnrelatedCategorySortIndex + FirstNonAttenuationIndexOffsetFromBase;
				}
				// Otherwise, this unrelated category index doesn't need to be moved around (it's already before us)
				else
				{
					continue;
				}
			}

			Pair.Value->SetSortOrder(CategoryNewSortIndex);
		}
	}
}

TSharedRef<IPropertyTypeCustomization> FSoundAttenuationSettingsCustomization::MakeInstance() 
{
	return MakeShared<FSoundAttenuationSettingsCustomization>();
}

TSharedRef<IPropertyTypeCustomization> FForceFeedbackAttenuationSettingsCustomization::MakeInstance() 
{
	return MakeShared<FForceFeedbackAttenuationSettingsCustomization>();
}

void FBaseAttenuationSettingsCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

TSharedPtr<IPropertyHandle> FBaseAttenuationSettingsCustomization::GetOverrideAttenuationHandle(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	TSharedPtr<IPropertyHandle> ParentHandle = StructPropertyHandle->GetParentHandle();
	if (TSharedPtr<IPropertyHandle> GrandParentHandle = ParentHandle->GetParentHandle())
	{
		ParentHandle = GrandParentHandle;
	}
	return ParentHandle->GetChildHandle(TEXT("bOverrideAttenuation"), true);
}

void FBaseAttenuationSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
	 
	// Get the override attenuation handle, if it exists
	bOverrideAttenuationHandle = GetOverrideAttenuationHandle(StructPropertyHandle);

	for (uint32 ChildIndex = 0; ChildIndex <NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	AttenuationShapeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, AttenuationShape));
	DistanceAlgorithmHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, DistanceAlgorithm));

	TSharedRef<IPropertyHandle> AttenuationExtentsHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, AttenuationShapeExtents)).ToSharedRef();

	uint32 NumExtentChildren;
	AttenuationExtentsHandle->GetNumChildren(NumExtentChildren);

	TSharedPtr<IPropertyHandle> ExtentXHandle;
	TSharedPtr<IPropertyHandle> ExtentYHandle;
	TSharedPtr<IPropertyHandle> ExtentZHandle;

	for(uint32 ExtentChildIndex = 0; ExtentChildIndex <NumExtentChildren; ++ExtentChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = AttenuationExtentsHandle->GetChildHandle(ExtentChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FVector, X))
		{
			ExtentXHandle = ChildHandle;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FVector, Y))
		{
			ExtentYHandle = ChildHandle;
		}
		else
		{
			check(PropertyName == GET_MEMBER_NAME_CHECKED(FVector, Z));
			ExtentZHandle = ChildHandle;
		}
	}

	// Get layout build of category so properties can be added to categories
	IDetailLayoutBuilder& LayoutBuilder = ChildBuilder.GetParentCategory().GetParentLayout();

	LayoutBuilder.AddPropertyToCategory(DistanceAlgorithmHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, CustomAttenuationCurve)))
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsCustomCurveSelected))
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);

	DbAttenuationAtMaxHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, dBAttenuationAtMax));
	LayoutBuilder.AddPropertyToCategory(DbAttenuationAtMaxHandle)
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsNaturalSoundSelected))
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, FalloffMode)))
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsNaturalSoundSelected))
		.EditCondition(GetIsFalloffModeEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(AttenuationShapeHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);

	// Get the attenuation category directly here otherwise our category is going to be incorrect for the following custom rows (e.g. "Vector" vs "Attenuation")
	IDetailCategoryBuilder& AttenuationCategory = LayoutBuilder.EditCategory("AttenuationDistance");

	// Attenuation shapes all use the same FVector to store the shape's parameters.
	// All shapes don't need 3 numbers however, so we only show the relevant ones for each shape.
	// The Copy/Paste Category feature of the editor expects property handles to be referred exactly once per category, so we can't have multiple
	// rows for the same property just to, say, change its label/tooltip.
	// The unreal way is to use slate attributes for customization, and instantiate the property rows exactly once for each coordinate of the vector.
	
	// ExtentsX customization
	{
		TAttribute<FText> ExtentXLabelAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsSphereSelected() == EVisibility::Visible)
				{
					return LOCTEXT("RadiusLabel", "Inner Radius");
				}
				if (IsCapsuleSelected() == EVisibility::Visible)
				{
					return LOCTEXT("CapsuleHalfHeightLabel", "Capsule Half Height");
				}
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeRadiusLabel", "Cone Radius");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxXLabel", "Extents X");
				}
				return FText();
			});
		TAttribute<FText> ExtentXTooltipAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsSphereSelected() == EVisibility::Visible)
				{
					return LOCTEXT("RadiusToolTip", "The radius that defines when sound attenuation begins (or when a custom attenuation curve begins). Sounds played at a distance less than this will not be attenuated.");
				}
				if (IsCapsuleSelected() == EVisibility::Visible)
				{
					return LOCTEXT("CapsuleHalfHeightToolTip", "The attenuation capsule's half height.");
				}
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeRadiusToolTip", "The attenuation cone's radius.");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxXTooltip", "The X dimension of the box");
				}
				return FText();
			});
		
		AttenuationCategory.AddProperty(ExtentXHandle)
			.Visibility(IsAttenuationOverriddenVisibleAttribute())
			.IsEnabled(GetIsAttenuationEnabledAttribute())
			.CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Text(ExtentXLabelAttribute)
				.ToolTipText(ExtentXTooltipAttribute)
				.Font(StructCustomizationUtils.GetRegularFont())
				.IsEnabled(GetIsAttenuationEnabledAttribute())
			]
			.ValueContent()
			[
				ExtentXHandle->CreatePropertyValueWidget()
			];
	}
	
	// ExtentsY customization
	{
		TAttribute<FText> ExtentYLabelAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsCapsuleSelected() == EVisibility::Visible)
				{
					return LOCTEXT("CapsuleRadiusLabel", "Capsule Radius");
				}
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeAngleLabel", "Cone Angle");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxYLabel", "Extents Y");
				}
				return FText();
			});
		TAttribute<FText> ExtentYTooltipAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsCapsuleSelected() == EVisibility::Visible)
				{
					return LOCTEXT("CapsuleRadiusToolTip", "The attenuation capsule's radius.");
				}
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeAngleToolTip", "The angle of the inner edge of the attenuation cone's falloff. Inside this angle sounds will be at full volume.");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxYTooltip", "The Y dimension of the box");
				}
				return FText();
			});
		
		AttenuationCategory.AddProperty(ExtentYHandle)
			.IsEnabled(GetIsAttenuationEnabledAttribute())
			.Visibility(TAttribute<EVisibility>::CreateSPLambda(this, [this]()
			{
				return
					!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle)
		   				? EVisibility::Hidden
		   				: IsSphereSelected() == EVisibility::Hidden
							? EVisibility::Visible
							: EVisibility::Hidden;
			}))
			.CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Text(ExtentYLabelAttribute)
				.ToolTipText(ExtentYTooltipAttribute)
				.Font(StructCustomizationUtils.GetRegularFont())
				.IsEnabled(GetIsAttenuationEnabledAttribute())
			]
			.ValueContent()
			[
				ExtentYHandle->CreatePropertyValueWidget()
			];
	}

	// ExtentsZ customization
	{
		TAttribute<FText> ExtentZLabelAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeFalloffAngleLabel", "Cone Falloff Angle");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxZLabel", "Extents Z");
				}
				return FText();
			});
		TAttribute<FText> ExtentZTooltipAttribute = TAttribute<FText>::CreateSPLambda(this, [this]()
			{
				if (IsConeSelected() == EVisibility::Visible)
				{
					return LOCTEXT("ConeFalloffAngleToolTip", "The angle of the outer edge of the attenuation cone's falloff. Outside this angle sounds will be inaudible.");
				}
				if (IsBoxSelected() == EVisibility::Visible)
				{
					return LOCTEXT("BoxZTooltip", "The Z dimension of the box");
				}
				return FText();
			});
		
		AttenuationCategory.AddProperty(ExtentZHandle)
			.IsEnabled(GetIsAttenuationEnabledAttribute())
			.Visibility(TAttribute<EVisibility>::CreateSPLambda(this, [this]()
			{
				if (IsConeSelected() == EVisibility::Visible || IsBoxSelected() == EVisibility::Visible)
				{
					return EVisibility::Visible;
				}
				return EVisibility::Hidden;
			}))
			.CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Text(ExtentZLabelAttribute)
				.ToolTipText(ExtentZTooltipAttribute)
				.Font(StructCustomizationUtils.GetRegularFont())
				.IsEnabled(GetIsAttenuationEnabledAttribute())
			]
			.ValueContent()
			[
				ExtentZHandle->CreatePropertyValueWidget()
			];
	}

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, ConeOffset)))
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsConeSelected))
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);	

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, FalloffDistance)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, ConeSphereRadius)))
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsConeSelected))
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);	

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FBaseAttenuationSettings, ConeSphereFalloffDistance)))
		.Visibility(TAttribute<EVisibility>(this, &FBaseAttenuationSettingsCustomization::IsConeSelected))
		.EditCondition(GetIsAttenuationEnabledAttribute(), nullptr);
}

TAttribute<bool> FBaseAttenuationSettingsCustomization::IsAttenuationOverriddenAttribute() const
{
	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([bInOverrideAttenuationHandle = bOverrideAttenuationHandle]()
		{
			return AttenuationSettingsUtils::GetValue(bInOverrideAttenuationHandle);
		}
	));
}

TAttribute<EVisibility> FBaseAttenuationSettingsCustomization::IsAttenuationOverriddenVisibleAttribute() const
{
	return TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([bInOverrideAttenuationHandle = bOverrideAttenuationHandle]()
		{
			return AttenuationSettingsUtils::GetValue(bInOverrideAttenuationHandle)
				? EVisibility::Visible
				: EVisibility::Hidden;
		}
	));
}

TAttribute<bool> FBaseAttenuationSettingsCustomization::GetIsAttenuationEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsAttenuatedPropertyWeakPtr = bIsAttenuatedHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsAttenuatedPropertyWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakHandle = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsAttenuatedPropertyWeakHandle = bIsAttenuatedPropertyWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationPropertyWeakHandle);
		Value &= AttenuationSettingsUtils::GetValue(bIsAttenuatedPropertyWeakHandle);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FBaseAttenuationSettingsCustomization::GetIsFalloffModeEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsAttenuatedPropertyWeakPtr = bIsAttenuatedHandle;
	TWeakPtr<IPropertyHandle> DbAttenuationAtMaxHandleWeakPtr = DbAttenuationAtMaxHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsAttenuatedPropertyWeakPtr, DbAttenuationAtMaxHandleWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationPropertyPtr = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsAttenuatedPropertyPtr = bIsAttenuatedPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> DbAttenuationAtMaxHandlePtr = DbAttenuationAtMaxHandleWeakPtr.Pin();

		float AttenuationValue = -60.f;
		if (DbAttenuationAtMaxHandlePtr.IsValid())
		{
			DbAttenuationAtMaxHandlePtr->GetValue(AttenuationValue);
		}

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationPropertyPtr);
		Value &= AttenuationSettingsUtils::GetValue(bIsAttenuatedPropertyPtr);
		Value &= AttenuationValue> -60.f;
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

void FSoundAttenuationSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Property handle here is the base struct. We are going to hide it since we're showing it's properties directly.
	PropertyHandle->MarkHiddenByCustomization();
}

void FSoundAttenuationSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Get handle to layout builder to enable adding properties to categories
	IDetailLayoutBuilder& LayoutBuilder = ChildBuilder.GetParentCategory().GetParentLayout();

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
	for(uint32 ChildIndex = 0; ChildIndex <NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Get the override attenuation handle, if it exists
	bOverrideAttenuationHandle = GetOverrideAttenuationHandle(StructPropertyHandle);

	bIsOcclusionEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableOcclusion)).ToSharedRef();
	bIsSpatializedHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bSpatialize)).ToSharedRef();
	bIsAirAbsorptionEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bAttenuateWithLPF)).ToSharedRef();
	bIsReverbSendEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableReverbSend)).ToSharedRef();
	bIsPriorityAttenuationEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnablePriorityAttenuation)).ToSharedRef();
	bIsSubmixSendAttenuationEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableSubmixSends)).ToSharedRef();
	bIsSourceDataOverrideEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableSourceDataOverride)).ToSharedRef();
	bIsSendToAudioLinkEnabledHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableSendToAudioLink)).ToSharedRef();
	ReverbSendMethodHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ReverbSendMethod)).ToSharedRef();
	PriorityAttenuationMethodHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PriorityAttenuationMethod)).ToSharedRef();
	AbsorptionMethodHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, AbsorptionMethod)).ToSharedRef();

	bIsFocusedHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableListenerFocus)).ToSharedRef();

	// Set protected member so FBaseAttenuationSettingsCustomization knows how to make attenuation settings editable
	bIsAttenuatedHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bAttenuate)).ToSharedRef();

	LayoutBuilder.AddPropertyToCategory(bIsAttenuatedHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute());

	FBaseAttenuationSettingsCustomization::CustomizeChildren(StructPropertyHandle, ChildBuilder, StructCustomizationUtils);

	LayoutBuilder.AddPropertyToCategory(bIsSpatializedHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute());

	// Check to see if a spatialization plugin is enabled
	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, SpatializationAlgorithm)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, BinauralRadius)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonSpatializedRadiusMode)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);
	
	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonSpatializedRadiusEnd)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);
	
	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonSpatializedRadiusStart)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, StereoSpread)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bApplyNormalizationToStereoSounds)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSpatializationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bAttenuateWithLPF)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableListenerFocus)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(AbsorptionMethodHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);
	
	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, LPFRadiusMin)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, LPFRadiusMax)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, LPFFrequencyAtMin)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, LPFFrequencyAtMax)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, HPFFrequencyAtMin)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, HPFFrequencyAtMax)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableLogFrequencyScaling)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, CustomLowpassAirAbsorptionCurve)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsCustomAirAbsorptionCurveSelected))
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, CustomHighpassAirAbsorptionCurve)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsCustomAirAbsorptionCurveSelected))
		.EditCondition(GetIsAirAbsorptionEnabledAttribute(), nullptr);

	// Add the reverb send enabled handle
	LayoutBuilder.AddPropertyToCategory(bIsReverbSendEnabledHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(ReverbSendMethodHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ReverbWetLevelMin)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearMethodSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ReverbWetLevelMax)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearMethodSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, CustomReverbSendCurve)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsCustomReverbSendCurveSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ReverbDistanceMin)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearOrCustomReverbMethodSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ReverbDistanceMax)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearOrCustomReverbMethodSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ManualReverbSendLevel)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsManualReverbSendSelected))
		.EditCondition(GetIsReverbSendEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusAzimuth)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonFocusAzimuth)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusDistanceScale)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonFocusDistanceScale)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusPriorityScale)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonFocusPriorityScale)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusVolumeAttenuation)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, NonFocusVolumeAttenuation)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableFocusInterpolation)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusAttackInterpSpeed)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, FocusReleaseInterpSpeed)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsFocusEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(bIsOcclusionEnabledHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, OcclusionTraceChannel)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsOcclusionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, OcclusionLowPassFilterFrequency)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsOcclusionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, OcclusionVolumeAttenuation)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsOcclusionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, OcclusionInterpolationTime)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsOcclusionEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bUseComplexCollisionForOcclusion)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsOcclusionEnabledAttribute(), nullptr);

	// Add the attenuation priority
	LayoutBuilder.AddPropertyToCategory(bIsPriorityAttenuationEnabledHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PriorityAttenuationMethodHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PriorityAttenuationMin)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsPriorityAttenuationLinearMethodSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PriorityAttenuationMax)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsPriorityAttenuationLinearMethodSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, CustomPriorityAttenuationCurve)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsCustomPriorityAttenuationCurveSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PriorityAttenuationDistanceMin)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearOrCustomPriorityAttenuationSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PriorityAttenuationDistanceMax)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsLinearOrCustomPriorityAttenuationSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, ManualPriorityAttenuation)))
		.Visibility(TAttribute<EVisibility>(this, &FSoundAttenuationSettingsCustomization::IsManualPriorityAttenuationSelected))
		.EditCondition(GetIsPriorityAttenuationEnabledAttribute(), nullptr);

	// Add the submix send priority
	LayoutBuilder.AddPropertyToCategory(bIsSubmixSendAttenuationEnabledHandle)
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, SubmixSendSettings)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(GetIsSubmixSendAttenuationEnabledAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableSourceDataOverride)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, bEnableSendToAudioLink)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	LayoutBuilder.AddPropertyToCategory(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, AudioLinkSettingsOverride)))
		.Visibility(IsAttenuationOverriddenVisibleAttribute())
		.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);

	TSharedPtr<IPropertyHandle> PluginProperty = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSoundAttenuationSettings, PluginSettings));
	uint32 NumPluginChildren = 0;
	PluginProperty->GetNumChildren(NumPluginChildren);
	for (uint32 i = 0; i < NumPluginChildren; ++i)
	{
		LayoutBuilder.AddPropertyToCategory(PluginProperty->GetChildHandle(i))
			.Visibility(IsAttenuationOverriddenVisibleAttribute())
			.EditCondition(IsAttenuationOverriddenAttribute(), nullptr);
	}

	// Paul Tol's Muted qualitative color scheme (colorblind-safe palette; sRGB)
	// https://sronpersonalpages.nl/~pault/data/colourschemes.pdf
	constexpr FLinearColor CategoryColors[] = 
	{
		FLinearColor(FColor(51,   34, 136)), // #332288 indigo
		FLinearColor(FColor(221, 204, 119)), // #DDCC77 sand
		FLinearColor(FColor(17,  119, 51)),  // #117733 green
		FLinearColor(FColor(170,  68, 153)), // #AA4499 purple
		FLinearColor(FColor(136, 204, 238)), // #88CCEE cyan
		FLinearColor(FColor(136,  34, 85)),  // #882255 wine
		FLinearColor(FColor(153, 153, 51)),  // #999933 olive
		FLinearColor(FColor(204, 102, 119)), // #CC6677 rose
	};
	constexpr FLinearColor DefaultColor(FColor(221, 221, 221)); // #DDDDDD pale grey
	
	// Set category display names & colors
	LayoutBuilder.EditCategory("AttenuationDistance", LOCTEXT("AttenuationVolume", "Attenuation (Volume)")).SetColor(CategoryColors[0]);
	LayoutBuilder.EditCategory("AttenuationSpatialization", LOCTEXT("AttenuationSpatialization", "Attenuation (Spatialization)")).SetColor(CategoryColors[1]);
	LayoutBuilder.EditCategory("AttenuationAirAbsorption", LOCTEXT("AttenuationAirAbsorption", "Attenuation (Air Absorption)")).SetColor(CategoryColors[2]);
	LayoutBuilder.EditCategory("AttenuationListenerFocus", LOCTEXT("AttenuationListenerFocus", "Attenuation (Focus)")).SetColor(CategoryColors[3]);
	LayoutBuilder.EditCategory("AttenuationOcclusion", LOCTEXT("AttenuationOcclusion", "Attenuation (Occlusion)")).SetColor(CategoryColors[4]);
	LayoutBuilder.EditCategory("AttenuationSubmixSend", LOCTEXT("AttenuationSubmixSend", "Attenuation (Submix)")).SetColor(CategoryColors[5]);
	LayoutBuilder.EditCategory("AttenuationReverbSend", LOCTEXT("AttenuationReverbSend", "Attenuation (Reverb)")).SetColor(CategoryColors[6]);
	LayoutBuilder.EditCategory("AttenuationPriority", LOCTEXT("AttenuationPriority", "Attenuation (Priority)")).SetColor(CategoryColors[7]);
	LayoutBuilder.EditCategory("AttenuationPluginSettings", LOCTEXT("AttenuationPluginSettings", "Attenuation (Plugin Settings)")).SetColor(DefaultColor);
	LayoutBuilder.EditCategory("AttenuationSourceDataOverride", LOCTEXT("AttenuationSourceDataOverride", "Attenuation (Source Data Override)")).SetColor(DefaultColor);
	LayoutBuilder.EditCategory("AttenuationAudioLink", LOCTEXT("AttenuationAudioLink", "Attenuation (AudioLink)")).SetColor(DefaultColor);
	LayoutBuilder.SortCategories(AttenuationSettingsUtils::SortCategories);

	if (PropertyHandles.Num() != 70)
	{
		ensureMsgf(false, TEXT("Unexpected property handle(s) customizing FSoundAttenuationSettings. %d handles found"), PropertyHandles.Num());
	}
}

void FForceFeedbackAttenuationSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Property handle here is the base struct. We are going to hide it since we're showing it's properties directly.
	PropertyHandle->MarkHiddenByCustomization();
}

void FForceFeedbackAttenuationSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FBaseAttenuationSettingsCustomization::CustomizeChildren(StructPropertyHandle, ChildBuilder, StructCustomizationUtils);

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;

	for(uint32 ChildIndex = 0; ChildIndex <NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("NoNaturalSound", "Natural Sound is only available for Sound Attenuation")));
	const UEnum* const AttenuationDistanceModelEnum = StaticEnum<EAttenuationDistanceModel>();		
	EnumRestriction->AddHiddenValue(AttenuationDistanceModelEnum->GetNameStringByValue((uint8)EAttenuationDistanceModel::NaturalSound));
	DistanceAlgorithmHandle->AddRestriction(EnumRestriction.ToSharedRef());

	if (PropertyHandles.Num() != 10)
	{
		FString PropertyList;
		for (auto It(PropertyHandles.CreateConstIterator()); It; ++It)
		{
			PropertyList += It.Key().ToString() + TEXT(", ");
		}
		ensureMsgf(false, TEXT("Unexpected property handle(s) customizing FForceFeedbackAttenuationSettings: %s"), *PropertyList);
	}

}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsFocusEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsFocusedPropertyWeakPtr = bIsFocusedHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsFocusedPropertyWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsFocusedProperty = bIsFocusedPropertyWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsFocusedProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsOcclusionEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsOcclusionPropertyWeakPtr = bIsOcclusionEnabledHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsOcclusionPropertyWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bOcclusionProperty = bIsOcclusionPropertyWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bOcclusionProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsSpatializationEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsSpatializedHandleWeakPtr = bIsSpatializedHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsSpatializedHandleWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsSpatializedProperty = bIsSpatializedHandleWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsSpatializedProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsAirAbsorptionEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsAirAbsorptionHandleWeakPtr = bIsAirAbsorptionEnabledHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsAirAbsorptionHandleWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsAirAbsorptionProperty = bIsAirAbsorptionHandleWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsAirAbsorptionProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsReverbSendEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsReverbSendWeakPtr = bIsReverbSendEnabledHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsReverbSendWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsReverbSendProperty = bIsReverbSendWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsReverbSendProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsPriorityAttenuationEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsPriorityAttenuationEnabledWeakPtr = bIsPriorityAttenuationEnabledHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsPriorityAttenuationEnabledWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsPriorityAttenuationEnabledProperty = bIsPriorityAttenuationEnabledWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsPriorityAttenuationEnabledProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

TAttribute<bool> FSoundAttenuationSettingsCustomization::GetIsSubmixSendAttenuationEnabledAttribute() const
{
	TWeakPtr<IPropertyHandle> bOverrideAttenuationPropertyWeakPtr = bOverrideAttenuationHandle;
	TWeakPtr<IPropertyHandle> bIsSubmixSendWeakPtr = bIsSubmixSendAttenuationEnabledHandle;

	auto Lambda = [bOverrideAttenuationPropertyWeakPtr, bIsSubmixSendWeakPtr]()
	{
		TSharedPtr<IPropertyHandle> bOverrideAttenuationProperty = bOverrideAttenuationPropertyWeakPtr.Pin();
		TSharedPtr<IPropertyHandle> bIsSubmixSendAttenuationEnabledProperty = bIsSubmixSendWeakPtr.Pin();

		bool Value = AttenuationSettingsUtils::GetValue(bOverrideAttenuationProperty);
		Value &= AttenuationSettingsUtils::GetValue(bIsSubmixSendAttenuationEnabledProperty);
		return Value;
	};

	return TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(Lambda));
}

EVisibility FSoundAttenuationSettingsCustomization::IsLinearMethodSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 SendMethodValue;
	ReverbSendMethodHandle->GetValue(SendMethodValue);

	const EReverbSendMethod SendMethodType = (EReverbSendMethod)SendMethodValue;

	return (SendMethodType == EReverbSendMethod::Linear ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsCustomReverbSendCurveSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 SendMethodValue;
	ReverbSendMethodHandle->GetValue(SendMethodValue);

	const EReverbSendMethod SendMethodType = (EReverbSendMethod)SendMethodValue;

	return (SendMethodType == EReverbSendMethod::CustomCurve ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsCustomAirAbsorptionCurveSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 MethodValue;
	AbsorptionMethodHandle->GetValue(MethodValue);

	const EAirAbsorptionMethod MethodType = (EAirAbsorptionMethod)MethodValue;

	return (MethodType == EAirAbsorptionMethod::CustomCurve ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsLinearOrCustomReverbMethodSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 SendMethodValue;
	ReverbSendMethodHandle->GetValue(SendMethodValue);

	const EReverbSendMethod SendMethodType = (EReverbSendMethod)SendMethodValue;

	return (SendMethodType != EReverbSendMethod::Manual ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsManualReverbSendSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 SendMethodValue;
	ReverbSendMethodHandle->GetValue(SendMethodValue);

	const EReverbSendMethod SendMethodType = (EReverbSendMethod)SendMethodValue;

	return (SendMethodType == EReverbSendMethod::Manual ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsPriorityAttenuationLinearMethodSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 Value;
	PriorityAttenuationMethodHandle->GetValue(Value);

	const EPriorityAttenuationMethod MethodType = (EPriorityAttenuationMethod)Value;

	return (MethodType == EPriorityAttenuationMethod::Linear ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsCustomPriorityAttenuationCurveSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 Value;
	PriorityAttenuationMethodHandle->GetValue(Value);

	const EPriorityAttenuationMethod MethodType = (EPriorityAttenuationMethod)Value;

	return (MethodType == EPriorityAttenuationMethod::CustomCurve ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsLinearOrCustomPriorityAttenuationSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 Value;
	PriorityAttenuationMethodHandle->GetValue(Value);

	const EPriorityAttenuationMethod MethodType = (EPriorityAttenuationMethod)Value;

	return (MethodType != EPriorityAttenuationMethod::Manual ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FSoundAttenuationSettingsCustomization::IsManualPriorityAttenuationSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 Value;
	PriorityAttenuationMethodHandle->GetValue(Value);

	const EPriorityAttenuationMethod MethodType = (EPriorityAttenuationMethod)Value;

	return (MethodType == EPriorityAttenuationMethod::Manual ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsConeSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 AttenuationShapeValue;
	AttenuationShapeHandle->GetValue(AttenuationShapeValue);

	const EAttenuationShape::Type AttenuationShape = (EAttenuationShape::Type)AttenuationShapeValue;

	return (AttenuationShape == EAttenuationShape::Cone ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsSphereSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 AttenuationShapeValue;
	AttenuationShapeHandle->GetValue(AttenuationShapeValue);

	const EAttenuationShape::Type AttenuationShape = (EAttenuationShape::Type)AttenuationShapeValue;

	return (AttenuationShape == EAttenuationShape::Sphere ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsBoxSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 AttenuationShapeValue;
	AttenuationShapeHandle->GetValue(AttenuationShapeValue);

	const EAttenuationShape::Type AttenuationShape = (EAttenuationShape::Type)AttenuationShapeValue;

	return (AttenuationShape == EAttenuationShape::Box ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsCapsuleSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 AttenuationShapeValue;
	AttenuationShapeHandle->GetValue(AttenuationShapeValue);

	const EAttenuationShape::Type AttenuationShape = (EAttenuationShape::Type)AttenuationShapeValue;

	return (AttenuationShape == EAttenuationShape::Capsule ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsNaturalSoundSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 DistanceAlgorithmValue;
	DistanceAlgorithmHandle->GetValue(DistanceAlgorithmValue);

	const EAttenuationDistanceModel DistanceAlgorithm = (EAttenuationDistanceModel)DistanceAlgorithmValue;

	return (DistanceAlgorithm == EAttenuationDistanceModel::NaturalSound ? EVisibility::Visible : EVisibility::Hidden);
}

EVisibility FBaseAttenuationSettingsCustomization::IsCustomCurveSelected() const
{
	if (!AttenuationSettingsUtils::GetValue(bOverrideAttenuationHandle))
	{
		return EVisibility::Hidden;
	}

	uint8 DistanceAlgorithmValue;
	DistanceAlgorithmHandle->GetValue(DistanceAlgorithmValue);

	const EAttenuationDistanceModel DistanceAlgorithm = (EAttenuationDistanceModel)DistanceAlgorithmValue;

	return (DistanceAlgorithm == EAttenuationDistanceModel::Custom ? EVisibility::Visible : EVisibility::Hidden);
}

#undef LOCTEXT_NAMESPACE
