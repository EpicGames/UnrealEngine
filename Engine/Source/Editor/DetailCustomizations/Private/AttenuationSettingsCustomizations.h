// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

class FBaseAttenuationSettingsCustomization : public IPropertyTypeCustomization
{
public:
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	//~ End IPropertyTypeCustomization

	TAttribute<bool> IsAttenuationOverriddenAttribute() const;
	TAttribute<EVisibility> IsAttenuationOverriddenVisibleAttribute() const;

protected:

	EVisibility IsSphereSelected() const;
	EVisibility IsBoxSelected() const;
	EVisibility IsCapsuleSelected() const;
	EVisibility IsConeSelected() const;
	EVisibility IsNaturalSoundSelected() const;
	EVisibility IsCustomCurveSelected() const;

	TAttribute<bool> GetIsAttenuationEnabledAttribute() const;
	TAttribute<bool> GetIsFalloffModeEnabledAttribute() const;
	TSharedPtr<IPropertyHandle> GetOverrideAttenuationHandle(TSharedRef<IPropertyHandle> StructPropertyHandle);

	TSharedPtr<IPropertyHandle> bIsAttenuatedHandle;
	TSharedPtr<IPropertyHandle> AttenuationShapeHandle;
	TSharedPtr<IPropertyHandle> DistanceAlgorithmHandle;
	TSharedPtr<IPropertyHandle> DbAttenuationAtMaxHandle;
	TSharedPtr<IPropertyHandle> bOverrideAttenuationHandle;
};

class FSoundAttenuationSettingsCustomization : public FBaseAttenuationSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	//~ End IPropertyTypeCustomization

protected:

	TSharedPtr<IPropertyHandle> bIsSpatializedHandle;
	TSharedPtr<IPropertyHandle> bIsAirAbsorptionEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsFocusedHandle;
	TSharedPtr<IPropertyHandle> bIsOcclusionEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsReverbSendEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsPriorityAttenuationEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsSubmixSendAttenuationEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsSourceDataOverrideEnabledHandle;
	TSharedPtr<IPropertyHandle> bIsSendToAudioLinkEnabledHandle;
	TSharedPtr<IPropertyHandle> bEnableSourceDataOverrideHandle;
	TSharedPtr<IPropertyHandle> ReverbSendMethodHandle;
	TSharedPtr<IPropertyHandle> PriorityAttenuationMethodHandle;
	TSharedPtr<IPropertyHandle> AbsorptionMethodHandle;

	TAttribute<bool> IsFocusEnabledAttribute;

	TAttribute<bool> GetIsFocusEnabledAttribute() const;
	TAttribute<bool> GetIsOcclusionEnabledAttribute() const;
	TAttribute<bool> GetIsSpatializationEnabledAttribute() const;
	TAttribute<bool> GetIsAirAbsorptionEnabledAttribute() const;
	TAttribute<bool> GetIsReverbSendEnabledAttribute() const;
	TAttribute<bool> GetIsPriorityAttenuationEnabledAttribute() const;
	TAttribute<bool> GetIsSubmixSendAttenuationEnabledAttribute() const;

	EVisibility IsLinearMethodSelected() const;
	EVisibility IsCustomReverbSendCurveSelected() const;
	EVisibility IsLinearOrCustomReverbMethodSelected() const;
	EVisibility IsManualReverbSendSelected() const;
	EVisibility IsCustomAirAbsorptionCurveSelected() const;
	EVisibility IsPriorityAttenuationLinearMethodSelected() const;
	EVisibility IsCustomPriorityAttenuationCurveSelected() const;
	EVisibility IsLinearOrCustomPriorityAttenuationSelected() const;
	EVisibility IsManualPriorityAttenuationSelected() const;
};

class FForceFeedbackAttenuationSettingsCustomization : public FBaseAttenuationSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization
};