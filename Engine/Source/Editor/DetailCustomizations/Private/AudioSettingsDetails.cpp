// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSettingsDetails.h"

#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Sound/AudioSettings.h"
#include "Sound/DialogueWave.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AudioSettingsDetails"

TSharedRef<IDetailCustomization> FAudioSettingsDetails::MakeInstance()
{
	return MakeShareable(new FAudioSettingsDetails);
}

void FAudioSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DialogueFilenameFormatProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAudioSettings, DialogueFilenameFormat));
	DialogueFilenameFormatProperty->MarkHiddenByCustomization();

	IDetailCategoryBuilder& DialogueCategoryBuilder = DetailBuilder.EditCategory("Dialogue", LOCTEXT("DialogueCategoryLabel", "Dialogue"));
	DialogueCategoryBuilder.AddCustomRow(DialogueFilenameFormatProperty->GetPropertyDisplayName())
		.NameContent()
		[
			DialogueFilenameFormatProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				DialogueFilenameFormatProperty->CreatePropertyValueWidget()
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 30.0f, 0.0f))
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(this, &FAudioSettingsDetails::GetExampleDialogueFilename)
			]
		];
}

FText FAudioSettingsDetails::GetExampleDialogueFilename() const
{
	static const FGuid DummyGuid(0xa05875c2, 0xc7ca4601, 0x92e03564, 0x532674a3);
	static const FString DummyAssetName = TEXT("Bob_CannotDo_Dialogue");
	static const FString DummyContextId = TEXT("C174FFF6B897CD21");

	FString FormatString;
	DialogueFilenameFormatProperty->GetValue(FormatString);

	return FText::FromString(UDialogueWave::BuildRecordedAudioFilename(FormatString, DummyGuid, DummyAssetName, DummyContextId, 2));
}

#undef LOCTEXT_NAMESPACE
