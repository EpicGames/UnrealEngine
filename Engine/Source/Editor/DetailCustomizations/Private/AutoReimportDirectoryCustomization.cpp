// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoReimportDirectoryCustomization.h"

#include "Containers/Array.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "DesktopPlatformModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "IDesktopPlatform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FAutoReimportDirectoryCustomization"

void FAutoReimportWildcardCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	WildcardProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAutoReimportWildcard, Wildcard));
	IncludeProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAutoReimportWildcard, bInclude));
	
	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(150.0f)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f,0.f,4.f,0.f))
		[
			SNew(SEditableTextBox)
			.Text(this, &FAutoReimportWildcardCustomization::GetWildcardText)
			.OnTextChanged(this, &FAutoReimportWildcardCustomization::OnWildcardChanged)
			.OnTextCommitted(this, &FAutoReimportWildcardCustomization::OnWildcardCommitted)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FAutoReimportWildcardCustomization::GetCheckState)
			.OnCheckStateChanged(this, &FAutoReimportWildcardCustomization::OnCheckStateChanged)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Include_Label", "Include?"))
				.ToolTipText(LOCTEXT("Include_ToolTip", "When checked, files that match the wildcard will be included, otherwise files that match will be excluded from the monitor."))
			]
		]
	];
}

void FAutoReimportWildcardCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
}

FText FAutoReimportWildcardCustomization::GetWildcardText() const
{
	FText Ret;
	WildcardProperty->GetValueAsFormattedText(Ret);
	return Ret;
}

void FAutoReimportWildcardCustomization::OnWildcardCommitted(const FText& InValue, ETextCommit::Type CommitType)
{
	WildcardProperty->SetValue(InValue.ToString());
}

void FAutoReimportWildcardCustomization::OnWildcardChanged(const FText& InValue)
{
	WildcardProperty->SetValue(InValue.ToString());
}

ECheckBoxState FAutoReimportWildcardCustomization::GetCheckState() const
{
	bool bChecked = true;
	IncludeProperty->GetValue(bChecked);
	return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FAutoReimportWildcardCustomization::OnCheckStateChanged(ECheckBoxState InState)
{
	IncludeProperty->SetValue(InState == ECheckBoxState::Checked);
}

void FAutoReimportDirectoryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	SourceDirProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAutoReimportDirectoryConfig, SourceDirectory));
	MountPointProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAutoReimportDirectoryConfig, MountPoint));
	WildcardsProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAutoReimportDirectoryConfig, Wildcards));

	//We do not show the MountPoint for the default /Game/ entry
	FString SourceDirectory;
	SourceDirProperty->GetValueAsFormattedString(SourceDirectory);
	MountPathVisibility = SourceDirectory.Compare(TEXT("/Game/")) != 0 ? EVisibility::Visible : EVisibility::Collapsed;
	
	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(150.0f)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f,0.f,4.f,0.f))
		[
			SNew(SEditableTextBox)
			.Text(this, &FAutoReimportDirectoryCustomization::GetDirectoryText)
			.OnTextChanged(this, &FAutoReimportDirectoryCustomization::OnDirectoryChanged)
			.OnTextCommitted(this, &FAutoReimportDirectoryCustomization::OnDirectoryCommitted)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ContentPadding(FMargin(4.0f, 2.0f))
			.OnClicked(this, &FAutoReimportDirectoryCustomization::BrowseForFolder)
			.ToolTipText(LOCTEXT("BrowseForDirectory", "Browse for a directory"))
			.Text(LOCTEXT("Browse", "Browse"))
		]
	];
}

void FAutoReimportDirectoryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	{
		FDetailWidgetRow& DetailRow = InStructBuilder.AddCustomRow(LOCTEXT("MountPathName", "Mount Path"));

		DetailRow.Visibility(TAttribute<EVisibility>(this, &FAutoReimportDirectoryCustomization::GetMountPathVisibility));

		DetailRow.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("MountPath_Label", "Map Directory To"))
			.ToolTipText(LOCTEXT("MountPathToolTip", "Specify a mount path to which this physical directory relates. Any new files added on disk will be imported into this virtual path."))
		];

		DetailRow.ValueContent()
		.MaxDesiredWidth(150.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FAutoReimportDirectoryCustomization::GetMountPointText)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(PathPickerButton, SComboButton)
				.HasDownArrow(false)
				.ToolTipText(LOCTEXT("BrowseForMountPoint", "Choose a path"))
				.OnGetMenuContent(this, &FAutoReimportDirectoryCustomization::GetPathPickerContent)
				.ContentPadding(FMargin(4.0f, 2.0f))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Browse", "Browse"))
				]
			]
		];
	}

	InStructBuilder.AddProperty(WildcardsProperty.ToSharedRef());
}

TSharedRef<SWidget> FAutoReimportDirectoryCustomization::GetPathPickerContent()
{
	FPathPickerConfig PathPickerConfig;

	MountPointProperty->GetValueAsFormattedString(PathPickerConfig.DefaultPath);
	PathPickerConfig.DefaultPath.ReplaceInline(TEXT("\\"), TEXT("/"));

	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &FAutoReimportDirectoryCustomization::PathPickerPathSelected);
	PathPickerConfig.bAllowClassesFolder = false;
	PathPickerConfig.bAddDefaultPath = false;
	PathPickerConfig.bAllowContextMenu = false;
	PathPickerConfig.bFocusSearchBoxWhenOpened = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	return SNew(SBox)
		.WidthOverride(300.f)
		.HeightOverride(500.f)
		.Padding(4.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
			]
		];
}

void FAutoReimportDirectoryCustomization::PathPickerPathSelected(const FString& FolderPath)
{
	//When user choose a root mount point we have to add a / at the end of the path
	FString ApplyFolderPath = FolderPath;
	if (FPackageName::GetPackageMountPoint(FolderPath).IsNone())
	{
		TArray<FString> OutRootContentPaths;
		FPackageName::QueryRootContentPaths(OutRootContentPaths);
		for (const FString &RootMount : OutRootContentPaths)
		{
			if (RootMount.Len() - FolderPath.Len() == 1 && RootMount.StartsWith(FolderPath))
			{
				if (RootMount.EndsWith(TEXT("/")))
				{
					ApplyFolderPath = RootMount;
					break;
				}
			}
		}
	}
	PathPickerButton->SetIsOpen(false);
	MountPointProperty->SetValue(ApplyFolderPath);
}

EVisibility FAutoReimportDirectoryCustomization::GetMountPathVisibility() const
{
	return MountPathVisibility;
}

FText FAutoReimportDirectoryCustomization::GetDirectoryText() const
{
	FText Ret;
	SourceDirProperty->GetValueAsFormattedText(Ret);
	return Ret;
}

void FAutoReimportDirectoryCustomization::OnDirectoryCommitted(const FText& InValue, ETextCommit::Type CommitType)
{
	SetSourcePath(InValue.ToString());
}

void FAutoReimportDirectoryCustomization::OnDirectoryChanged(const FText& InValue)
{
	SetSourcePath(InValue.ToString());
}

FText FAutoReimportDirectoryCustomization::GetMountPointText() const
{
	FText Ret;
	MountPointProperty->GetValueAsFormattedText(Ret);
	return Ret;
}

FReply FAutoReimportDirectoryCustomization::BrowseForFolder()
{
	FString InitialDir;
	SourceDirProperty->GetValue(InitialDir);

	if (InitialDir.IsEmpty())
	{
		InitialDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
	}
	else if (!FPackageName::GetPackageMountPoint(InitialDir).IsNone())
	{
		InitialDir = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(InitialDir));
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		FString FolderName;
		const FString Title = LOCTEXT("BrowseForFolderTitle", "Choose a directory to monitor").ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr), Title, InitialDir, FolderName);
		if (bFolderSelected)
		{
			FolderName /= TEXT("");
			SetSourcePath(FolderName);
		}
	}

	return FReply::Handled();
}

void FAutoReimportDirectoryCustomization::SetSourcePath(const FString& InSourceDir)
{
	MountPathVisibility = EVisibility::Visible;

	// Don't log errors and warnings
	FAutoReimportDirectoryConfig::FParseContext Context(false);

	// Check to see if we need to reset mount point to empty string.
	FString ExistingMountPath, ExisitingSourceDir, DerivedMountPoint, ParseSourceDir = InSourceDir;
	SourceDirProperty->GetValue(ExisitingSourceDir);
	MountPointProperty->GetValue(ExistingMountPath);
	FString MountPoint(ExistingMountPath);

	// Does the supplied directory resolve successfully?
	if (FAutoReimportDirectoryConfig::ParseSourceDirectoryAndMountPoint(ParseSourceDir, MountPoint, Context))
	{
		// Parse previous path to determine if the mount point was implicit. If parsing fails keep existing mount point.
		if (FAutoReimportDirectoryConfig::ParseSourceDirectoryAndMountPoint(ExisitingSourceDir, DerivedMountPoint, Context))
		{
			// Set to empty to use implicit empty string mount point
			// Otherwise keep explicit mount point intact so user may change the source for the mount without losing the value.
			if (ExistingMountPath == DerivedMountPoint)
			{
				MountPointProperty->SetValue(FString());
			}
		}
	}
	
	// Set source dir regardless of it parsed and resolved correctly.
	// Could be in intermediate state from user typing for example.
	SourceDirProperty->SetValue(InSourceDir);
}

#undef LOCTEXT_NAMESPACE
