// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;

enum class ECheckBoxState : uint8;

/**
 * Implements a details view customization for the FAutoReimportDirectoryConfig struct.
 */
class FAutoReimportWildcardCustomization : public IPropertyTypeCustomization
{
public:

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

public:

	/** Creates an instance of this class. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FAutoReimportWildcardCustomization());
	}

private:

	FText GetWildcardText() const;
	void OnWildcardCommitted(const FText& InValue, ETextCommit::Type CommitType);
	void OnWildcardChanged(const FText& InValue);

	void OnCheckStateChanged(ECheckBoxState InState);
	ECheckBoxState GetCheckState() const;
	
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> WildcardProperty;
	TSharedPtr<IPropertyHandle> IncludeProperty;
};

/**
 * Implements a details view customization for the FAutoReimportDirectoryConfig struct.
 */
class FAutoReimportDirectoryCustomization : public IPropertyTypeCustomization
{
public:

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

public:

	/** Creates an instance of this class. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FAutoReimportDirectoryCustomization());
	}

private:


	EVisibility GetMountPathVisibility() const;

	FText GetDirectoryText() const;
	void OnDirectoryCommitted(const FText& InValue, ETextCommit::Type CommitType);
	void OnDirectoryChanged(const FText& InValue);

	FText GetMountPointText() const;

	void SetSourcePath(const FString& InSourceDir);
	FReply BrowseForFolder();

	TSharedRef<class SWidget> GetPathPickerContent();
	void PathPickerPathSelected(const FString& FolderPath);

	EVisibility MountPathVisibility;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> SourceDirProperty;
	TSharedPtr<IPropertyHandle> MountPointProperty;
	TSharedPtr<IPropertyHandle> WildcardsProperty;
	TSharedPtr<class SComboButton> PathPickerButton;
};
