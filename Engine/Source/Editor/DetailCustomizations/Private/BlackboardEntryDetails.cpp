// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackboardEntryDetails.h"

#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "BlackboardEntryDetails"

TSharedRef<IPropertyTypeCustomization> FBlackboardEntryDetails::MakeInstance()
{
	return MakeShareable( new FBlackboardEntryDetails );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBlackboardEntryDetails::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	MyNameProperty = StructPropertyHandle->GetChildHandle(TEXT("EntryName"));
	MyDescriptionProperty = StructPropertyHandle->GetChildHandle(TEXT("EntryDescription"));
	MyValueProperty = StructPropertyHandle->GetChildHandle(TEXT("KeyType"));

	// dont show a header row
	HeaderRow.WholeRowContent()
	[
		SNullWidget::NullWidget
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FBlackboardEntryDetails::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	if (StructPropertyHandle->IsValidHandle())
	{
		StructBuilder.AddProperty(MyNameProperty.ToSharedRef());
		StructBuilder.AddProperty(MyDescriptionProperty.ToSharedRef());
		StructBuilder.AddProperty(MyValueProperty.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE
