// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

class FBlackboardEntryDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:
	
	TSharedPtr<IPropertyHandle> MyNameProperty;
	TSharedPtr<IPropertyHandle> MyDescriptionProperty;
	TSharedPtr<IPropertyHandle> MyValueProperty;
};
