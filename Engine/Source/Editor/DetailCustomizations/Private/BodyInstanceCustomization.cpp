// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodyInstanceCustomization.h"

#include "Components/PrimitiveComponent.h"
#include "Components/ShapeComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "DeformableInterface.h"
#include "DestructibleInterface.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/CollisionProfile.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Layout/Margin.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ObjectEditorUtils.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class USceneComponent;

#define LOCTEXT_NAMESPACE "BodyInstanceCustomization"

#define RowWidth_Customization 50.f

////////////////////////////////////////////////////////////////
FBodyInstanceCustomization::FBodyInstanceCustomization()
{
	CollisionProfile = UCollisionProfile::Get();

	RefreshCollisionProfiles();
}

UStaticMeshComponent* FBodyInstanceCustomization::GetDefaultCollisionProvider(const FBodyInstance* BI) const
{
	if (!BI)
	{
		return nullptr;
	}

	UPrimitiveComponent* OwnerComp = BI->OwnerComponent.Get();
	if (!OwnerComp)
	{
		TWeakObjectPtr<UPrimitiveComponent> FoundComp = BodyInstanceToPrimComponent.FindRef(BI);
		OwnerComp = FoundComp.Get();
	}
	
	UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(OwnerComp);
	return SMC && SMC->SupportsDefaultCollision() ? SMC : nullptr;
}

bool FBodyInstanceCustomization::CanUseDefaultCollision() const
{
	bool bResult = BodyInstances.Num() > 0;
	for (const FBodyInstance* BI : BodyInstances)
	{
		bResult &= GetDefaultCollisionProvider(BI) != nullptr;
	}

	return bResult;
}

void FBodyInstanceCustomization::RefreshCollisionProfiles()
{
	int32 NumProfiles = CollisionProfile->GetNumOfProfiles();

	bool bCanUseDefaultCollision = CanUseDefaultCollision();

	// first create profile combo list
	CollisionProfileComboList.Empty(NumProfiles + (bCanUseDefaultCollision ? 2 : 1));	//If we can use default collision we'll add a "Default" option

	// first one is default one
	if(bCanUseDefaultCollision)
	{
		CollisionProfileComboList.Add(MakeShareable(new FString(TEXT("Default"))));
	}
	
	CollisionProfileComboList.Add(MakeShareable(new FString(TEXT("Custom..."))));

	// go through profile and see if it has mine
	for (int32 ProfileId = 0; ProfileId < NumProfiles; ++ProfileId)
	{
		CollisionProfileComboList.Add(MakeShareable(new FString(CollisionProfile->GetProfileByIndex(ProfileId)->Name.ToString())));
	}

	if(CollisionProfileComboBox.IsValid())
	{
		CollisionProfileComboBox->RefreshOptions();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBodyInstanceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	BodyInstanceHandle = StructPropertyHandle;

	// copy all bodyinstances I'm accessing right now
	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData(StructPtrs);
	check(StructPtrs.Num() != 0);

	BodyInstances.AddZeroed(StructPtrs.Num());
	for (auto Iter = StructPtrs.CreateIterator(); Iter; ++Iter)
	{
		check(*Iter);
		BodyInstances[Iter.GetIndex()] = (FBodyInstance*)(*Iter);
	}

	TArray<UObject*> OwningObjects;
	StructPropertyHandle->GetOuterObjects(OwningObjects);

	PrimComponents.Empty(OwningObjects.Num());
	for (UObject* Obj : OwningObjects)
{
		if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Obj))
		{
			PrimComponents.Add(PrimComponent);

			if (FBodyInstance* BI = PrimComponent->GetBodyInstance())
			{
				BodyInstanceToPrimComponent.Add(BI, PrimComponent);
			}
		}
	}

	// get all parent instances
	TSharedPtr<IPropertyHandle> CollisionCategoryHandle = StructPropertyHandle->GetParentHandle();
	TSharedPtr<IPropertyHandle> StaticMeshComponentHandle = CollisionCategoryHandle->GetParentHandle();

	if (CollisionCategoryHandle.IsValid())
	{
		UseDefaultCollisionHandle = CollisionCategoryHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UStaticMeshComponent, bUseDefaultCollision));
	}

	if (StaticMeshComponentHandle.IsValid())
	{
		StaticMeshHandle = StaticMeshComponentHandle->GetChildHandle(UStaticMeshComponent::GetMemberNameChecked_StaticMesh());
		if (StaticMeshHandle.IsValid())
		{
			FSimpleDelegate OnStaticMeshChangedDelegate = FSimpleDelegate::CreateSP(this, &FBodyInstanceCustomization::RefreshCollisionProfiles);
			StaticMeshHandle->SetOnPropertyValueChanged(OnStaticMeshChangedDelegate);
		}
	}

	TSharedPtr<IPropertyHandle> SimulatePhysicsPropertyHandle = BodyInstanceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bSimulatePhysics));
	if (SimulatePhysicsPropertyHandle.IsValid())
	{
		FSimpleDelegate OnSimulatePhysicsChangedDelegate = FSimpleDelegate::CreateSP(this, &FBodyInstanceCustomization::OnSimulatePhysicsChanged);
		SimulatePhysicsPropertyHandle->SetOnPropertyValueChanged(OnSimulatePhysicsChangedDelegate);
	}

	// The selection of BodyInstances modifies the collision profiles available, refresh them now : 
	RefreshCollisionProfiles();

	CollisionProfileNameHandle = StructPropertyHandle->GetChildHandle(TEXT("CollisionProfileName"));
	check(CollisionProfileNameHandle.IsValid());

	CollisionEnabledHandle = StructPropertyHandle->GetChildHandle(TEXT("CollisionEnabled"));
	check(CollisionEnabledHandle.IsValid());

	ObjectTypeHandle = StructPropertyHandle->GetChildHandle(TEXT("ObjectType"));
	check(ObjectTypeHandle.IsValid());

	CollisionResponsesHandle = StructPropertyHandle->GetChildHandle(TEXT("CollisionResponses"));
	check (CollisionResponsesHandle.IsValid());
	
	// Expensive to validate
    // Presets update the DetailView on event, cannot be changed when custom from scripts so is safe
    // to skip validation each from for.
    CollisionResponsesHandle->SetIgnoreValidation(true);

	// need to find profile name
	FName ProfileName;
	TSharedPtr< FString > DisplayName = CollisionProfileComboList[0];
	bool bDisplayAdvancedCollisionSettings = true;

	// if I have valid profile name
	if (!AreAllCollisionUsingDefault() && CollisionProfileNameHandle->GetValue(ProfileName) ==  FPropertyAccess::Result::Success && FBodyInstance::IsValidCollisionProfileName(ProfileName) )
	{
		DisplayName = GetProfileString(ProfileName);
		bDisplayAdvancedCollisionSettings = false;
	}

	const FString PresetsDocLink = TEXT("Shared/Collision");
	TSharedPtr<SToolTip> ProfileTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("SelectCollisionPreset", "Select collision presets. You can set this data in Project settings."), NULL, PresetsDocLink, TEXT("PresetDetail"));

	HeaderRow
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		TAttribute<bool>::Create([this]() { return ShouldShowResetToDefaultProfile(); }),
		FSimpleDelegate::CreateLambda([this]() { SetToDefaultProfile(); })
	))
	.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[ 
			SNew(STextBlock)
			.Text(LOCTEXT("CollisionPresetsLabel", "Collision Presets"))
			.ToolTip(ProfileTooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled(this, &FBodyInstanceCustomization::IsCollisionEnabled)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
					SAssignNew(CollisionProfileComboBox, SComboBox< TSharedPtr<FString> >)
			.OptionsSource(&CollisionProfileComboList)
			.OnGenerateWidget(this, &FBodyInstanceCustomization::MakeCollisionProfileComboWidget)
						.OnSelectionChanged(this, &FBodyInstanceCustomization::OnCollisionProfileChanged)
			.OnComboBoxOpening(this, &FBodyInstanceCustomization::OnCollisionProfileComboOpening)
			.InitiallySelectedItem(DisplayName)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FBodyInstanceCustomization::GetCollisionProfileComboBoxContent)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTipText(this, &FBodyInstanceCustomization::GetCollisionProfileComboBoxToolTip)
			]
		]
		]
		.ShouldAutoExpand(bDisplayAdvancedCollisionSettings);
}

void FBodyInstanceCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	IDetailCategoryBuilder& CatBuilder = StructBuilder.GetParentCategory();
	DetailBuilder = &CatBuilder.GetParentLayout();

	// now create custom set up
	CreateCustomCollisionSetup(StructPropertyHandle, StructBuilder);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 FBodyInstanceCustomization::InitializeObjectTypeComboList()
{
	ObjectTypeComboList.Empty();
	ObjectTypeValues.Empty();

	UEnum * Enum = StaticEnum<ECollisionChannel>();
	const FString KeyName = TEXT("DisplayName");
	const FString QueryType = TEXT("TraceQuery");

	int32 NumEnum = Enum->NumEnums();
	int32 Selected = 0;
	uint8 ObjectTypeIndex = 0;
	if (ObjectTypeHandle->GetValue(ObjectTypeIndex) != FPropertyAccess::Result::Success)
	{
		ObjectTypeIndex = 0; // if multi, just let it be 0
	}

	// go through enum and fill up the list
	for (int32 EnumIndex = 0; EnumIndex < NumEnum; ++EnumIndex)
	{
		// make sure the enum entry is object channel
		const FString& QueryTypeMetaData = Enum->GetMetaData(*QueryType, EnumIndex);
		// if query type is object, we allow it to be on movement channel
		if (QueryTypeMetaData.Len() == 0 || QueryTypeMetaData[0] == '0')
		{
			const FString& KeyNameMetaData = Enum->GetMetaData(*KeyName, EnumIndex);

			if ( KeyNameMetaData.Len() > 0 )
			{
				int32 NewIndex = ObjectTypeComboList.Add( MakeShareable( new FString (KeyNameMetaData) ));
				// @todo: I don't think this would work well if we customize entry, but I don't think we can do that yet
				// i.e. enum a { a1=5, a2=6 }
				ObjectTypeValues.Add((ECollisionChannel)EnumIndex);

				// this solution poses problem when the item was saved with ALREADY INVALID movement channel
				// that will automatically select 0, but I think that is the right solution
				if (ObjectTypeIndex == EnumIndex)
				{
					Selected = NewIndex;
				}
			}
		}
	}

	// it can't be zero. If so you need to fix it
	check ( ObjectTypeComboList.Num() > 0 );

	return Selected;
}

int32 FBodyInstanceCustomization::GetNumberOfSpecialProfiles() const
{
	return CanUseDefaultCollision() ? 2 : 1;
}

int32 FBodyInstanceCustomization::GetCustomIndex() const
{
	return CanUseDefaultCollision() ? 1 : 0;
}

int32 FBodyInstanceCustomization::GetDefaultIndex() const
{
	ensure(CanUseDefaultCollision());
	return 0;
}

TSharedPtr<FString> FBodyInstanceCustomization::GetProfileString(FName ProfileName) const
{
	FString ProfileNameString = ProfileName.ToString();

	// go through profile and see if it has mine
	int32 NumProfiles = CollisionProfile->GetNumOfProfiles();
	// refresh collision count
	if( NumProfiles + GetNumberOfSpecialProfiles() == CollisionProfileComboList.Num() )
	{
		for(int32 ProfileId = 0; ProfileId < NumProfiles; ++ProfileId)
		{
			if(*CollisionProfileComboList[ProfileId+GetNumberOfSpecialProfiles()].Get() == ProfileNameString)
			{
				return CollisionProfileComboList[ProfileId+GetNumberOfSpecialProfiles()];
			}
		}
	}

	return CollisionProfileComboList[GetCustomIndex()];
}

// filter through find valid index of enum values matching each item
// this needs refresh when displayname of the enum has change
// which can happen when we have engine project settings in place working
void FBodyInstanceCustomization::UpdateValidCollisionChannels()
{
	// find the enum
	UEnum * Enum = StaticEnum<ECollisionChannel>();
	// we need this Enum
	check (Enum);
	const FString KeyName = TEXT("DisplayName");
	const FString TraceType = TEXT("TraceQuery");

	// need to initialize displaynames separate
	int32 NumEnum = Enum->NumEnums();
	ValidCollisionChannels.Empty(NumEnum);

	// first go through enum entry, and add suffix to displaynames
	for ( int32 EnumIndex=0; EnumIndex<NumEnum; ++EnumIndex )
	{
		const FString& MetaData = Enum->GetMetaData(*KeyName, EnumIndex);
		if ( MetaData.Len() > 0 )
		{
			FCollisionChannelInfo Info;
			Info.DisplayName = MetaData;
			Info.CollisionChannel = (ECollisionChannel)EnumIndex;
			if (Enum->GetMetaData(*TraceType, EnumIndex) == TEXT("1"))
			{
				Info.TraceType = true;
			}
			else
			{
				Info.TraceType = false;
			}

			ValidCollisionChannels.Add(Info);
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBodyInstanceCustomization::CreateCustomCollisionSetup(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder)
{
	UpdateValidCollisionChannels();

	if (ValidCollisionChannels.Num() == 0)
	{
		return;
	}

	int32 TotalNumChildren = ValidCollisionChannels.Num();
	TAttribute<bool> CollisionEnabled(this, &FBodyInstanceCustomization::IsCollisionEnabled );
	TAttribute<bool> CustomCollisionEnabled( this, &FBodyInstanceCustomization::ShouldEnableCustomCollisionSetup );
	TAttribute<EVisibility> CustomCollisionVisibility(this, &FBodyInstanceCustomization::ShouldShowCustomCollisionSetup);

	// initialize ObjectTypeComboList
	// we only display things that has "DisplayName"
	int32 IndexSelected = InitializeObjectTypeComboList();
	StructBuilder.AddProperty(CollisionEnabledHandle.ToSharedRef())
	.IsEnabled(CustomCollisionEnabled)
	.Visibility(CustomCollisionVisibility);

	if (!StructPropertyHandle->GetProperty()->GetBoolMetaData(TEXT("HideObjectType")))
	{
		StructBuilder.AddCustomRow(LOCTEXT("ObjectType", "Object Type"))
		.Visibility(CustomCollisionVisibility)
		.IsEnabled(CustomCollisionEnabled)
		.NameContent()
		[
			ObjectTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SAssignNew(ObjectTypeComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&ObjectTypeComboList)
			.OnGenerateWidget(this, &FBodyInstanceCustomization::MakeObjectTypeComboWidget)
			.OnSelectionChanged(this, &FBodyInstanceCustomization::OnObjectTypeChanged)
			.InitiallySelectedItem(ObjectTypeComboList[IndexSelected])
			.ContentPadding(2.f)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FBodyInstanceCustomization::GetObjectTypeComboBoxContent)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}

	// Add Title
	StructBuilder.AddCustomRow(LOCTEXT("CustomCollision", "Custom Collision"))
	.IsEnabled(CustomCollisionEnabled)
	.Visibility(CustomCollisionVisibility)
	.ValueContent()
	.MaxDesiredWidth(0.f)
	.MinDesiredWidth(0.f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(RowWidth_Customization)
			.HAlign( HAlign_Left )
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IgnoreCollisionLabel", "Ignore"))
				.Font( IDetailLayoutBuilder::GetDetailFontBold() )
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.HAlign( HAlign_Left )
			.WidthOverride(RowWidth_Customization)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OverlapCollisionLabel", "Overlap"))
				.Font( IDetailLayoutBuilder::GetDetailFontBold() )
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlockCollisionLabel", "Block"))
			.Font( IDetailLayoutBuilder::GetDetailFontBold() )
		]
	];

	// Add All check box
	StructBuilder.AddCustomRow(LOCTEXT("All", "All"))
	.IsEnabled(CustomCollisionEnabled)
	.Visibility(CustomCollisionVisibility)
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( 2.0f )
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew( STextBlock )
			.Text(LOCTEXT("CollisionResponsesLabel", "Collision Responses"))
			.Font( IDetailLayoutBuilder::GetDetailFontBold() )
			.ToolTipText(LOCTEXT("CollsionResponse_ToolTip", "When trace by channel, this information will be used for filtering."))
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			IDocumentation::Get()->CreateAnchor( TEXT("Engine/Physics/Collision") )
		]
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(RowWidth_Customization)
			.Content()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnAllCollisionChannelChanged, ECR_Ignore )
				.IsChecked( this, &FBodyInstanceCustomization::IsAllCollisionChannelChecked, ECR_Ignore )
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(RowWidth_Customization)
			.Content()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnAllCollisionChannelChanged, ECR_Overlap )
				.IsChecked( this, &FBodyInstanceCustomization::IsAllCollisionChannelChecked, ECR_Overlap )
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(RowWidth_Customization)
			.Content()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnAllCollisionChannelChanged, ECR_Block )
				.IsChecked( this, &FBodyInstanceCustomization::IsAllCollisionChannelChecked, ECR_Block )
			]
		]
	];

	// add header
	// Add Title
	StructBuilder.AddCustomRow(LOCTEXT("TraceResponses", "Trace Responses"))
	.IsEnabled(CustomCollisionEnabled)
	.Visibility(CustomCollisionVisibility)
	.NameContent()
	[
		SNew(SBox)
		.Padding(FMargin(10, 0))
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CollisionTraceResponsesLabel", "Trace Responses"))
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
	];

	// each channel set up
	FName MetaData(TEXT("DisplayName"));
	// Add option for each channel - first do trace
	for ( int32 Index=0; Index<TotalNumChildren; ++Index )
	{
		if (ValidCollisionChannels[Index].TraceType)
		{
			FString DisplayName = ValidCollisionChannels[Index].DisplayName;
			EVisibility Visibility = EVisibility::Visible;

			StructBuilder.AddCustomRow(LOCTEXT("CollisionChannel", "Collision Channel"))
			.IsEnabled(CustomCollisionEnabled)
			.Visibility(CustomCollisionVisibility)
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::Create([this, Index]() { return ShouldShowResetToDefaultResponse(Index); }),
				FSimpleDelegate::CreateLambda([this, Index]() { SetToDefaultResponse(Index); })
			))
			.NameContent()
			[
				SNew(SBox)
				.Padding(FMargin(15, 0))
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromString(DisplayName))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
			]
			.ValueContent()
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(RowWidth_Customization)
					.Content()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Ignore )
						.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Ignore )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(RowWidth_Customization)
					.Content()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Overlap )
						.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Overlap )
					]

				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Block )
					.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Block )
				]
			];
		}
	}

	// Add Title
	StructBuilder.AddCustomRow(LOCTEXT("ObjectResponses", "Object Responses"))
	.IsEnabled(CustomCollisionEnabled)
	.Visibility(CustomCollisionVisibility)
	.NameContent()
	[
		SNew(SBox)
		.Padding(FMargin(10, 0))
		.Content()
		[
			SNew( STextBlock )
			.Text(LOCTEXT("CollisionObjectResponses", "Object Responses"))
			.Font( IDetailLayoutBuilder::GetDetailFontBold() )
		]
	];

	for ( int32 Index=0; Index<TotalNumChildren; ++Index )
	{
		if (!ValidCollisionChannels[Index].TraceType)
		{
			FString DisplayName = ValidCollisionChannels[Index].DisplayName;
			EVisibility Visibility = EVisibility::Visible;

			StructBuilder.AddCustomRow(LOCTEXT("CollisionChannel", "Collision Channel"))
			.IsEnabled(CustomCollisionEnabled)
			.Visibility(CustomCollisionVisibility)
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::Create([this, Index]() { return ShouldShowResetToDefaultResponse(Index); }),
				FSimpleDelegate::CreateLambda([this, Index]() { SetToDefaultResponse(Index); })
			))
			.NameContent()
			[
				SNew(SBox)
				.Padding(FMargin(15, 0))
				.Content()
				[
					SNew(STextBlock)
					.Text(FText::FromString(DisplayName))
					.Font( IDetailLayoutBuilder::GetDetailFont() )
				]
			]
			.ValueContent()
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(RowWidth_Customization)
					.Content()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Ignore )
						.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Ignore )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(RowWidth_Customization)
					.Content()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Overlap )
						.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Overlap )
					]

				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &FBodyInstanceCustomization::OnCollisionChannelChanged, Index, ECR_Block )
					.IsChecked( this, &FBodyInstanceCustomization::IsCollisionChannelChecked, Index, ECR_Block )
				]
			];
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> FBodyInstanceCustomization::MakeObjectTypeComboWidget( TSharedPtr<FString> InItem )
{
	return SNew(STextBlock) .Text( FText::FromString(*InItem) ) .Font( IDetailLayoutBuilder::GetDetailFont() );
}

void FBodyInstanceCustomization::OnObjectTypeChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo  )
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();
		ECollisionChannel NewEnumVal = ECC_WorldStatic;
		// find index of NewValue
		for (auto Iter = ObjectTypeComboList.CreateConstIterator(); Iter; ++Iter)
		{
			// if value is same
			if (*(Iter->Get()) == NewValue)
			{
				NewEnumVal = ObjectTypeValues[Iter.GetIndex()];
			}
		}
		ensure(ObjectTypeHandle->SetValue((uint8)NewEnumVal) == FPropertyAccess::Result::Success);
	}
}

FText FBodyInstanceCustomization::GetObjectTypeComboBoxContent() const
{
	FName ObjectTypeName;
	if (ObjectTypeHandle->GetValue(ObjectTypeName) == FPropertyAccess::Result::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*ObjectTypeComboBox.Get()->GetSelectedItem().Get());
}

TSharedRef<SWidget> FBodyInstanceCustomization::MakeCollisionProfileComboWidget(TSharedPtr<FString> InItem)
{
	FString ProfileMessage;

	FCollisionResponseTemplate ProfileData;
	if (CollisionProfile->GetProfileTemplate(FName(**InItem), ProfileData))
	{
		ProfileMessage = ProfileData.HelpMessage;
	}

	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.ToolTipText(FText::FromString(ProfileMessage))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void FBodyInstanceCustomization::OnSimulatePhysicsChanged()
{
	UpdateCollisionProfile();
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////
// NOTE! I have a lot of ensure to make sure it's set correctly
// in case for if type changes or any set up changes, this won't work, but ensure will remind you that! :)
/////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FBodyInstanceCustomization::OnCollisionProfileComboOpening()
{
	if(AreAllCollisionUsingDefault())
	{
		TSharedPtr<FString> ComboStringPtr = CollisionProfileComboList[GetDefaultIndex()];
		if (ComboStringPtr.IsValid())
		{
			CollisionProfileComboBox->SetSelectedItem(ComboStringPtr);
			return;
		}
	}

	FName ProfileName;
	if (CollisionProfileNameHandle->GetValue(ProfileName) != FPropertyAccess::Result::MultipleValues)
	{
		TSharedPtr<FString> ComboStringPtr = GetProfileString(ProfileName);
		if( ComboStringPtr.IsValid() )
		{
			CollisionProfileComboBox->SetSelectedItem(ComboStringPtr);
		}
	}
}

void FBodyInstanceCustomization::MarkAllBodiesDefaultCollision(bool bUseDefaultCollision)
{
	if(PrimComponents.Num() && UseDefaultCollisionHandle.IsValid())	//If we have prim components we might be coming from bp editor which needs to propagate all instances
	{
		for(UPrimitiveComponent* PrimComp : PrimComponents)
		{
			if(UStaticMeshComponent* SMC = Cast<UStaticMeshComponent>(PrimComp))
			{
				const bool bOldDefault = SMC->bUseDefaultCollision;
				const bool bNewDefault = bUseDefaultCollision;

				TSet<USceneComponent*> UpdatedInstances;
				FComponentEditorUtils::PropagateDefaultValueChange(SMC, UseDefaultCollisionHandle->GetProperty(), bOldDefault, bNewDefault, UpdatedInstances);

				SMC->bUseDefaultCollision = bNewDefault;
			}
		}
	}
	else
	{
		for (const FBodyInstance* BI : BodyInstances)
		{
			if (UStaticMeshComponent* SMC = GetDefaultCollisionProvider(BI))
			{
				SMC->bUseDefaultCollision = bUseDefaultCollision;
			}
		}
	}
}

void FBodyInstanceCustomization::OnCollisionProfileChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();
		int32 NumProfiles = CollisionProfile->GetNumOfProfiles();
		for (int32 ProfileId = 0; ProfileId < NumProfiles; ++ProfileId)
		{
			const FCollisionResponseTemplate* CurProfile = CollisionProfile->GetProfileByIndex(ProfileId);
			if ( NewValue == CurProfile->Name.ToString() )
			{
				// trigget transaction before UpdateCollisionProfile
				const FScopedTransaction Transaction( LOCTEXT( "ChangeCollisionProfile", "Change Collision Profile" ) );
				// set profile set up
				MarkAllBodiesDefaultCollision(false);
				ensure ( CollisionProfileNameHandle->SetValue(NewValue) ==  FPropertyAccess::Result::Success );
				UpdateCollisionProfile();
				return;
			}
		}

		if(CanUseDefaultCollision())
		{
			if(NewSelection == CollisionProfileComboList[GetDefaultIndex()])
			{
				CollisionResponsesHandle->NotifyPreChange();
				MarkAllBodiesDefaultCollision(true);
				CollisionResponsesHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				return;
			}
		}

		if( NewSelection == CollisionProfileComboList[GetCustomIndex()])
		{
			// Force expansion when the user chooses the selected item
			if (IDetailPropertyRow* BodyInstanceProperty = DetailBuilder->EditPropertyFromRoot(BodyInstanceHandle))
			{
				BodyInstanceProperty->ShouldAutoExpand(true);
			}
		}

		// if none of them found, clear it
		FName Name=UCollisionProfile::CustomCollisionProfileName;
		MarkAllBodiesDefaultCollision(false);
		ensure(CollisionProfileNameHandle->SetValue(Name) == FPropertyAccess::Result::Success);
	}
}

void FBodyInstanceCustomization::UpdateCollisionProfile()
{
	FName ProfileName;

	// if I have valid profile name
	if (!AreAllCollisionUsingDefault() && CollisionProfileNameHandle->GetValue(ProfileName) ==  FPropertyAccess::Result::Success && FBodyInstance::IsValidCollisionProfileName(ProfileName) )
	{
		int32 NumProfiles = CollisionProfile->GetNumOfProfiles();
		const int32 NumSpecialProfiles = GetNumberOfSpecialProfiles();
		for (int32 ProfileId = 0; ProfileId < NumProfiles; ++ProfileId)
		{
			// find the profile
			const FCollisionResponseTemplate* CurProfile = CollisionProfile->GetProfileByIndex(ProfileId);
			if (ProfileName == CurProfile->Name)
			{
				// set the profile set up
				ensure(CollisionEnabledHandle->SetValue((uint8)CurProfile->CollisionEnabled) == FPropertyAccess::Result::Success);
				ensure(ObjectTypeHandle->SetValue((uint8)CurProfile->ObjectType) == FPropertyAccess::Result::Success);

				SetCollisionResponseContainer(CurProfile->ResponseToChannels);

				// now update combo box
				CollisionProfileComboBox.Get()->SetSelectedItem(CollisionProfileComboList[ProfileId+NumSpecialProfiles]);
				if (ObjectTypeComboBox.IsValid())
				{
					for (auto Iter = ObjectTypeValues.CreateConstIterator(); Iter; ++Iter)
					{
						if (*Iter == CurProfile->ObjectType)
						{
							ObjectTypeComboBox.Get()->SetSelectedItem(ObjectTypeComboList[Iter.GetIndex()]);
							break;
						}
					}
				}

				return;
			}
		}
	}

	CollisionProfileComboBox.Get()->SetSelectedItem(CollisionProfileComboList[AreAllCollisionUsingDefault() ? GetDefaultIndex() : GetCustomIndex()]);
}

void FBodyInstanceCustomization::SetToDefaultProfile()
{
	// trigger transaction before UpdateCollisionProfile
	const FScopedTransaction Transaction( LOCTEXT( "ResetCollisionProfile", "Reset Collision Profile" ) );
	MarkAllBodiesDefaultCollision(false);
	CollisionProfileNameHandle.Get()->ResetToDefault();
	UpdateCollisionProfile();
}

bool FBodyInstanceCustomization::ShouldShowResetToDefaultProfile() const
{
	return CollisionProfileNameHandle.Get()->DiffersFromDefault();
}

void FBodyInstanceCustomization::SetToDefaultResponse(int32 Index)
{
	if (ValidCollisionChannels.IsValidIndex(Index))
	{
		const FScopedTransaction Transaction( LOCTEXT( "ResetCollisionResponse", "Reset Collision Response" ) );
		const ECollisionResponse DefaultResponse = FCollisionResponseContainer::GetDefaultResponseContainer().GetResponse(ValidCollisionChannels[Index].CollisionChannel);

		SetResponse(Index, DefaultResponse);
	}
}

bool FBodyInstanceCustomization::ShouldShowResetToDefaultResponse(int32 Index) const
{
	
	if (ValidCollisionChannels.IsValidIndex(Index))
	{
		const ECollisionResponse DefaultResponse = FCollisionResponseContainer::GetDefaultResponseContainer().GetResponse(ValidCollisionChannels[Index].CollisionChannel);

		if (IsCollisionChannelChecked(Index, DefaultResponse) != ECheckBoxState::Checked)
		{
			return true;
		}
	}

	return false;
}

bool FBodyInstanceCustomization::AreAllCollisionUsingDefault() const
{
	bool bAllUsingDefault = BodyInstances.Num() > 0;
	for(const FBodyInstance* BI : BodyInstances)
	{
		if(UStaticMeshComponent* SMC = GetDefaultCollisionProvider(BI))
		{
			bAllUsingDefault &= SMC->bUseDefaultCollision;
		}
		else
		{
			bAllUsingDefault = false;
			break;
		}
	}

	return bAllUsingDefault;
}

bool FBodyInstanceCustomization::IsCollisionEnabled() const
{
	bool bEnabled = false;
	if(BodyInstanceHandle.IsValid())
	{
		bEnabled = BodyInstanceHandle->IsEditable() && FSlateApplication::Get().GetNormalExecutionAttribute().Get();
	}

	return bEnabled;
}

bool FBodyInstanceCustomization::ShouldEnableCustomCollisionSetup() const
{
	FName ProfileName;
	if (!AreAllCollisionUsingDefault() && CollisionProfileNameHandle->GetValue(ProfileName) == FPropertyAccess::Result::Success && FBodyInstance::IsValidCollisionProfileName(ProfileName) == false)
	{
		return IsCollisionEnabled();
	}

	return false;
}

EVisibility FBodyInstanceCustomization::ShouldShowCustomCollisionSetup() const
{
	return AreAllCollisionUsingDefault() ? EVisibility::Hidden : EVisibility::Visible;
}

FText FBodyInstanceCustomization::GetCollisionProfileComboBoxContent() const
{
	bool bAllUseDefaultCollision = BodyInstances.Num() > 0;
	bool bSomeUseDefaultCollision = false;

	for(const FBodyInstance* BI : BodyInstances)
	{
		if(UStaticMeshComponent* SMC = GetDefaultCollisionProvider(BI))
		{
			bAllUseDefaultCollision &= SMC->bUseDefaultCollision;
			bSomeUseDefaultCollision |= SMC->bUseDefaultCollision;
		}
		else
		{
			bAllUseDefaultCollision = false;
		}
	}

	if(bAllUseDefaultCollision)
	{
		return FText::FromString(*CollisionProfileComboList[GetDefaultIndex()]);
	}

	FName ProfileName;
	if (bSomeUseDefaultCollision || CollisionProfileNameHandle->GetValue(ProfileName) == FPropertyAccess::Result::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*GetProfileString(ProfileName).Get());
}

FText FBodyInstanceCustomization::GetCollisionProfileComboBoxToolTip() const
{
	if(AreAllCollisionUsingDefault())
	{
		return LOCTEXT("DefaultCollision", "Default collision preset specified in the StaticMesh asset");
	}

	FName ProfileName;
	if (CollisionProfileNameHandle->GetValue(ProfileName) == FPropertyAccess::Result::Success)
	{
		FCollisionResponseTemplate ProfileData;
		if ( CollisionProfile->GetProfileTemplate(ProfileName, ProfileData) )
		{
			return FText::FromString(ProfileData.HelpMessage);
		}
		return FText::GetEmpty();
	}

	return LOCTEXT("MultipleValues", "Multiple Values");
}

void FBodyInstanceCustomization::OnCollisionChannelChanged(ECheckBoxState InNewValue, int32 ValidIndex, ECollisionResponse InCollisionResponse)
{
	if ( ValidCollisionChannels.IsValidIndex(ValidIndex) )
	{
		SetResponse(ValidIndex, InCollisionResponse);
	}
}

struct FUpdateCollisionResponseHelper
{
	FUpdateCollisionResponseHelper(UPrimitiveComponent* InPrimComp, TSharedPtr<IPropertyHandle> InCollisionResponseHandle)
	: PrimComp(InPrimComp)
	, CollisionResponsesHandle(InCollisionResponseHandle)
	{
		OldCollision = PrimComp->BodyInstance.GetCollisionResponse();
	}

	~FUpdateCollisionResponseHelper()
	{
		if(CollisionResponsesHandle.IsValid())
		{
			const SIZE_T PropertyOffset = (SIZE_T)&((UPrimitiveComponent*)0)->BodyInstance.CollisionResponses;
			check(PropertyOffset <= MAX_int32);

			TSet<USceneComponent*> UpdatedInstances;
			FComponentEditorUtils::PropagateDefaultValueChange(PrimComp, CollisionResponsesHandle->GetProperty(), OldCollision, PrimComp->BodyInstance.GetCollisionResponse(), UpdatedInstances, PropertyOffset);
		}
	}

	UPrimitiveComponent* PrimComp;
	TSharedPtr<IPropertyHandle> CollisionResponsesHandle;
	FCollisionResponse OldCollision;
};

void FBodyInstanceCustomization::SetResponse(int32 ValidIndex, ECollisionResponse InCollisionResponse)
{
	const FScopedTransaction Transaction( LOCTEXT( "ChangeIndividualChannel", "Change Individual Channel" ) );

	CollisionResponsesHandle->NotifyPreChange();

	if(PrimComponents.Num())	//If we have owning prim components we may be in blueprint editor which means we have to propagate to instances.
	{
		for (UPrimitiveComponent* PrimComp : PrimComponents)
		{
			FUpdateCollisionResponseHelper UpdateCollisionResponseHelper(PrimComp, CollisionResponsesHandle);
			FBodyInstance& BodyInstance = PrimComp->BodyInstance;
			BodyInstance.CollisionResponses.SetResponse(ValidCollisionChannels[ValidIndex].CollisionChannel, InCollisionResponse);
		}
	}
	else
	{
		for (FBodyInstance* BodyInstance : BodyInstances)
		{
			BodyInstance->CollisionResponses.SetResponse(ValidCollisionChannels[ValidIndex].CollisionChannel, InCollisionResponse);
		}
	}

	CollisionResponsesHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

ECheckBoxState FBodyInstanceCustomization::IsCollisionChannelChecked( int32 ValidIndex, ECollisionResponse InCollisionResponse) const
{
	TArray<ECollisionResponse> CollisionResponses;

	if ( ValidCollisionChannels.IsValidIndex(ValidIndex) )
	{
		for (auto Iter=BodyInstances.CreateConstIterator(); Iter; ++Iter)
		{
			FBodyInstance* BodyInstance = *Iter;

			CollisionResponses.AddUnique(BodyInstance->CollisionResponses.GetResponse(ValidCollisionChannels[ValidIndex].CollisionChannel));
		}

		if (CollisionResponses.Num() == 1)
		{
			if (CollisionResponses[0] == InCollisionResponse)
			{
				return ECheckBoxState::Checked;
			}
			else
			{
				return ECheckBoxState::Unchecked;
			}
		}
		else if (CollisionResponses.Contains(InCollisionResponse))
		{
			return ECheckBoxState::Undetermined;
		}

		// if it didn't contain and it's not found, return Unchecked
		return ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void FBodyInstanceCustomization::OnAllCollisionChannelChanged(ECheckBoxState InNewValue, ECollisionResponse InCollisionResponse)
{
	FCollisionResponseContainer NewContainer;
	NewContainer.SetAllChannels(InCollisionResponse);
	SetCollisionResponseContainer(NewContainer);
}

ECheckBoxState FBodyInstanceCustomization::IsAllCollisionChannelChecked(ECollisionResponse InCollisionResponse) const
{
	ECheckBoxState State = ECheckBoxState::Undetermined;

	uint32 TotalNumChildren = ValidCollisionChannels.Num();
	if (TotalNumChildren >= 1)
	{
		State = IsCollisionChannelChecked(0, InCollisionResponse);

		for (uint32 Index = 1; Index < TotalNumChildren; ++Index)
		{
			if (State != IsCollisionChannelChecked(Index, InCollisionResponse))
			{
				State = ECheckBoxState::Undetermined;
				break;
			}
		}
	}

	return State;
}

void FBodyInstanceCustomization::SetCollisionResponseContainer(const FCollisionResponseContainer& ResponseContainer)
{
	// trigget transaction before UpdateCollisionProfile
	uint32 TotalNumChildren = ValidCollisionChannels.Num();

	if(TotalNumChildren)
	{
		const FScopedTransaction Transaction(LOCTEXT("Collision", "Collision Channel Changes"));

		CollisionResponsesHandle->NotifyPreChange();


		// iterate through bodyinstance and fix it
		if(PrimComponents.Num())	//If we have owning prim components we may be in blueprint editor which means we have to propagate to instances.
		{

			for (UPrimitiveComponent* PrimComponent : PrimComponents)
			{
				FUpdateCollisionResponseHelper UpdateCollisionResponseHelper(PrimComponent, CollisionResponsesHandle);
			
				FBodyInstance& BodyInstance = PrimComponent->BodyInstance;
				// only go through valid channels
				for (uint32 Index = 0; Index < TotalNumChildren; ++Index)
				{
					ECollisionChannel Channel = ValidCollisionChannels[Index].CollisionChannel;
					ECollisionResponse Response = ResponseContainer.GetResponse(Channel);

					BodyInstance.CollisionResponses.SetResponse(Channel, Response);
				}
			}
		}
		else
		{
			for (FBodyInstance* BodyInstance : BodyInstances)
			{
				// only go through valid channels
				for (uint32 Index = 0; Index < TotalNumChildren; ++Index)
				{
					ECollisionChannel Channel = ValidCollisionChannels[Index].CollisionChannel;
					ECollisionResponse Response = ResponseContainer.GetResponse(Channel);

					BodyInstance->CollisionResponses.SetResponse(Channel, Response);
				}
			}
		}

		CollisionResponsesHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
	
}

FBodyInstanceCustomizationHelper::FBodyInstanceCustomizationHelper(const TArray<TWeakObjectPtr<UObject>>& InObjectsCustomized)
: ObjectsCustomized(InObjectsCustomized)
{
}

void FBodyInstanceCustomizationHelper::UpdateFilters()
{
	bDisplayMass = true;
	bDisplayConstraints = true;
	bDisplayEnablePhysics = true;
	bDisplayAsyncScene = true;
	bDisplayLinearDamping = true;
	bDisplayAngularDamping = true;
	bDisplayEnableGravity = true;
	bDisplayInertiaConditioning = true;
	bDisplayInitialOverlapDepenetration = true;
	bDisplayWalkableSlopeOverride = true;
	bDisplayAutoWeld = true;
	bDisplayStartAwake = true;
	bDisplayCOMNudge = true;
	bDisplayMassScale = true;
	bDisplayMaxAngularVelocity = true;

	for (int32 i = 0; i < ObjectsCustomized.Num(); ++i)
	{
		if (ObjectsCustomized[i].IsValid())
		{
			if(Cast<IDestructibleInterface>(ObjectsCustomized[i].Get()))
			{
				bDisplayMass = false;
				bDisplayConstraints = false;
			}
			else if (Cast<IDeformableInterface>(ObjectsCustomized[i].Get()))
			{
				bDisplayMass = false;
				bDisplayConstraints = false;
				bDisplayEnablePhysics = false;
				bDisplayAsyncScene = false;
				bDisplayLinearDamping = false;
				bDisplayAngularDamping = false;
				bDisplayEnableGravity = false;
				bDisplayInertiaConditioning = false;
				bDisplayInitialOverlapDepenetration = false;
				bDisplayWalkableSlopeOverride = false;
				bDisplayAutoWeld = false;
				bDisplayStartAwake = false;
				bDisplayCOMNudge = false;
				bDisplayMassScale = false;
				bDisplayMaxAngularVelocity = false;
			}
			else
			{
				if(ObjectsCustomized[i]->IsA(UBodySetup::StaticClass()))
				{
					bDisplayEnablePhysics = false;
					bDisplayConstraints = false;
				}
			}

			if(ObjectsCustomized[i]->IsA(USkeletalMeshComponent::StaticClass()))
			{
				bDisplayMass = false;
				bDisplayAsyncScene = false;
			}
		}
	}
}


void FBodyInstanceCustomizationHelper::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler)
{
	CustomizeDetails(DetailBuilder, BodyInstanceHandler, TFunction<void(TSharedRef<IPropertyHandle>)>());
}

void FBodyInstanceCustomizationHelper::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler, TFunction<void(TSharedRef<IPropertyHandle>)> CustomizeCoMNudge)
{
	if( BodyInstanceHandler->IsValidHandle() )
	{
		UpdateFilters();

		IDetailCategoryBuilder& PhysicsCategory = DetailBuilder.EditCategory("Physics");

		TSharedRef<IPropertyHandle> PhysicsEnable = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bSimulatePhysics)).ToSharedRef();
		if(bDisplayEnablePhysics)
		{
			PhysicsCategory.AddProperty(PhysicsEnable).EditCondition(TAttribute<bool>(this, &FBodyInstanceCustomizationHelper::IsSimulatePhysicsEditable), NULL);
		}
		else
		{
			PhysicsEnable->MarkHiddenByCustomization();
		}

		AddMassInKg(PhysicsCategory, BodyInstanceHandler);


		auto EnablePhysicsProperty = [BodyInstanceHandler, &PhysicsCategory](FName PropertyName, bool bEnable)
		{
			TSharedRef<IPropertyHandle> Property = BodyInstanceHandler->GetChildHandle(PropertyName).ToSharedRef();
			if (bEnable)
			{
				PhysicsCategory.AddProperty(Property);
			}
			else
			{
				Property->MarkHiddenByCustomization();
			}
			return Property;
		};
		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, LinearDamping), bDisplayLinearDamping);
		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, AngularDamping), bDisplayAngularDamping);
		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, bEnableGravity), bDisplayEnableGravity);
		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, bInertiaConditioning), bDisplayInertiaConditioning);
		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, WalkableSlopeOverride), bDisplayWalkableSlopeOverride);

		AddBodyConstraint(PhysicsCategory, BodyInstanceHandler);

		//ADVANCED
		PhysicsCategory.AddProperty(BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bAutoWeld)))
			.Visibility(TAttribute<EVisibility>(this, &FBodyInstanceCustomizationHelper::IsAutoWeldVisible));

		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, bStartAwake), bDisplayStartAwake);

		if (CustomizeCoMNudge && bDisplayCOMNudge)
		{
			CustomizeCoMNudge(BodyInstanceHandler);
		}
		else
		{
			EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, COMNudge), bDisplayCOMNudge);
		}

		EnablePhysicsProperty(GET_MEMBER_NAME_CHECKED(FBodyInstance, MassScale), bDisplayMassScale);

		if (bDisplayMaxAngularVelocity)
		{
			AddMaxAngularVelocity(PhysicsCategory, BodyInstanceHandler);
		}

		//Add the rest
		uint32 NumChildren = 0;
		BodyInstanceHandler->GetNumChildren(NumChildren);
		for(uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
		{
			TSharedPtr<IPropertyHandle> ChildProp = BodyInstanceHandler->GetChildHandle(ChildIdx);

			FName CategoryName = FObjectEditorUtils::GetCategoryFName(ChildProp->GetProperty());
			if(ChildProp->IsCustomized() == false && CategoryName == FName(TEXT("Physics")))	//add the rest of the physics properties
			{
				PhysicsCategory.AddProperty(ChildProp);
			}
		}
	}
}

bool FBodyInstanceCustomizationHelper::IsSimulatePhysicsEditable() const
{
	// Check whether to enable editing of bSimulatePhysics - this will happen if all objects are UPrimitiveComponents & have collision geometry.
	bool bEnableSimulatePhysics = ObjectsCustomized.Num() > 0;
	for (TWeakObjectPtr<UObject> CustomizedObject : ObjectsCustomized)
	{
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(CustomizedObject.Get()))
		{
			if (!PrimitiveComponent->CanEditSimulatePhysics())
			{
				bEnableSimulatePhysics = false;
				break;
			}
		}
	}

	return bEnableSimulatePhysics;
}


EVisibility FBodyInstanceCustomizationHelper::IsMassVisible(bool bOverrideMass) const
{
	bool bIsMassReadOnly = IsBodyMassReadOnly();
	if (bOverrideMass)
	{
		return bIsMassReadOnly ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else
	{
		return bIsMassReadOnly ? EVisibility::Visible : EVisibility::Collapsed;
	}
}

bool FBodyInstanceCustomizationHelper::IsBodyMassReadOnly() const
{
	for (auto ObjectIt = ObjectsCustomized.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (ObjectIt->IsValid() && (*ObjectIt)->IsA(UPrimitiveComponent::StaticClass()))
		{
			if (UPrimitiveComponent* Comp = Cast<UPrimitiveComponent>(ObjectIt->Get()))
			{
				if (Comp->BodyInstance.bOverrideMass == false) { return true; }
			}
		}else if(ObjectIt->IsValid() && (*ObjectIt)->IsA(UBodySetup::StaticClass()))
		{
			UBodySetup* BS = Cast<UBodySetup>(ObjectIt->Get());
			if (BS->DefaultInstance.bOverrideMass == false)
			{
				return true;
			}
		}
	}

	return false;
}

TOptional<float> FBodyInstanceCustomizationHelper::OnGetBodyMaxAngularVelocity() const
{
	UPrimitiveComponent* Comp = nullptr;

	const float DefaultMaxAngularVelocity = UPhysicsSettings::Get()->MaxAngularVelocity;
	float MaxAngularVelocity = DefaultMaxAngularVelocity;
	bool bFoundComponent = false;

	for (auto ObjectIt = ObjectsCustomized.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (ObjectIt->IsValid() && (*ObjectIt)->IsA(UPrimitiveComponent::StaticClass()))
		{
			Comp = Cast<UPrimitiveComponent>(ObjectIt->Get());

			const float CompMaxAngularVelocity = Comp->BodyInstance.bOverrideMaxAngularVelocity ?
													Comp->BodyInstance.MaxAngularVelocity :
													DefaultMaxAngularVelocity;

			if (!bFoundComponent)
			{
				bFoundComponent = true;
				MaxAngularVelocity = CompMaxAngularVelocity;
			}
			else if (MaxAngularVelocity != CompMaxAngularVelocity)
			{
				return TOptional<float>();
			}
		}
	}

	return MaxAngularVelocity;
}

bool FBodyInstanceCustomizationHelper::IsMaxAngularVelocityReadOnly() const
{
	for (auto ObjectIt = ObjectsCustomized.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (ObjectIt->IsValid() && (*ObjectIt)->IsA(UPrimitiveComponent::StaticClass()))
		{
			if (UPrimitiveComponent* Comp = Cast<UPrimitiveComponent>(ObjectIt->Get()))
			{
				if (Comp->BodyInstance.bOverrideMaxAngularVelocity == false) { return true; }
			}
		}
	}

	return false;
}

EVisibility FBodyInstanceCustomizationHelper::IsMaxAngularVelocityVisible(bool bOverrideMaxAngularVelocity) const
{
	bool bIsMaxAngularVelocityReadOnly = IsMaxAngularVelocityReadOnly();
	if (bOverrideMaxAngularVelocity)
	{
		return bIsMaxAngularVelocityReadOnly ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else
	{
		return bIsMaxAngularVelocityReadOnly ? EVisibility::Visible : EVisibility::Collapsed;
	}
}

EVisibility FBodyInstanceCustomizationHelper::IsAutoWeldVisible() const
{
	for (int32 i = 0; i < ObjectsCustomized.Num(); ++i)
	{
		if (ObjectsCustomized[i].IsValid() && !(ObjectsCustomized[i]->IsA(UStaticMeshComponent::StaticClass()) || ObjectsCustomized[i]->IsA(UShapeComponent::StaticClass())))
		{
			return EVisibility::Collapsed;
		}
	}

	return EVisibility::Visible;
}

void FBodyInstanceCustomizationHelper::OnSetBodyMass(float BodyMass, ETextCommit::Type Commit)
{
	MassInKgOverrideHandle->SetValue(BodyMass);
}


TOptional<float> FBodyInstanceCustomizationHelper::OnGetBodyMass() const
{
	UPrimitiveComponent* Comp = nullptr;
	UBodySetup* BS = nullptr;

	float Mass = 0.0f;
	bool bMultipleValue = false;

	for (auto ObjectIt = ObjectsCustomized.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		float NewMass = 0.f;
		if (ObjectIt->IsValid() && (*ObjectIt)->IsA(UPrimitiveComponent::StaticClass()))
		{
			Comp = Cast<UPrimitiveComponent>(ObjectIt->Get());
			NewMass = Comp->CalculateMass();
		}
		else if (ObjectIt->IsValid() && (*ObjectIt)->IsA(UBodySetup::StaticClass()))
		{
			BS = Cast<UBodySetup>(ObjectIt->Get());

			NewMass = BS->CalculateMass();
		}

		if (Mass == 0.0f || FMath::Abs(Mass - NewMass) < SMALL_NUMBER)
		{
			Mass = NewMass;
		}
		else
		{
			bMultipleValue = true;
			break;
		}
	}

	if (bMultipleValue)
	{
		return TOptional<float>();
	}

	return Mass;
}

EVisibility FBodyInstanceCustomizationHelper::IsDOFMode(EDOFMode::Type Mode) const
{
	bool bVisible = false;
	if (DOFModeProperty.IsValid() && bDisplayConstraints)
	{
		uint8 CurrentMode;
		if (DOFModeProperty->GetValue(CurrentMode) == FPropertyAccess::Success)
		{
			EDOFMode::Type PropertyDOF = FBodyInstance::ResolveDOFMode(static_cast<EDOFMode::Type>(CurrentMode));
			bVisible = PropertyDOF == Mode;
		}
	}

	return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
}


void FBodyInstanceCustomizationHelper::AddMassInKg(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler)
{
	MassInKgOverrideHandle = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, MassInKgOverride)).ToSharedRef();

	if (bDisplayMass)
	{
		PhysicsCategory.AddProperty(MassInKgOverrideHandle).CustomWidget()
		.NameContent()
		[
			MassInKgOverrideHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<float>)
			.IsEnabled(this, &FBodyInstanceCustomizationHelper::IsBodyMassEnabled)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(this, &FBodyInstanceCustomizationHelper::OnGetBodyMass)
			.OnValueCommitted(this, &FBodyInstanceCustomizationHelper::OnSetBodyMass)
		];
	}
	else
	{
		MassInKgOverrideHandle->MarkHiddenByCustomization();
	}
}

void FBodyInstanceCustomizationHelper::AddMaxAngularVelocity(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler)
{
	TSharedRef<IPropertyHandle> MaxAngularVelocityHandle = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, MaxAngularVelocity)).ToSharedRef();
	
	PhysicsCategory.AddProperty(MaxAngularVelocityHandle).CustomWidget()
	.NameContent()
	[
		MaxAngularVelocityHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<float>)
			.IsEnabled(false)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(this, &FBodyInstanceCustomizationHelper::OnGetBodyMaxAngularVelocity)
			.Visibility(this, &FBodyInstanceCustomizationHelper::IsMaxAngularVelocityVisible, false)
		]

		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			.Visibility(this, &FBodyInstanceCustomizationHelper::IsMaxAngularVelocityVisible, true)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				MaxAngularVelocityHandle->CreatePropertyValueWidget()
			]
		]
	];
}

void FBodyInstanceCustomizationHelper::AddBodyConstraint(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler)
{
	const float XYZPadding = 5.f;
	
	TSharedPtr<IPropertyHandle> bLockXTranslation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockXTranslation));
	bLockXTranslation->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bLockYTranslation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockYTranslation));
	bLockYTranslation->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bLockZTranslation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockZTranslation));
	bLockZTranslation->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bLockXRotation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockXRotation));
	bLockXRotation->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bLockYRotation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockYRotation));
	bLockYRotation->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> bLockZRotation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockZRotation));
	bLockZRotation->MarkHiddenByCustomization();

	DOFModeProperty = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, DOFMode)).ToSharedRef();
	DOFModeProperty->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> bLockRotation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockRotation)).ToSharedRef();
	bLockRotation->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> bLockTranslation = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, bLockTranslation)).ToSharedRef();
	bLockTranslation->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> CustomDOFPlaneNormal = BodyInstanceHandler->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBodyInstance, CustomDOFPlaneNormal)).ToSharedRef();
	CustomDOFPlaneNormal->MarkHiddenByCustomization();

	//the above are all marked hidden even if we don't display constraints because the user wants to hide it anyway

	if (bDisplayConstraints)
	{
		IDetailGroup& ConstraintsGroup = PhysicsCategory.AddGroup(TEXT("ConstraintsGroup"), LOCTEXT("Constraints", "Constraints"));

		ConstraintsGroup.AddWidgetRow()
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBodyInstanceCustomizationHelper::IsDOFMode, EDOFMode::SixDOF)))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LockPositionLabel", "Lock Position"))
			.ToolTipText(LOCTEXT("LockPositionTooltip", "Locks movement along the specified axis"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Justification(ETextJustify::Right)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockXTranslation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockXTranslation->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockYTranslation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockYTranslation->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockZTranslation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockZTranslation->CreatePropertyValueWidget()
				]
			]
		];

		ConstraintsGroup.AddWidgetRow()
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBodyInstanceCustomizationHelper::IsDOFMode, EDOFMode::SixDOF)))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LockRotationLabel", "Lock Rotation"))
			.ToolTipText(LOCTEXT("LockRotationTooltip", "Locks rotation about the specified axis"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockXRotation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockXRotation->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockYRotation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockYRotation->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, XYZPadding, 0.f)
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockZRotation->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					bLockZRotation->CreatePropertyValueWidget()
				]
			]
		];

		//we only show the custom plane normal if we've selected that mode
		ConstraintsGroup.AddPropertyRow(CustomDOFPlaneNormal).Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBodyInstanceCustomizationHelper::IsDOFMode, EDOFMode::CustomPlane)));
		ConstraintsGroup.AddPropertyRow(bLockTranslation).Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBodyInstanceCustomizationHelper::IsDOFMode, EDOFMode::CustomPlane)));
		ConstraintsGroup.AddPropertyRow(bLockRotation).Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FBodyInstanceCustomizationHelper::IsDOFMode, EDOFMode::CustomPlane)));
		ConstraintsGroup.AddPropertyRow(DOFModeProperty.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
#undef RowWidth_Customization
