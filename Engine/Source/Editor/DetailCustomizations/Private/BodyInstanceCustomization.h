// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Serialization/Archive.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SComboBox.h"

class FDetailWidgetRow;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UCollisionProfile;
class UObject;
class UPrimitiveComponent;
class UStaticMeshComponent;

struct FCollisionChannelInfo
{
	FString				DisplayName;
	ECollisionChannel	CollisionChannel;
	bool				TraceType;
};

/**
 * Customizes a DataTable asset to use a dropdown
 */
class FBodyInstanceCustomization : public IPropertyTypeCustomization
{
public:
	FBodyInstanceCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable( new FBodyInstanceCustomization );
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	// Simulate physics toggle
	void OnSimulatePhysicsChanged();

	// Profile combo related
	TSharedRef<SWidget> MakeCollisionProfileComboWidget( TSharedPtr<FString> InItem );
	void OnCollisionProfileChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCollisionProfileComboBoxContent() const;
	FText GetCollisionProfileComboBoxToolTip() const;
	void OnCollisionProfileComboOpening();

	// Movement channel related
	TSharedRef<SWidget> MakeObjectTypeComboWidget(TSharedPtr<FString> InItem);
	void OnObjectTypeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetObjectTypeComboBoxContent() const;
	int32 InitializeObjectTypeComboList();

	// set to default for profile setting
	void SetToDefaultProfile();
	bool ShouldShowResetToDefaultProfile() const;

	void SetToDefaultResponse(int32 ValidIndex);
	bool ShouldShowResetToDefaultResponse(int32 ValidIndex) const;

	// collision channel check boxes
	void OnCollisionChannelChanged(ECheckBoxState InNewValue, int32 ValidIndex, ECollisionResponse InCollisionResponse);
	ECheckBoxState IsCollisionChannelChecked( int32 ValidIndex, ECollisionResponse InCollisionResponse) const;
	// all collision channel check boxes
	void OnAllCollisionChannelChanged(ECheckBoxState InNewValue, ECollisionResponse InCollisionResponse);
	ECheckBoxState IsAllCollisionChannelChecked(ECollisionResponse InCollisionResponse) const;

	// should show custom prop
	bool ShouldEnableCustomCollisionSetup() const;
	EVisibility ShouldShowCustomCollisionSetup() const;

	bool IsCollisionEnabled() const;

	// whether we can edit collision or if we're getting it from a default
	bool AreAllCollisionUsingDefault() const;

	// utility functions between property and struct
	void CreateCustomCollisionSetup(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder);
	void SetCollisionResponseContainer(const FCollisionResponseContainer& ResponseContainer);
	void SetResponse(int32 ValidIndex, ECollisionResponse InCollisionResponse);
	void UpdateCollisionProfile();
	TSharedPtr<FString> GetProfileString(FName ProfileName) const;

	void UpdateValidCollisionChannels();

	void RefreshCollisionProfiles();

	UStaticMeshComponent* GetDefaultCollisionProvider(const FBodyInstance* BI) const;
	void MarkAllBodiesDefaultCollision(bool bUseDefaultCollision);
	bool CanUseDefaultCollision() const;
	bool CanShowDefaultCollision() const;
	int32 GetNumberOfSpecialProfiles() const;
	int32 GetCustomIndex() const;
	int32 GetDefaultIndex() const;

private:
	// property handles
	TSharedPtr<IPropertyHandle> BodyInstanceHandle;
	TSharedPtr<IPropertyHandle> CollisionProfileNameHandle;
	TSharedPtr<IPropertyHandle> CollisionEnabledHandle;
	TSharedPtr<IPropertyHandle> ObjectTypeHandle;
	TSharedPtr<IPropertyHandle> CollisionResponsesHandle;
	TSharedPtr<IPropertyHandle> UseDefaultCollisionHandle;
	TSharedPtr<IPropertyHandle> StaticMeshHandle;

	// widget related variables
	TSharedPtr<class SComboBox<TSharedPtr<FString>>> CollisionProfileComboBox;
	TArray<TSharedPtr<FString>> CollisionProfileComboList;

	// movement channel related options
	TSharedPtr<class SComboBox<TSharedPtr<FString>>> ObjectTypeComboBox;
	TArray<TSharedPtr<FString>> ObjectTypeComboList;
	// matching ObjectType value to ComboList, technically you can search DisplayName all the time, but this seems just easier
	TArray<ECollisionChannel> ObjectTypeValues; 

	// default collision profile object
	UCollisionProfile* CollisionProfile;

	TArray<FBodyInstance*> BodyInstances;
	TArray<UPrimitiveComponent*> PrimComponents;
	TMap<FBodyInstance*, TWeakObjectPtr<UPrimitiveComponent>> BodyInstanceToPrimComponent;

	TArray<FCollisionChannelInfo> ValidCollisionChannels;
	IDetailLayoutBuilder* DetailBuilder = nullptr;
};

class FBodyInstanceCustomizationHelper  : public TSharedFromThis<FBodyInstanceCustomizationHelper>
{
public:
	FBodyInstanceCustomizationHelper(const TArray<TWeakObjectPtr<UObject>>& InObjectsCustomized);
	void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler);
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler, TFunction<void(TSharedRef<IPropertyHandle>)> CustomizeCoMNudge);

private:
	void UpdateFilters();
	bool IsSimulatePhysicsEditable() const;

	TOptional<float> OnGetBodyMass() const;
	void OnSetBodyMass(float InBodyMass, ETextCommit::Type Commit);
	bool IsBodyMassReadOnly() const;
	EVisibility IsMassVisible(bool bOverrideMass) const;
	bool IsBodyMassEnabled() const { return !IsBodyMassReadOnly(); }
	void AddMassInKg(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler);
	void AddBodyConstraint(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler);
	void AddMaxAngularVelocity(IDetailCategoryBuilder& PhysicsCategory, TSharedRef<IPropertyHandle> BodyInstanceHandler);

	EVisibility IsAutoWeldVisible() const;

	EVisibility IsMaxAngularVelocityVisible(bool bOverrideMaxAngularVelocity) const;
	TOptional<float> OnGetBodyMaxAngularVelocity() const;

	bool IsMaxAngularVelocityReadOnly() const;
	EVisibility IsDOFMode(EDOFMode::Type Mode) const;

private:
	bool bDisplayMass;
	bool bDisplayConstraints;
	bool bDisplayEnablePhysics;
	bool bDisplayAsyncScene;
	bool bDisplayLinearDamping;
	bool bDisplayAngularDamping;
	bool bDisplayEnableGravity;
	bool bDisplayInertiaConditioning;
	bool bDisplayInitialOverlapDepenetration;
	bool bDisplayWalkableSlopeOverride;
	bool bDisplayAutoWeld;
	bool bDisplayStartAwake;
	bool bDisplayCOMNudge;
	bool bDisplayMassScale;
	bool bDisplayMaxAngularVelocity;

	TSharedPtr<IPropertyHandle> MassInKgOverrideHandle;
	TSharedPtr<IPropertyHandle> DOFModeProperty;
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
};
