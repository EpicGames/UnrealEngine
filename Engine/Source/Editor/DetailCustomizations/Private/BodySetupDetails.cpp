// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodySetupDetails.h"

#include "BodyInstanceCustomization.h"
#include "BodySetupEnums.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "ObjectEditorUtils.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "BodySetupDetails"

TSharedRef<IDetailCustomization> FBodySetupDetails::MakeInstance()
{
	return MakeShareable( new FBodySetupDetails );
}

void FBodySetupDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Customize collision section
	{
		static const FName CollisionCategoryName(TEXT("Collision"));

		if ( DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, DefaultInstance))->IsValidHandle() )
		{
			DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
			TSharedPtr<IPropertyHandle> BodyInstanceHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, DefaultInstance));
			
			BodyInstanceCustomizationHelper = MakeShareable(new FBodyInstanceCustomizationHelper(ObjectsCustomized));
			BodyInstanceCustomizationHelper->CustomizeDetails(DetailBuilder, BodyInstanceHandler.ToSharedRef(), [this, &DetailBuilder](TSharedRef<IPropertyHandle> BodyInstanceHandle) { this->CustomizeCoMNudge(DetailBuilder, BodyInstanceHandle); });

			IDetailCategoryBuilder& CollisionCategory = DetailBuilder.EditCategory("Collision");
			DetailBuilder.HideProperty(BodyInstanceHandler);

			TSharedPtr<IPropertyHandle> CollisionTraceHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, CollisionTraceFlag));
			DetailBuilder.HideProperty(CollisionTraceHandler);

			// add physics properties to physics category
			uint32 NumChildren = 0;
			BodyInstanceHandler->GetNumChildren(NumChildren);

			// add all properties of this now - after adding 
			for (uint32 ChildIndex=0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = BodyInstanceHandler->GetChildHandle(ChildIndex);
				FName CategoryName = FObjectEditorUtils::GetCategoryFName(ChildProperty->GetProperty());
				if (CategoryName == CollisionCategoryName)
				{
					CollisionCategory.AddProperty(ChildProperty);
				}
			}
		}

		CollisionReponseHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBodySetup, CollisionReponse));
		TWeakPtr<IPropertyHandle> WeakCollisionReponseHandle = CollisionReponseHandle;

		IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(CollisionCategoryName);
		DetailCategoryBuilder.AddProperty(CollisionReponseHandle)
		.CustomWidget()
		.NameContent()
		[
			CollisionReponseHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([WeakCollisionReponseHandle]() 
			{
				int Value = 0;
				if(WeakCollisionReponseHandle.IsValid() && WeakCollisionReponseHandle.Pin()->GetValue(Value) == FPropertyAccess::Success)
				{
					return (Value == EBodyCollisionResponse::BodyCollision_Enabled) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}

				return ECheckBoxState::Undetermined;
			})
			.OnCheckStateChanged_Lambda([WeakCollisionReponseHandle](ECheckBoxState InCheckBoxState)
			{
				if(WeakCollisionReponseHandle.IsValid())
				{
					int Value = InCheckBoxState == ECheckBoxState::Checked ? EBodyCollisionResponse::BodyCollision_Enabled : EBodyCollisionResponse::BodyCollision_Disabled;
					WeakCollisionReponseHandle.Pin()->SetValue(Value);
				}
			})
		];
	}
}


TSharedRef<IDetailCustomization> FSkeletalBodySetupDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalBodySetupDetails);
}


void FSkeletalBodySetupDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	IDetailCategoryBuilder& Cat = DetailBuilder.EditCategory(TEXT("PhysicalAnimation"));

	const TArray<TWeakObjectPtr<UObject>>& ObjectsCustomizedLocal = ObjectsCustomized;
	auto PhysAnimEditable = [ObjectsCustomizedLocal]() -> bool
	{
		bool bVisible = ObjectsCustomizedLocal.Num() > 0;
		for (TWeakObjectPtr<UObject> WeakObj : ObjectsCustomizedLocal)
		{
			if (USkeletalBodySetup* BS = Cast<USkeletalBodySetup>(WeakObj.Get()))
			{
				if (!BS->FindPhysicalAnimationProfile(BS->GetCurrentPhysicalAnimationProfileName()))
				{
					bVisible = false;
					break;
				}
			}
			else
			{
				bVisible = false;
				break;
			}
		}

		return bVisible;
	};

	TAttribute<EVisibility> PhysAnimVisible = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([PhysAnimEditable]()
	{
		return PhysAnimEditable() == true ? EVisibility::Visible : EVisibility::Collapsed;
	}));

	TAttribute<EVisibility> NewPhysAnimButtonVisible = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([PhysAnimEditable]()
	{
		return PhysAnimEditable() == true ? EVisibility::Collapsed : EVisibility::Visible;
	}));

	IDetailCategoryBuilder& PhysicsCat = DetailBuilder.EditCategory(TEXT("Physics"));
	PhysicsCat.HeaderContent(
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.DecoratorStyleSet(&FAppStyle::Get())
				.Text_Lambda([ObjectsCustomizedLocal]()
				{
					if (ObjectsCustomizedLocal.Num() > 0)
					{
						if (USkeletalBodySetup* BS = Cast<USkeletalBodySetup>(ObjectsCustomizedLocal[0].Get()))
						{
							FName CurrentProfileName = BS->GetCurrentPhysicalAnimationProfileName();
							if(CurrentProfileName != NAME_None)
							{
								if(BS->FindPhysicalAnimationProfile(CurrentProfileName) != nullptr)
								{
									return FText::Format(LOCTEXT("ProfileFormatAssigned", "Assigned to Profile: <RichTextBlock.Bold>{0}</>"), FText::FromName(CurrentProfileName));
								}
								else
								{
									return FText::Format(LOCTEXT("ProfileFormatNotAssigned", "Not Assigned to Profile: <RichTextBlock.Bold>{0}</>"), FText::FromName(CurrentProfileName));
								}
							}
							else
							{
								return LOCTEXT("ProfileFormatNone", "Current Profile: <RichTextBlock.Bold>None</>");
							}
						}
					}

					return FText();
				})
			]
		);

	TSharedPtr<IPropertyHandle> PhysicalAnimationProfile = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletalBodySetup, CurrentPhysicalAnimationProfile));
	PhysicalAnimationProfile->MarkHiddenByCustomization();

	uint32 NumChildren = 0;
	TSharedPtr<IPropertyHandle> ProfileData = PhysicalAnimationProfile->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPhysicalAnimationProfile, PhysicalAnimationData));
	ProfileData->GetNumChildren(NumChildren);
	for(uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> Child = ProfileData->GetChildHandle(ChildIdx);
		if(!Child->IsCustomized())
		{
			Cat.AddProperty(Child)
			.Visibility(PhysAnimVisible);
		}
	}
}

#undef LOCTEXT_NAMESPACE

