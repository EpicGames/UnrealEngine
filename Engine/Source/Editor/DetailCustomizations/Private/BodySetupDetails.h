// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class IDetailLayoutBuilder;
class IPropertyHandle;
class UObject;

class FBodySetupDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	virtual void CustomizeCoMNudge(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> BodyInstanceHandler) {};

private:
	UE_API void AddPhysicalAnimation(IDetailLayoutBuilder& DetailBuilder);

private:
	TSharedPtr<class FBodyInstanceCustomizationHelper> BodyInstanceCustomizationHelper;
	TArray< TWeakObjectPtr<UObject> > ObjectsCustomized;
	TSharedPtr<IPropertyHandle> CollisionReponseHandle;
};



class FSkeletalBodySetupDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TArray< TWeakObjectPtr<UObject> > ObjectsCustomized;
};

#undef UE_API
