// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoundsCopyComponentDetails.h"

#include "Components/BoundsCopyComponent.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class UObject;

#define LOCTEXT_NAMESPACE "BoundsCopyComponentDetails"

FBoundsCopyComponentDetailsCustomization::FBoundsCopyComponentDetailsCustomization()
{
}

TSharedRef<IDetailCustomization> FBoundsCopyComponentDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FBoundsCopyComponentDetailsCustomization);
}

void FBoundsCopyComponentDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Get and store the linked UBoundsCopyComponent
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	BoundsCopyComponent = Cast<UBoundsCopyComponent>(ObjectsBeingCustomized[0].Get());
	if (!BoundsCopyComponent.IsValid())
	{
		return;
	}

	// Only reason for having any of the logic here is that CallInEditor doesn't seem to work to add buttons for Copy functions.
	IDetailCategoryBuilder& BoundsCategory = DetailBuilder.EditCategory("TransformFromBounds", FText::GetEmpty(), ECategoryPriority::Important);

	// Hide and re-add BoundsSourceActor property otherwise we lose the ordering of this property first.
	TSharedPtr<IPropertyHandle> SourceActorValue = DetailBuilder.GetProperty("BoundsSourceActor");
	DetailBuilder.HideProperty(SourceActorValue);

	BoundsCategory.AddCustomRow(SourceActorValue->GetPropertyDisplayName())
	.NameContent()
	[
		SourceActorValue->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	[
		SourceActorValue->CreatePropertyValueWidget()
	];

	// Add Copy buttons.
	BoundsCategory
	.AddCustomRow(LOCTEXT("Button_CopyRotation", "Copy Rotation"))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_CopyRotation", "Copy Rotation"))
	]
	.ValueContent()
	.MinDesiredWidth(125.f)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ContentPadding(2.f)
		.Text(LOCTEXT("Button_Copy", "Copy"))
		.ToolTipText(LOCTEXT("Button_CopyRotation_Tooltip", "Set the actor rotation to match the source actor"))
		.OnClicked(this, &FBoundsCopyComponentDetailsCustomization::SetRotation)
		.IsEnabled(this, &FBoundsCopyComponentDetailsCustomization::IsCopyEnabled)
	];

	const FText CheckboxCopy_X_Tooltip = LOCTEXT("CheckboxCopy_X_Tooltip", "Limit the change to the X component of the bounds");
	const FText CheckboxCopy_Y_Tooltip = LOCTEXT("CheckboxCopy_Y_Tooltip", "Limit the change to the Y component of the bounds");
	const FText CheckboxCopy_Z_Tooltip = LOCTEXT("CheckboxCopy_Z_Tooltip", "Limit the change to the Z component of the bounds");

	BoundsCategory
	.AddCustomRow(LOCTEXT("Button_CopyBounds", "Copy Bounds"))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("Button_CopyBounds", "Copy Bounds"))
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ContentPadding(2.f)
				.Text(LOCTEXT("Button_Copy", "Copy"))
				.ToolTipText(LOCTEXT("Button_CopyBounds_Tooltip", "Set the actor transform so that it includes the full bounds of the source actor"))
				.OnClicked(this, &FBoundsCopyComponentDetailsCustomization::SetTransformToBounds)
				.IsEnabled(this, &FBoundsCopyComponentDetailsCustomization::IsCopyEnabled)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsXChanged)
					.IsChecked(this, &FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsXChecked)
					.IsEnabled(this, &FBoundsCopyComponentDetailsCustomization::IsCopyEnabled)
					.ToolTipText(CheckboxCopy_X_Tooltip)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CheckboxCopy_X", "X"))
						.ToolTipText(CheckboxCopy_X_Tooltip)
						.MinDesiredWidth(10.0f)
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsYChanged)
					.IsChecked(this, &FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsYChecked)
					.IsEnabled(this, &FBoundsCopyComponentDetailsCustomization::IsCopyEnabled)
					.ToolTipText(CheckboxCopy_Y_Tooltip)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CheckboxCopy_Y", "Y"))
						.ToolTipText(CheckboxCopy_Y_Tooltip)
						.MinDesiredWidth(10.0f)
					]
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsZChanged)
					.IsChecked(this, &FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsZChecked)
					.IsEnabled(this, &FBoundsCopyComponentDetailsCustomization::IsCopyEnabled)
					.ToolTipText(CheckboxCopy_Z_Tooltip)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CheckboxCopy_Z", "Z"))
						.ToolTipText(CheckboxCopy_Z_Tooltip)
						.MinDesiredWidth(10.0f)
					]
				]
			]
		]
	];
}

void FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsXChanged(ECheckBoxState NewState)
{
	if (BoundsCopyComponent.IsValid())
	{
		BoundsCopyComponent->bCopyXBounds = !BoundsCopyComponent->bCopyXBounds;
	}
}

void FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsYChanged(ECheckBoxState NewState)
{
	if (BoundsCopyComponent.IsValid())
	{
		BoundsCopyComponent->bCopyYBounds = !BoundsCopyComponent->bCopyYBounds;
	}
}

void FBoundsCopyComponentDetailsCustomization::OnBoundsComponentsZChanged(ECheckBoxState NewState)
{
	if (BoundsCopyComponent.IsValid())
	{
		BoundsCopyComponent->bCopyZBounds = !BoundsCopyComponent->bCopyZBounds;
	}
}

ECheckBoxState FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsXChecked() const
{
	return BoundsCopyComponent.IsValid() ? (BoundsCopyComponent->bCopyXBounds ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
}

ECheckBoxState FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsYChecked() const
{
	return BoundsCopyComponent.IsValid() ? (BoundsCopyComponent->bCopyYBounds ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
}

ECheckBoxState FBoundsCopyComponentDetailsCustomization::IsBoundsComponentsZChecked() const
{
	return BoundsCopyComponent.IsValid() ? (BoundsCopyComponent->bCopyZBounds ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
}

bool FBoundsCopyComponentDetailsCustomization::IsCopyEnabled() const
{
	return BoundsCopyComponent.IsValid() && BoundsCopyComponent->BoundsSourceActor.IsValid();
}

FReply FBoundsCopyComponentDetailsCustomization::SetRotation()
{
	FScopedTransaction BakeTransaction(LOCTEXT("Transaction_CopyRotation", "Copy Rotation"));
	BoundsCopyComponent->SetRotation();
	return FReply::Handled();
}

FReply FBoundsCopyComponentDetailsCustomization::SetTransformToBounds()
{
	FScopedTransaction BakeTransaction(LOCTEXT("Transaction_CopyBounds", "Copy Bounds"));
	BoundsCopyComponent->SetTransformToBounds();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
