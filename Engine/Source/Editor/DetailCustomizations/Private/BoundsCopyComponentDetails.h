// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;

enum class ECheckBoxState : uint8;

class UBoundsCopyComponent;

/** UI customization for UBoundsCopyComponent */
class FBoundsCopyComponentDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FBoundsCopyComponentDetailsCustomization();

	/** Callback for Copy Rotation button */
	FReply SetRotation();
	/** Callback for Copy Bounds button */
	FReply SetTransformToBounds();

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	void OnBoundsComponentsXChanged(ECheckBoxState NewState);
	void OnBoundsComponentsYChanged(ECheckBoxState NewState);
	void OnBoundsComponentsZChanged(ECheckBoxState NewState);
	ECheckBoxState IsBoundsComponentsXChecked() const;
	ECheckBoxState IsBoundsComponentsYChecked() const;
	ECheckBoxState IsBoundsComponentsZChecked() const;
	bool IsCopyEnabled() const;

private:
	TWeakObjectPtr<UBoundsCopyComponent> BoundsCopyComponent;
};
