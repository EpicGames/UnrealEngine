// Copyright Epic Games, Inc. All Rights Reserved.

#include "BrushDetails.h"

#include "ActorEditorUtils.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Brush.h"
#include "Engine/BrushBuilder.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/Volume.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealEdGlobals.h"
#include "Subsystems/EditorActorSubsystem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class AActor;
class FUICommandList;
class SWidget;

#define LOCTEXT_NAMESPACE "BrushDetails"


TSharedRef<IDetailCustomization> FBrushDetails::MakeInstance()
{
	return MakeShareable( new FBrushDetails );
}

FBrushDetails::~FBrushDetails()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FBrushDetails::CustomizeDetails( IDetailLayoutBuilder& InDetailLayout )
{
	PropertyUtils = InDetailLayout.GetPropertyUtilities();

	// Get level editor commands for our menus
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TSharedRef<const FUICommandList> CommandBindings = LevelEditor.GetGlobalLevelEditorActions();
	const FLevelEditorCommands& Commands = LevelEditor.GetLevelEditorCommands();

	// See if we have a volume. If we do - we hide the BSP stuff (solidity, order)
	bool bHaveAVolume = false;
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = InDetailLayout.GetSelectedObjects();
	for (int32 ObjIdx = 0; ObjIdx < SelectedObjects.Num(); ObjIdx++)
	{
		if (ABrush* Brush = Cast<ABrush>(SelectedObjects[ObjIdx].Get()))
		{
			if (AVolume* Volume = Cast<AVolume>(Brush))
			{
				bHaveAVolume = true;
			}

			if (!FActorEditorUtils::IsABuilderBrush(Brush))
			{
				// Store the selected actors for use later. Its fine to do this when CustomizeDetails is called because if the selected actors changes, CustomizeDetails will be called again on a new instance
				// and our current resource would be destroyed.
				SelectedBrushes.Add(Brush);
			}
		}
	}

	FMenuBuilder PolygonsMenuBuilder( true, CommandBindings );
	{
		PolygonsMenuBuilder.BeginSection("BrushDetailsPolygons");
		{
			PolygonsMenuBuilder.AddMenuEntry( Commands.MergePolys );
			PolygonsMenuBuilder.AddMenuEntry( Commands.SeparatePolys );
		}
		PolygonsMenuBuilder.EndSection();
	}

	FMenuBuilder SolidityMenuBuilder( true, CommandBindings );
	{
		SolidityMenuBuilder.AddMenuEntry( Commands.MakeSolid );
		SolidityMenuBuilder.AddMenuEntry( Commands.MakeSemiSolid );
		SolidityMenuBuilder.AddMenuEntry( Commands.MakeNonSolid );
	}

	FMenuBuilder OrderMenuBuilder( true, CommandBindings );
	{
		OrderMenuBuilder.AddMenuEntry( Commands.OrderFirst );
		OrderMenuBuilder.AddMenuEntry( Commands.OrderLast );
	}

	// Hide the brush builder if it is NULL
	BrushBuilderHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ABrush, BrushBuilder));
	UObject* BrushBuilderObject = nullptr;
	BrushBuilderHandle->GetValue(BrushBuilderObject);
	if(BrushBuilderObject == nullptr)
	{
		InDetailLayout.HideProperty("BrushBuilder");
	}
	else
	{
		BrushBuilderObject->SetFlags( RF_Transactional );
	}

	IDetailCategoryBuilder& BrushBuilderCategory = InDetailLayout.EditCategory( "BrushSettings", FText::GetEmpty() );

	BrushBuilderCategory.AddProperty( GET_MEMBER_NAME_CHECKED(ABrush, BrushType) );
	BrushBuilderCategory.AddCustomRow( LOCTEXT("BrushShape", "Brush Shape") )
	.NameContent()
	[
		SNew( STextBlock )
		.Text( LOCTEXT("BrushShape", "Brush Shape"))
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	]
	.ValueContent()
	.MinDesiredWidth(105.f)
	.MaxDesiredWidth(105.f)
	[
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("BspModeBuildTooltip", "Rebuild this brush from a parametric builder."))
		.OnGetMenuContent(this, &FBrushDetails::GenerateBuildMenuContent)
		.ContentPadding(2.f)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &FBrushDetails::GetBuilderText)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
	];

	BrushBuilderCategory.AddCustomRow( FText::GetEmpty(), true )
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(1.0f)
		[
			SNew(SComboButton)
			.ContentPadding(2.f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("BrushDetails", "PolygonsMenu", "Polygons"))
				.ToolTipText(NSLOCTEXT("BrushDetails", "PolygonsMenu_ToolTip", "Polygon options"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.MenuContent()
			[
				PolygonsMenuBuilder.MakeWidget()
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(1.0f)
		[
			SNew(SComboButton)
			.ContentPadding(2.f)
			.Visibility(bHaveAVolume ? EVisibility::Collapsed : EVisibility::Visible)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("BrushDetails", "SolidityMenu", "Solidity"))
				.ToolTipText(NSLOCTEXT("BrushDetails", "SolidityMenu_ToolTip", "Solidity options"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.MenuContent()
			[
				SolidityMenuBuilder.MakeWidget()
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(1.0f)
		[
			SNew(SComboButton)
			.ContentPadding(2.f)
			.Visibility(bHaveAVolume ? EVisibility::Collapsed : EVisibility::Visible)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("BrushDetails", "OrderMenu", "Order"))
				.ToolTipText(NSLOCTEXT("BrushDetails", "OrderMenu_ToolTip", "Order options"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.MenuContent()
			[
				OrderMenuBuilder.MakeWidget()
			]
		]
	];

	TSharedPtr< SHorizontalBox > BrushHorizontalBox;

	BrushBuilderCategory.AddCustomRow( FText::GetEmpty(), true)
	[
		SAssignNew(BrushHorizontalBox, SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew( SButton )
			.ToolTipText( LOCTEXT("AlignBrushVerts_Tooltip", "Aligns each vertex of the brush to the grid.") )
			.OnClicked( FOnClicked::CreateSP(this, &FBrushDetails::ExecuteExecCommand, FString( TEXT("ACTOR ALIGN VERTS") ) ) )
			.HAlign( HAlign_Center )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("AlignBrushVerts", "Align Brush Vertices") )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		]
	];

	if (SelectedBrushes.Num() > 0)
	{
		BrushHorizontalBox->AddSlot()
		[
			SNew( SButton )
			.ToolTipText( LOCTEXT("CreateStaticMeshActor_Tooltip", "Creates a static mesh from selected brushes or volumes and replaces them in the scene with the new static mesh") )
			.OnClicked( this, &FBrushDetails::OnCreateStaticMesh )
			.HAlign( HAlign_Center )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("CreateStaticMeshActor", "Create Static Mesh") )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	}
}

FReply FBrushDetails::ExecuteExecCommand(FString InCommand)
{
	GUnrealEd->Exec(GWorld, *InCommand);
	return FReply::Handled();
}

TSharedRef<SWidget> FBrushDetails::GenerateBuildMenuContent()
{
	class FBrushFilter : public IClassViewerFilter
	{
	public:
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs)
		{
			return !InClass->HasAnyClassFlags(CLASS_NotPlaceable) && !InClass->HasAnyClassFlags(CLASS_Abstract) && InClass->IsChildOf(UBrushBuilder::StaticClass());
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs)
		{
			return false;
		}
	};

	FClassViewerInitializationOptions Options;
	Options.ClassFilters.Add(MakeShareable(new FBrushFilter));
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::ListView;
	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &FBrushDetails::OnClassPicked));
}

void FBrushDetails::OnClassPicked(UClass* InChosenClass)
{
	FSlateApplication::Get().DismissAllMenus();

	TArray<UObject*> OuterObjects;
	BrushBuilderHandle->GetOuterObjects(OuterObjects);

	struct FNewBrushBuilder
	{
		UBrushBuilder* Builder;
		ABrush* Brush;
	};

	TArray<FNewBrushBuilder> NewBuilders;
	TArray<FString> NewObjectPaths;

	if(BrushBuilderHandle->IsValidHandle() && OuterObjects.Num() > 0)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "BrushSet", "Brush Set"));
		for (UObject* OuterObject : OuterObjects)
		{
			UBrushBuilder* NewUObject = NewObject<UBrushBuilder>(OuterObject, InChosenClass, NAME_None, RF_Transactional);

			FNewBrushBuilder NewBuilder;
			NewBuilder.Builder = NewUObject;
			NewBuilder.Brush = CastChecked<ABrush>(OuterObject);

			NewBuilders.Add(NewBuilder);
			NewObjectPaths.Add(NewUObject->GetPathName());
		}

		// make sure the brushes are rebuilt
		for (FNewBrushBuilder& NewObject : NewBuilders)
		{
			NewObject.Builder->Build(NewObject.Brush->GetWorld(), NewObject.Brush);
		}

		BrushBuilderHandle->SetPerObjectValues(NewObjectPaths);

		GEditor->RebuildAlteredBSP();

		if (PropertyUtils.IsValid())
		{
			PropertyUtils.Pin()->ForceRefresh();
		}
	}
}

FText FBrushDetails::GetBuilderText() const
{
	UObject* Object = nullptr;
	BrushBuilderHandle->GetValue(Object);
	if (Object != nullptr)
	{
		UBrushBuilder* BrushBuilder = CastChecked<UBrushBuilder>(Object);
		const FText NameText = BrushBuilder->GetClass()->GetDisplayNameText();
		if (!NameText.IsEmpty())
		{
			return NameText;
		}
		else
		{
			return FText::FromString(FName::NameToDisplayString(BrushBuilder->GetClass()->GetName(), false));
		}
	}

	return LOCTEXT("None", "None");
}


END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply FBrushDetails::OnCreateStaticMesh()
{
	TArray<AActor*> ValidSelectedBrushes;
	CopyFromWeakArray(ValidSelectedBrushes, SelectedBrushes);

	UEditorActorSubsystem::ConvertActors(ValidSelectedBrushes, AStaticMeshActor::StaticClass(), TSet<FString>(), true);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

