// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ABrush;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class SHorizontalBox;
class SWidget;
class UClass;

class FBrushDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	~FBrushDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& InDetailLayout ) override;

private:
	/** Callback for creating a static mesh from valid selected brushes. */
	FReply OnCreateStaticMesh();

	FReply ExecuteExecCommand(FString InCommand);

	TSharedRef<SWidget> GenerateBuildMenuContent();

	void OnClassPicked(UClass* InChosenClass);

	FText GetBuilderText() const;

private:
	TSharedPtr<IPropertyHandle> BrushBuilderHandle;

	/** Holds a list of BSP brushes or volumes, used for converting to static meshes */
	TArray< TWeakObjectPtr<ABrush> > SelectedBrushes;

	/** Container widget for the geometry mode tools */
	TSharedPtr< SHorizontalBox > GeometryToolsContainer;

	TWeakPtr<IPropertyUtilities> PropertyUtils;
};
