// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraCropSettingsCustomization.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "ScopedTransaction.h"
#include "CineCameraSettings.h"

#define LOCTEXT_NAMESPACE "CameraCropSettingsCustomization"

FCameraCropSettingsCustomization::FCameraCropSettingsCustomization()
{
	TArray<FNamedPlateCropPreset> const& Presets = UCineCameraSettings::GetCropPresets();

	int32 const NumPresets = Presets.Num();
	// first create preset combo list
	PresetComboList.Empty(NumPresets + 1);

	// put all presets in the list
	for (FNamedPlateCropPreset const& P : Presets)
	{
		PresetComboList.Add(MakeShareable(new FString(P.Name)));
	}

	PresetComboList.Add(MakeShareable(new FString(TEXT("Custom..."))));
}

TSharedRef<IPropertyTypeCustomization> FCameraCropSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FCameraCropSettingsCustomization);
}

void FCameraCropSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// We only want the dropdown list outside of the settings class as the settings class is the thing
	// defining the presets we use for the dropdown
	const bool bInSettingsClass = StructPropertyHandle->GetOuterBaseClass() == UCineCameraSettings::StaticClass();

	if (!bInSettingsClass)
	{
		HeaderRow.
			NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(0.f)
			[
				SAssignNew(PresetComboBox, SComboBox< TSharedPtr<FString> >)
				.OptionsSource(&PresetComboList)
				.OnGenerateWidget(this, &FCameraCropSettingsCustomization::MakePresetComboWidget)
				.OnSelectionChanged(this, &FCameraCropSettingsCustomization::OnPresetChanged)
				.IsEnabled(this, &FCameraCropSettingsCustomization::IsPresetEnabled)
				.ContentPadding(2.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FCameraCropSettingsCustomization::GetPresetComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FCameraCropSettingsCustomization::GetPresetComboBoxContent)
				]
			];
	}
}



void FCameraCropSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}
	
	// Retrieve special case properties
	AspectRatioHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FPlateCropSettings, AspectRatio));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
	}
}

TSharedRef<SWidget> FCameraCropSettingsCustomization::MakePresetComboWidget(TSharedPtr<FString> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

bool FCameraCropSettingsCustomization::IsPresetEnabled() const
{
	bool bEnabled = false;
	if (AspectRatioHandle.IsValid())
	{
		bEnabled = (AspectRatioHandle->IsEditable() &&
				FSlateApplication::Get().GetNormalExecutionAttribute().Get());
	}
	return bEnabled;
}

void FCameraCropSettingsCustomization::OnPresetChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString const NewPresetName = *NewSelection.Get();

		// search presets for one that matches
		TArray<FNamedPlateCropPreset> const& Presets = UCineCameraSettings::GetCropPresets();
		int32 const NumPresets = Presets.Num();
		for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
		{
			FNamedPlateCropPreset const& P = Presets[PresetIdx];
			if (P.Name == NewPresetName)
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangeCropPreset", "Change Crop Preset"));

				// copy data from preset into properties
				// all SetValues except the last set to Interactive so we don't rerun construction scripts and invalidate subsequent property handles
				ensure(AspectRatioHandle->SetValue(P.CropSettings.AspectRatio, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				
				break;
			}
		}

		// if none of them found, do nothing
	}
}


FText FCameraCropSettingsCustomization::GetPresetComboBoxContent() const
{
	return FText::FromString(*GetPresetString().Get());
}


TSharedPtr<FString> FCameraCropSettingsCustomization::GetPresetString() const
{
  	float AspectRatio;
 	AspectRatioHandle->GetValue(AspectRatio);
 
	// search presets for one that matches
	TArray<FNamedPlateCropPreset> const& Presets = UCineCameraSettings::GetCropPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedPlateCropPreset const& P = Presets[PresetIdx];
		if (P.CropSettings.AspectRatio == AspectRatio)
		{
			// this is the one
			if (PresetComboList.IsValidIndex(PresetIdx))
			{
				return PresetComboList[PresetIdx];
			}
		}
	}

	return PresetComboList.Last();
}

#undef LOCTEXT_NAMESPACE // CameraCropSettingsCustomization
