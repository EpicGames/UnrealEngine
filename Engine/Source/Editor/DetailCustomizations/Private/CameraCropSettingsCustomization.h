// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IPropertyHandle;

class FCameraCropSettingsCustomization : public IPropertyTypeCustomization
{
public:
	FCameraCropSettingsCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:

	TSharedPtr<IPropertyHandle> AspectRatioHandle;

	TSharedPtr<class SComboBox< TSharedPtr<FString> > > PresetComboBox;
	TArray< TSharedPtr< FString > >						PresetComboList;

	bool IsPresetEnabled() const;
	void OnPresetChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> MakePresetComboWidget(TSharedPtr<FString> InItem);

	FText GetPresetComboBoxContent() const;
	TSharedPtr<FString> GetPresetString() const;

protected:
};
