// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraDetails.h"

#include "Camera/CameraComponent.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorCategoryUtils.h"
#include "IDetailGroup.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "CameraDetails"

const float FCameraDetails::MinAspectRatio = 0.1f;
const float FCameraDetails::MaxAspectRatio = 100.0f;
const float FCameraDetails::LowestCommonAspectRatio = 1.0f;
const float FCameraDetails::HighestCommonAspectRatio = 2.5f;


TSharedRef<IDetailCustomization> FCameraDetails::MakeInstance()
{
	return MakeShareable( new FCameraDetails );
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FCameraDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	FSlateFontInfo FontStyle = FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont"));

	LastParsedAspectRatioValue = -1.0f;

	TSharedPtr<IPropertyHandle> bConstrainAspectRatioProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bConstrainAspectRatio));
	TSharedPtr<IPropertyHandle> ProjectionModeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, ProjectionMode));
	AspectRatioProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, AspectRatio));
	check(AspectRatioProperty.IsValid());

	if (AspectRatioProperty.IsValid() && AspectRatioProperty->IsValidHandle())
	{
		FSimpleDelegate OnAspectRatioChangedDelegate = FSimpleDelegate::CreateSP(this, &FCameraDetails::OnAspectRatioChanged);
		AspectRatioProperty->SetOnPropertyValueChanged(OnAspectRatioChangedDelegate);
	}

	// see if CameraSettings category should be hidden
	bool bCameraSettingsHidden = false;
	{
		TArray< TWeakObjectPtr<UObject> > Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);

		for (TWeakObjectPtr<UObject> ObjWP : Objects)
		{
			UObject* const Obj = ObjWP.Get();
			if (Obj && (FEditorCategoryUtils::IsCategoryHiddenFromClass(Obj->GetClass(), TEXT("CameraSettings"))))
			{
				bCameraSettingsHidden = true;
				break;
			}
		}
	}

	IDetailCategoryBuilder& CurrentCameraSettingsCategory = DetailLayout.EditCategory("Current Camera Settings", FText::GetEmpty(), ECategoryPriority::Important);

	if (bCameraSettingsHidden == false)
	{
		IDetailCategoryBuilder& CameraCategory = DetailLayout.EditCategory( "CameraSettings", FText::GetEmpty(), ECategoryPriority::Important );

		// Organize the properties
		CameraCategory.AddProperty(ProjectionModeProperty);

		// Perspective-specific properties
		IDetailPropertyRow& FieldOfViewRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, FieldOfView)));
		FieldOfViewRow.Visibility( TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP( this, &FCameraDetails::ProjectionModeMatches, ProjectionModeProperty, ECameraProjectionMode::Perspective ) ) );

		// Orthographic-specific properties
		TAttribute<EVisibility> OrthographicVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FCameraDetails::ProjectionModeMatches, ProjectionModeProperty, ECameraProjectionMode::Orthographic));

		IDetailPropertyRow& OrthoWidthRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoWidth)));
		OrthoWidthRow.Visibility(OrthographicVisibility);

		IDetailPropertyRow& AutoCalculateOrthoPlanesRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bAutoCalculateOrthoPlanes)));
		AutoCalculateOrthoPlanesRow.Visibility(OrthographicVisibility);

		IDetailPropertyRow& AutoPlanesShift = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, AutoPlaneShift)));
		AutoPlanesShift.Visibility(OrthographicVisibility);

		IDetailPropertyRow& OrthoNearClipPlaneRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoNearClipPlane)));
		OrthoNearClipPlaneRow.Visibility(OrthographicVisibility);

		IDetailPropertyRow& OrthoFarClipPlaneRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, OrthoFarClipPlane)));
		OrthoFarClipPlaneRow.Visibility(OrthographicVisibility);

		IDetailPropertyRow& UpdateOrthoPlanesRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bUpdateOrthoPlanes)));
		UpdateOrthoPlanesRow.Visibility(OrthographicVisibility);

		IDetailPropertyRow& CameraHeightAsViewTargetRow = CameraCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bUseCameraHeightAsViewTarget)));
		CameraHeightAsViewTargetRow.Visibility(OrthographicVisibility);

		// Aspect ratio
		IDetailPropertyRow& AspectRatioRow = CameraCategory.AddProperty(AspectRatioProperty);

		// Provide the special aspect ratio row
		AspectRatioRow.CustomWidget()
			.NameContent()
			[
				AspectRatioProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(0.0f, 2.0f, 5.0f, 2.0f)
				[
					SNew(SNumericEntryBox<float>)
					.AllowSpin(true)
					.Value(this, &FCameraDetails::GetAspectRatio)
					.Font(FontStyle)
					.MinValue(MinAspectRatio)
					.MaxValue(MaxAspectRatio)
					.MinSliderValue(LowestCommonAspectRatio)
					.MaxSliderValue(HighestCommonAspectRatio)
					.OnValueChanged(this, &FCameraDetails::OnAspectRatioSpinnerChanged)
					.ToolTipText(LOCTEXT("AspectFloatTooltip", "Aspect Ratio (Width/Height)"))
				]
				+SHorizontalBox::Slot()
				[
					SNew(SComboButton)
					.OnGetMenuContent( this, &FCameraDetails::OnGetComboContent )
					.ContentPadding(0.0f)
					.ButtonStyle( FAppStyle::Get(), "ToggleButton" )
					.ForegroundColor(FSlateColor::UseForeground())
					.VAlign(VAlign_Center)
					.ButtonContent()
					[
						SAssignNew(AspectTextBox, SEditableTextBox)
						.HintText( LOCTEXT("AspectTextHint", "width x height") )
						.ToolTipText( LOCTEXT("AspectTextTooltip", "Enter a ratio in the form \'width x height\' or \'width:height\'") )
						.Font(FontStyle)
						.OnTextCommitted(this, &FCameraDetails::OnCommitAspectRatioText)
					]
				]
			];

		IDetailGroup& OverscanGroup = CameraCategory.AddGroup(TEXT("Overscan"), LOCTEXT("OverscanGroup", "Overscan"));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, Overscan)));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bScaleResolutionWithOverscan)));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bCropOverscan)));
	}
	else
	{
		IDetailGroup& OverscanGroup = CurrentCameraSettingsCategory.AddGroup(TEXT("Overscan"), LOCTEXT("OverscanGroup", "Overscan"));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, Overscan)));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bScaleResolutionWithOverscan)));
		OverscanGroup.AddPropertyRow(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bCropOverscan)));
	}

	IDetailCategoryBuilder& CameraSettingsCategory = DetailLayout.EditCategory( "CameraOptions", FText::GetEmpty(), ECategoryPriority::Important );
	CameraSettingsCategory.AddProperty( bConstrainAspectRatioProperty );
	CameraSettingsCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, bUsePawnControlRotation)));
	CameraSettingsCategory.AddProperty(DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UCameraComponent, PostProcessBlendWeight)));

	UpdateAspectTextFromProperty();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FCameraDetails::OnAspectRatioChanged()
{
	// Callback set as the SetOnPropertyValueChanged() on the actual aspect ratio property
	UpdateAspectTextFromProperty();
}

TOptional<float> FCameraDetails::GetAspectRatio() const
{
	// Gets the actual aspect ratio property value
	float Value = 0.0f;
	if (AspectRatioProperty->GetValue(Value) == FPropertyAccess::Success)
	{
		return Value;
	}
	return TOptional<float>();
}

void FCameraDetails::OnAspectRatioSpinnerChanged(float InValue)
{
	// Called when the user inputs a new aspect ratio into the spinbox
	AspectRatioProperty->SetValue(InValue);
	UpdateAspectTextFromProperty();
}

void FCameraDetails::UpdateAspectTextFromProperty()
{
	// Called whenever the actual aspect ratio property changes - clears the text box if the value no longer matches the current text
	if (AspectTextBox.IsValid())
	{
		TOptional<float> Value = GetAspectRatio();
		if (!Value.IsSet() || Value.GetValue() < LastParsedAspectRatioValue - DELTA || Value.GetValue() > LastParsedAspectRatioValue + DELTA)
		{
			LastParsedAspectRatioValue = -1.0f;
			if (!AspectTextBox->GetText().IsEmpty())
			{
				AspectTextBox->SetText(FText::GetEmpty());
			}
		}
	}
}

TSharedRef<SWidget> FCameraDetails::OnGetComboContent() const
{
	// Fill the combo menu with presets of common screen resolutions
	FMenuBuilder MenuBuilder(true, NULL);

	TArray<FText> Items;
	Items.Add(LOCTEXT("PresetRatio640x480", "640x480 (4:3, 1.33) SDTV"));
	Items.Add(LOCTEXT("PresetRatio852x480", "852x480 (16:9, 1.78) SDTV Widescreen"));
	Items.Add(LOCTEXT("PresetRatio1280x720", "1280x720 (16:9, 1.78) HDTV 720"));
	Items.Add(LOCTEXT("PresetRatio1920x1080", "1920x1080 (16:9, 1.78) HDTV 1080"));
	Items.Add(LOCTEXT("PresetRatio960x544", "960x544 (16:9, 1.76) PS Vita"));
	Items.Add(LOCTEXT("PresetRatio1024x640", "1024x640 (1.6)"));
	Items.Add(LOCTEXT("PresetRatio1024x76", "1024x768 (4:3, 1.33)"));
	Items.Add(LOCTEXT("PresetRatio1366x768", "1366x768 (16:9, 1.78)"));
	Items.Add(LOCTEXT("PresetRatio2048x1536", "2048x1536 (4:3, 1.33) iPad 3"));
	Items.Add(LOCTEXT("PresetRatio4096x2304", "4096x2304 (16:9, 1.78) 4K"));

	for (auto ItemIter = Items.CreateConstIterator(); ItemIter; ++ItemIter)
	{
		FText ItemText = *ItemIter;
		FUIAction ItemAction( FExecuteAction::CreateSP( const_cast<FCameraDetails*>(this), &FCameraDetails::CommitAspectRatioText, ItemText ) );
		MenuBuilder.AddMenuEntry(ItemText, TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

void FCameraDetails::CommitAspectRatioText(FText ItemText)
{
	// placing new text into the box - so set the actual text then run the 'oncommit' handler
	AspectTextBox->SetText(ItemText);
	OnCommitAspectRatioText(ItemText, ETextCommit::Default);
}

void FCameraDetails::OnCommitAspectRatioText(const FText& ItemFText, ETextCommit::Type CommitInfo)
{
	// Parse the text assuming the following format
	// <INTEGER><optional whitespace><x or : or /><optional whitespace><INTEGER><optional extra info>
	FString ItemText = ItemFText.ToString();
	float ParsedRatio = -1.0f;
	int32 DelimIdx = INDEX_NONE;

	if (!ItemText.FindChar(TCHAR('x'), DelimIdx))
	{
		if (!ItemText.FindChar(TCHAR('X'), DelimIdx))
		{
			if (!ItemText.FindChar(TCHAR(':'), DelimIdx))
			{
				ItemText.FindChar(TCHAR('/'), DelimIdx);
			}
		}
	}

	if (DelimIdx != INDEX_NONE)
	{
		int32 Width;
		TTypeFromString<int32>::FromString(Width, *ItemText.Mid(0, DelimIdx).TrimStartAndEnd());
		if (Width > 0)
		{
			int32 WSIdx;
			FString RemainingText = ItemText.Mid(DelimIdx + 1).TrimStart();
			if (RemainingText.FindChar(TCHAR(' '), WSIdx))
			{
				RemainingText.LeftInline(WSIdx, EAllowShrinking::No);
			}
			int32 Height;
			TTypeFromString<int32>::FromString(Height, *RemainingText);
			if (Height > 0)
			{
				ParsedRatio = (float)Width / (float)Height;
			}
		}
	}

	if (ParsedRatio < 0.0f)
	{
		// invalid text - value couldn't be read
	}
	else if (ParsedRatio < MinAspectRatio)
	{
		// invalid value - too small
	}
	else if (ParsedRatio > MaxAspectRatio)
	{
		// invalid value - too large
	}
	else
	{
		// valid ratio parsed from text
		LastParsedAspectRatioValue = ParsedRatio;
		AspectRatioProperty->SetValue(ParsedRatio);
	}
}

EVisibility FCameraDetails::ProjectionModeMatches(TSharedPtr<IPropertyHandle> Property, ECameraProjectionMode::Type DesiredMode) const
{
	if (Property.IsValid())
	{
		uint8 ValueAsByte;
		FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

		if (Result == FPropertyAccess::Success)
		{
			return (((ECameraProjectionMode::Type)ValueAsByte) == DesiredMode) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	// If there are multiple values, show all properties
	return EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE

