// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraTypes.h"
#include "IDetailCustomization.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FText;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class SWidget;

class FCameraDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	void OnAspectRatioChanged();
	TOptional<float> GetAspectRatio() const;
	void OnAspectRatioSpinnerChanged(float InValue);
	void UpdateAspectTextFromProperty();
	TSharedRef<SWidget> OnGetComboContent() const;
	void CommitAspectRatioText(FText ItemText);
	void OnCommitAspectRatioText(const FText& ItemFText, ETextCommit::Type CommitInfo);

	EVisibility ProjectionModeMatches(TSharedPtr<IPropertyHandle> Property, ECameraProjectionMode::Type DesiredMode) const;

	TSharedPtr<IPropertyHandle> AspectRatioProperty;
	TSharedPtr<SEditableTextBox> AspectTextBox;
	float LastParsedAspectRatioValue;

	static const float MinAspectRatio;
	static const float MaxAspectRatio;
	static const float LowestCommonAspectRatio;
	static const float HighestCommonAspectRatio;
};
