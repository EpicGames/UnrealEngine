// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraFilmbackSettingsCustomization.h"

#include "CineCameraComponent.h"
#include "Containers/Map.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

class IDetailPropertyRow;
class SWidget;

#define LOCTEXT_NAMESPACE "CameraFilmbackSettingsCustomization"

void FCameraFilmbackSettingsCustomization::BuildPresetComboList()
{
	TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();

	int32 const NumPresets = Presets.Num();
	// first create preset combo list
	PresetComboList.Empty(NumPresets + 1);

	// first one is default one
	PresetComboList.Add(MakeShareable(new FString(TEXT("Custom..."))));

	// put all presets in the list
	for (FNamedFilmbackPreset const& P : Presets)
	{
		PresetComboList.Add(MakeShareable(new FString(P.Name)));
	}
}

FCameraFilmbackSettingsCustomization::FCameraFilmbackSettingsCustomization()
{
	BuildPresetComboList();
}

TSharedRef<IPropertyTypeCustomization> FCameraFilmbackSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FCameraFilmbackSettingsCustomization);
}

void FCameraFilmbackSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];

	// We only want the dropdown list outside of the settings class as the settings class is the thing
	// defining the presets we use for the dropdown
	const bool bInSettingsClass = StructPropertyHandle->GetOuterBaseClass() == UCineCameraSettings::StaticClass();

	if (!bInSettingsClass)
	{
		HeaderRow.
			ValueContent()
			.MaxDesiredWidth(0.f)
			[
				SAssignNew(PresetComboBox, SComboBox< TSharedPtr<FString> >)
				.OptionsSource(&PresetComboList)
				.OnGenerateWidget(this, &FCameraFilmbackSettingsCustomization::MakePresetComboWidget)
				.OnSelectionChanged(this, &FCameraFilmbackSettingsCustomization::OnPresetChanged)
				.OnComboBoxOpening(this, &FCameraFilmbackSettingsCustomization::BuildPresetComboList)
				.IsEnabled(this, &FCameraFilmbackSettingsCustomization::IsPresetEnabled)
				.ContentPadding(2.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FCameraFilmbackSettingsCustomization::GetPresetComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FCameraFilmbackSettingsCustomization::GetPresetComboBoxContent)
				]
			];
	}
}

void FCameraFilmbackSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren( NumChildren );	
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;	
	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle( ChildIndex ).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}
	
	// Retrieve special case properties
	SensorWidthHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFilmbackSettings, SensorWidth));
	SensorHeightHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFilmbackSettings, SensorHeight));
	SensorHorizontalOffsetHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFilmbackSettings, SensorHorizontalOffset));
	SensorVerticalOffsetHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFilmbackSettings, SensorVerticalOffset));

	for( auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter  )
	{
		IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
	}	
}

TSharedRef<SWidget> FCameraFilmbackSettingsCustomization::MakePresetComboWidget(TSharedPtr<FString> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

bool FCameraFilmbackSettingsCustomization::IsPresetEnabled() const
{
	bool bEnabled = false;
	if (SensorHeightHandle.IsValid() && 
			SensorWidthHandle.IsValid() &&
			SensorHorizontalOffsetHandle.IsValid() &&
			SensorVerticalOffsetHandle.IsValid())
	{
		bEnabled = (
				SensorHeightHandle->IsEditable() &&
				SensorWidthHandle->IsEditable() &&
				SensorHorizontalOffsetHandle->IsEditable() &&
				SensorVerticalOffsetHandle->IsEditable() &&
				FSlateApplication::Get().GetNormalExecutionAttribute().Get());
	}
	return bEnabled;
}

void FCameraFilmbackSettingsCustomization::OnPresetChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString const NewPresetName = *NewSelection.Get();

		// search presets for one that matches
		TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();
		int32 const NumPresets = Presets.Num();
		for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
		{
			FNamedFilmbackPreset const& P = Presets[PresetIdx];
			if (P.Name == NewPresetName)
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangeFilmbackPreset", "Change Filmback Preset"));
				
				// copy data from preset into properties
				// all SetValues except the last set to Interactive so we don't rerun construction scripts and invalidate subsequent property handles
				ensure(SensorHeightHandle->SetValue(P.FilmbackSettings.SensorHeight, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(SensorWidthHandle->SetValue(P.FilmbackSettings.SensorWidth, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(SensorHorizontalOffsetHandle->SetValue(P.FilmbackSettings.SensorHorizontalOffset, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(SensorVerticalOffsetHandle->SetValue(P.FilmbackSettings.SensorVerticalOffset, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				
				break;
			}
		}

		// if none of them found, do nothing
	}
}


FText FCameraFilmbackSettingsCustomization::GetPresetComboBoxContent() const
{
	// just test one variable for multiple selection
	float CurSensorWidth;
	if (SensorWidthHandle->GetValue(CurSensorWidth) == FPropertyAccess::Result::MultipleValues)
	{
		// multiple selection
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*GetPresetString().Get());
}


TSharedPtr<FString> FCameraFilmbackSettingsCustomization::GetPresetString() const
{
	float CurSensorWidth;
	SensorWidthHandle->GetValue(CurSensorWidth);

	float CurSensorHeight;
	SensorHeightHandle->GetValue(CurSensorHeight);

	float CurSensorHorizontalOffset;
	SensorHorizontalOffsetHandle->GetValue(CurSensorHorizontalOffset);

	float CurSensorVerticalOffset;
	SensorVerticalOffsetHandle->GetValue(CurSensorVerticalOffset);

	// search presets for one that matches
	TArray<FNamedFilmbackPreset> const& Presets = UCineCameraSettings::GetFilmbackPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedFilmbackPreset const& P = Presets[PresetIdx];
		if ((P.FilmbackSettings.SensorWidth == CurSensorWidth) && 
				(P.FilmbackSettings.SensorHeight == CurSensorHeight) &&
				(P.FilmbackSettings.SensorHorizontalOffset == CurSensorHorizontalOffset) &&
				(P.FilmbackSettings.SensorVerticalOffset == CurSensorVerticalOffset))
		{
			// this is the one
			if (PresetComboList.IsValidIndex(PresetIdx + 1))
			{
				return PresetComboList[PresetIdx + 1];
			}
		}
	}

	return PresetComboList[0];
}


#undef LOCTEXT_NAMESPACE // CameraFilmbackSettingsCustomization
