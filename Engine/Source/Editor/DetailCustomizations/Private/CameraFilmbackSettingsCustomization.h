// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboBox.h"

class FDetailWidgetRow;
class IPropertyHandle;
class SWidget;

class FCameraFilmbackSettingsCustomization : public IPropertyTypeCustomization
{
public:
	FCameraFilmbackSettingsCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:

	TSharedPtr<IPropertyHandle> SensorWidthHandle;
	TSharedPtr<IPropertyHandle> SensorHeightHandle;
	TSharedPtr<IPropertyHandle> SensorHorizontalOffsetHandle;
	TSharedPtr<IPropertyHandle> SensorVerticalOffsetHandle;

	TSharedPtr<class SComboBox< TSharedPtr<FString> > > PresetComboBox;
	TArray< TSharedPtr< FString > >						PresetComboList;

	void BuildPresetComboList();

	bool IsPresetEnabled() const;
	void OnPresetChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> MakePresetComboWidget(TSharedPtr<FString> InItem);
		
	FText GetPresetComboBoxContent() const;
	TSharedPtr<FString> GetPresetString() const;
};
