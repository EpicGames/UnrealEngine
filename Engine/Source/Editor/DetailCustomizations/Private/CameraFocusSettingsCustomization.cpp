// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraFocusSettingsCustomization.h"

#include "Camera/CameraComponent.h"
#include "CineCameraComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SceneDepthPickerMode.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "CameraFocusSettingsCustomization"

static FName NAME_Category(TEXT("Category"));
static FString ManualFocusSettingsString(TEXT("Manual Focus Settings"));
static FString TrackingFocusSettingsString(TEXT("Tracking Focus Settings"));
static FString GeneralFocusSettingsString(TEXT("Focus Settings"));

TSharedRef<IPropertyTypeCustomization> FCameraFocusSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FCameraFocusSettingsCustomization);
}

void FCameraFocusSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FCameraFocusSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Retrieve special case properties
	FocusMethodHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFocusSettings, FocusMethod));
	ManualFocusDistanceHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraFocusSettings, ManualFocusDistance));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// make the widget
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());

		// set up delegate to know if we need to hide it
		FString const& Category = Iter.Value()->GetMetaData(NAME_Category);
		if (Category == ManualFocusSettingsString)
		{
			PropertyRow.Visibility(TAttribute<EVisibility>(this, &FCameraFocusSettingsCustomization::IsManualSettingGroupVisible));
		}
		else if (Category == TrackingFocusSettingsString)
		{
			PropertyRow.Visibility(TAttribute<EVisibility>(this, &FCameraFocusSettingsCustomization::IsTrackingSettingGroupVisible));
		}
		else if (Category == GeneralFocusSettingsString)
		{
			PropertyRow.Visibility(TAttribute<EVisibility>(this, &FCameraFocusSettingsCustomization::IsGeneralSettingGroupVisible));
		}

		// special customization to show scene depth picker widget
		if (Iter.Value() == ManualFocusDistanceHandle)
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			FDetailWidgetRow Row;
			PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

			PropertyRow.CustomWidget(/*bShowChildren*/ true)
			.NameContent()
			[
				NameWidget.ToSharedRef()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					ValueWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					PropertyCustomizationHelpers::MakeSceneDepthPicker(FOnSceneDepthLocationSelected::CreateSP(this, &FCameraFocusSettingsCustomization::OnSceneDepthLocationSelected))
				]
			];
		}
	}

}

void FCameraFocusSettingsCustomization::OnSceneDepthLocationSelected(FVector PickedSceneLoc)
{
	if (PickedSceneLoc != FVector::ZeroVector)
	{
		// find the camera component and set it relative to that
		UCameraComponent* OuterCameraComponent = nullptr;
		{
			TArray<UObject*> OuterObjects;
			ManualFocusDistanceHandle->GetOuterObjects(OuterObjects);
			for (UObject* Obj : OuterObjects)
			{
				UCameraComponent* const CamComp = dynamic_cast<UCameraComponent*>(Obj);
				if (CamComp)
				{
					OuterCameraComponent = CamComp;
					break;
				}
			}
		}

		if (OuterCameraComponent)
		{
			FVector const CamToPickedLoc = PickedSceneLoc - OuterCameraComponent->GetComponentLocation();
			FVector const CamForward = OuterCameraComponent->GetComponentRotation().Vector();

			// if picked behind camera, don't set it
			if ((CamToPickedLoc | CamForward) > 0.f)
			{
				float const FinalSceneDepth = static_cast<float>(CamToPickedLoc.ProjectOnToNormal(CamForward).Size());

				const FScopedTransaction Transaction(LOCTEXT("PickedSceneDepth", "Pick Scene Depth"));
				ensure(ManualFocusDistanceHandle->SetValue(FinalSceneDepth, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
			}
		}
	}
}

EVisibility FCameraFocusSettingsCustomization::IsManualSettingGroupVisible() const
{
	uint8 FocusMethodNumber;
	FocusMethodHandle->GetValue(FocusMethodNumber);
	ECameraFocusMethod const FocusMethod = static_cast<ECameraFocusMethod>(FocusMethodNumber);
	if (FocusMethod == ECameraFocusMethod::Manual)
	{
		// if focus method is set to none, all non-none setting groups are collapsed
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FCameraFocusSettingsCustomization::IsTrackingSettingGroupVisible() const
{
	uint8 FocusMethodNumber;
	FocusMethodHandle->GetValue(FocusMethodNumber);
	ECameraFocusMethod const FocusMethod = static_cast<ECameraFocusMethod>(FocusMethodNumber);
	if (FocusMethod == ECameraFocusMethod::Tracking)
	{
		// if focus method is set to none, all non-none setting groups are collapsed
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FCameraFocusSettingsCustomization::IsGeneralSettingGroupVisible() const
{
	uint8 FocusMethodNumber;
	FocusMethodHandle->GetValue(FocusMethodNumber);
	ECameraFocusMethod const FocusMethod = static_cast<ECameraFocusMethod>(FocusMethodNumber);
	if (FocusMethod != ECameraFocusMethod::Disable && FocusMethod != ECameraFocusMethod::DoNotOverride)
	{
		// if focus method is set to disable, all non-none setting groups are collapsed
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE // CameraFocusSettingsCustomization
