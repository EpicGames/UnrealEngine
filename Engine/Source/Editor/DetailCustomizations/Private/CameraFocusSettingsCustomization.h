// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IPropertyHandle;

class FCameraFocusSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:
 	TSharedPtr<IPropertyHandle> FocusMethodHandle;
 	TSharedPtr<IPropertyHandle> ManualFocusDistanceHandle;

private:
	EVisibility IsManualSettingGroupVisible() const;
	EVisibility IsSpotSettingGroupVisible() const;
	EVisibility IsTrackingSettingGroupVisible() const;
	EVisibility IsGeneralSettingGroupVisible() const;

	void OnSceneDepthLocationSelected(FVector PickedSceneLoc);
};
