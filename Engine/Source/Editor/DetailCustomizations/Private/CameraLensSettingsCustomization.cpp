// Copyright Epic Games, Inc. All Rights Reserved.

#include "CameraLensSettingsCustomization.h"

#include "CineCameraComponent.h"
#include "Containers/Map.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

class IDetailPropertyRow;
class SWidget;

#define LOCTEXT_NAMESPACE "CameraLensSettingsCustomization"

void FCameraLensSettingsCustomization::BuildPresetComboList()
{
	TArray<FNamedLensPreset> const& Presets = UCineCameraSettings::GetLensPresets();

	int32 const NumPresets = Presets.Num();
	// first create preset combo list
	PresetComboList.Empty(NumPresets + 1);

	// first one is default one
	PresetComboList.Add(MakeShareable(new FString(TEXT("Custom..."))));

	// put all presets in the list
	for (FNamedLensPreset const& P : Presets)
	{
		PresetComboList.Add(MakeShareable(new FString(P.Name)));
	}
}

FCameraLensSettingsCustomization::FCameraLensSettingsCustomization()
{
	BuildPresetComboList();
}

TSharedRef<IPropertyTypeCustomization> FCameraLensSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FCameraLensSettingsCustomization);
}

void FCameraLensSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];

	// We only want the dropdown list outside of the settings class as the settings class is the thing
	// defining the presets we use for the dropdown
	const bool bInSettingsClass = StructPropertyHandle->GetOuterBaseClass() == UCineCameraSettings::StaticClass();

	if (!bInSettingsClass)
	{
		HeaderRow.
			ValueContent()
			.MaxDesiredWidth(0.f)
			[
				SAssignNew(PresetComboBox, SComboBox< TSharedPtr<FString> >)
				.OptionsSource(&PresetComboList)
				.OnGenerateWidget(this, &FCameraLensSettingsCustomization::MakePresetComboWidget)
				.OnSelectionChanged(this, &FCameraLensSettingsCustomization::OnPresetChanged)
				.OnComboBoxOpening(this, &FCameraLensSettingsCustomization::BuildPresetComboList)
				.IsEnabled(this, &FCameraLensSettingsCustomization::IsPresetEnabled)
				.ContentPadding(2.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FCameraLensSettingsCustomization::GetPresetComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FCameraLensSettingsCustomization::GetPresetComboBoxContent)
				]
			];
	}
}



void FCameraLensSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		PropertyHandles.Add(PropertyName, ChildHandle);
	}
	
	// Retrieve special case properties
	MinFocalLengthHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MinFocalLength));
	MaxFocalLengthHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MaxFocalLength));
	MinFStopHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MinFStop));
	MaxFStopHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MaxFStop));
	MinFocusDistanceHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, MinimumFocusDistance));
	SqueezeFactorHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, SqueezeFactor));
	DiaphragmBladeCountHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FCameraLensSettings, DiaphragmBladeCount));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		if (Iter.Value() == MinFocusDistanceHandle)
		{
			// skip showing these in the panel for now, as we don't really use them
			continue;
		}

		IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
	}
}

TSharedRef<SWidget> FCameraLensSettingsCustomization::MakePresetComboWidget(TSharedPtr<FString> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

bool FCameraLensSettingsCustomization::IsPresetEnabled() const
{
	bool bEnabled = false;
	if (MinFocalLengthHandle.IsValid() &&
			MaxFocalLengthHandle.IsValid() &&
			MinFStopHandle.IsValid() &&
			MaxFStopHandle.IsValid() &&
			MinFocusDistanceHandle.IsValid() &&
			SqueezeFactorHandle.IsValid() &&
			DiaphragmBladeCountHandle.IsValid())
	{
		bEnabled = (MinFocalLengthHandle->IsEditable() &&
				MaxFocalLengthHandle->IsEditable() &&
				MinFStopHandle->IsEditable() &&
				MaxFStopHandle->IsEditable() &&
				MinFocusDistanceHandle->IsEditable() &&
				SqueezeFactorHandle->IsEditable() &&
				DiaphragmBladeCountHandle->IsEditable() &&
				FSlateApplication::Get().GetNormalExecutionAttribute().Get());
	}
	return bEnabled;
}

void FCameraLensSettingsCustomization::OnPresetChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString const NewPresetName = *NewSelection.Get();

		// search presets for one that matches
		TArray<FNamedLensPreset> const& Presets = UCineCameraSettings::GetLensPresets();
		int32 const NumPresets = Presets.Num();
		for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
		{
			FNamedLensPreset const& P = Presets[PresetIdx];
			if (P.Name == NewPresetName)
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangeLensPreset", "Change Lens Preset"));

				// copy data from preset into properties
				// all SetValues except the last set to Interactive so we don't rerun construction scripts and invalidate subsequent property handles
				ensure(MinFocalLengthHandle->SetValue(P.LensSettings.MinFocalLength, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(MaxFocalLengthHandle->SetValue(P.LensSettings.MaxFocalLength, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(MinFStopHandle->SetValue(P.LensSettings.MinFStop, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(MaxFStopHandle->SetValue(P.LensSettings.MaxFStop, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(MinFocusDistanceHandle->SetValue(P.LensSettings.MinimumFocusDistance, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(SqueezeFactorHandle->SetValue(P.LensSettings.SqueezeFactor, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);
				ensure(DiaphragmBladeCountHandle->SetValue(P.LensSettings.DiaphragmBladeCount, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Result::Success);

				break;
			}
		}

		// if none of them found, do nothing
	}
}


FText FCameraLensSettingsCustomization::GetPresetComboBoxContent() const
{
	// just test one variable for multiple selection
	float MaxFocalLength;
	if (MinFocalLengthHandle->GetValue(MaxFocalLength) == FPropertyAccess::Result::MultipleValues)
	{
		// multiple selection
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*GetPresetString().Get());
}


TSharedPtr<FString> FCameraLensSettingsCustomization::GetPresetString() const
{
  	float MinFocalLength;
 	MinFocalLengthHandle->GetValue(MinFocalLength);
 
 	float MaxFocalLength;
 	MaxFocalLengthHandle->GetValue(MaxFocalLength);
 
 	float MinFStop;
 	MinFStopHandle->GetValue(MinFStop);
 
 	float MaxFStop;
 	MaxFStopHandle->GetValue(MaxFStop);

	float MinFocusDistance;
	MinFocusDistanceHandle->GetValue(MinFocusDistance);

	float SqueezeFactor;
	SqueezeFactorHandle->GetValue(SqueezeFactor);

	int32 DiaphragmBladeCount;
	DiaphragmBladeCountHandle->GetValue(DiaphragmBladeCount);

	// search presets for one that matches
	TArray<FNamedLensPreset> const& Presets = UCineCameraSettings::GetLensPresets();
	int32 const NumPresets = Presets.Num();
	for (int32 PresetIdx = 0; PresetIdx < NumPresets; ++PresetIdx)
	{
		FNamedLensPreset const& P = Presets[PresetIdx];
		if ((P.LensSettings.MinFocalLength == MinFocalLength) && (P.LensSettings.MaxFocalLength == MaxFocalLength) && (P.LensSettings.MinFStop == MinFStop) && 
			(P.LensSettings.MaxFStop == MaxFStop) && (P.LensSettings.MinimumFocusDistance == MinFocusDistance) && (P.LensSettings.SqueezeFactor == SqueezeFactor) && (P.LensSettings.DiaphragmBladeCount == DiaphragmBladeCount) )
		{
			// this is the one
			if (PresetComboList.IsValidIndex(PresetIdx + 1))
			{
				return PresetComboList[PresetIdx + 1];
			}
		}
	}

	return PresetComboList[0];
}

#undef LOCTEXT_NAMESPACE // CameraLensSettingsCustomization
