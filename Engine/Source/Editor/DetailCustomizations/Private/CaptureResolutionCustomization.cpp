// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureResolutionCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "MovieSceneCaptureSettings.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CaptureResolutionCustomization"

TSharedRef<IPropertyTypeCustomization> FCaptureResolutionCustomization::MakeInstance()
{
	return MakeShareable( new FCaptureResolutionCustomization );
}

void FCaptureResolutionCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils )
{
	PropertyHandle = InPropertyHandle;

	ResXHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCaptureResolution, ResX));
	ResYHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCaptureResolution, ResY));

	// Resolution
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionA", "320 x 240 (4:3)"), 320, 240} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionB", "640 x 480 (4:3)"), 640, 480} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionC", "640 x 360 (16:9)"), 640, 360} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionD", "1280 x 720 (16:9)"), 1280, 720} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionE", "1920 x 1080 (16:9)"), 1920, 1080} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionF", "3840 x 2160 (16:9)"), 3840, 2160} ));
	Resolutions.Add(MakeShareable( new FPredefinedResolution{LOCTEXT("ResolutionG", "Custom"), 1920, 1080} ));

	int32 CurrentResX = 0, CurrentResY = 0;
	ResXHandle->GetValue(CurrentResX);
	ResYHandle->GetValue(CurrentResY);

	CurrentIndex = Resolutions.IndexOfByPredicate([&](const TSharedPtr<FPredefinedResolution>& In){
		return In->ResX == CurrentResX && In->ResY == CurrentResY;
	});

	if (CurrentIndex == INDEX_NONE)
	{
		CurrentIndex = Resolutions.Num() - 1;
	}

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	.MaxDesiredWidth(TOptional<float>())
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoHeight()
		[
			SNew(SComboBox<TSharedPtr<FPredefinedResolution>>)
			.OptionsSource(&Resolutions)
			.OnSelectionChanged_Lambda([&](TSharedPtr<FPredefinedResolution> Resolution, ESelectInfo::Type){
				
				CurrentIndex = Resolutions.IndexOfByPredicate([&](const TSharedPtr<FPredefinedResolution>& In){
					return In == Resolution;
				});
				if (CurrentIndex == INDEX_NONE)
				{
					CurrentIndex = Resolutions.Num() - 1;
				}

				UpdateProperty();
			})
			.OnGenerateWidget_Lambda([&](TSharedPtr<FPredefinedResolution> Resolution){
				return SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(Resolution->DisplayName);
			})
			.InitiallySelectedItem(Resolutions[CurrentIndex])
			[
				SAssignNew(CurrentText, STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(Resolutions[CurrentIndex]->DisplayName)
			]
		]

		+ SVerticalBox::Slot()
		.Padding(FMargin(0,4.f))
		.AutoHeight()
		[
			SAssignNew(CustomSliders, SHorizontalBox)
			.Visibility(CurrentIndex == Resolutions.Num() - 1 ? EVisibility::Visible : EVisibility::Collapsed)

			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0,0,4.f,0))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("Width", "Width"))
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0,0,4.f,0))
				[
					ResXHandle->CreatePropertyValueWidget()
				]
			]

			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0,0,4.f,0))
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("Height", "Height"))
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(0,0,4.f,0))
				[
					ResYHandle->CreatePropertyValueWidget()
				]
			]
		]
	];
}

void FCaptureResolutionCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils )
{

}

void FCaptureResolutionCustomization::UpdateProperty()
{
	const TSharedPtr<FPredefinedResolution>& Resolution = Resolutions[CurrentIndex];

	if (CurrentIndex == Resolutions.Num() - 1)
	{
		// Show custom stuff
		CustomSliders->SetVisibility(EVisibility::Visible);
	}
	else
	{
		// Hide custom Stuff
		CustomSliders->SetVisibility(EVisibility::Collapsed);
		ResXHandle->SetValue(Resolution->ResX);
		ResYHandle->SetValue(Resolution->ResY);
	}

	CurrentText->SetText(Resolution->DisplayName);
}

#undef LOCTEXT_NAMESPACE
