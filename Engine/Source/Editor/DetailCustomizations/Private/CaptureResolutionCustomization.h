// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class STextBlock;
class SWidget;

class FCaptureResolutionCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;

private:

	/** Respond to a selection change event from the combo box */
	void UpdateProperty();
	
private:

	/** Array of predefined resolutions */
	struct FPredefinedResolution
	{
		FText DisplayName;
		uint32 ResX, ResY;
	};
	TArray<TSharedPtr<FPredefinedResolution>> Resolutions;

	/** The index of the resolution we're currently displaying */
	int32 CurrentIndex;
	/** The text of the current selection */
	TSharedPtr<STextBlock> CurrentText;
	/** The custom sliders to be hidden and shown based on combo box selection */
	TSharedPtr<SWidget> CustomSliders;

	/** Property handles of the properties we're editing */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> ResXHandle, ResYHandle;
};
