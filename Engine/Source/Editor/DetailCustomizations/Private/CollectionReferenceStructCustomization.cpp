// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionReferenceStructCustomization.h"

#include "CollectionManagerTypes.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "CollectionReferenceStructCustomization"

TSharedRef<IPropertyTypeCustomization> FCollectionReferenceStructCustomization::MakeInstance()
{
	return MakeShareable(new FCollectionReferenceStructCustomization());
}

void FCollectionReferenceStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> CollectionNameProperty = StructPropertyHandle->GetChildHandle("CollectionName");

	if(CollectionNameProperty.IsValid())
	{		
		HeaderRow.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(600.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				CollectionNameProperty->CreatePropertyValueWidget()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(PickerButton, SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("ComboToolTipText", "Choose a collection"))
				.OnClicked(FOnClicked::CreateSP(this, &FCollectionReferenceStructCustomization::OnPickContent, CollectionNameProperty.ToSharedRef()))
				.ContentPadding(2.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsFocusable(false)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}
}

void FCollectionReferenceStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
}

FReply FCollectionReferenceStructCustomization::OnPickContent(TSharedRef<IPropertyHandle> PropertyHandle) 
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FCollectionPickerConfig PickerConfig;
	PickerConfig.AllowCollectionButtons = false;
	PickerConfig.AllowRightClickMenu = false;
	PickerConfig.OnCollectionSelected = FOnCollectionSelected::CreateSP(this, &FCollectionReferenceStructCustomization::OnCollectionPicked, PropertyHandle);
	
	FMenuBuilder MenuBuilder(true, NULL);
	MenuBuilder.AddWidget(SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(300.0f)
		[
			ContentBrowserModule.Get().CreateCollectionPicker(PickerConfig)
		], FText());


	PickerMenu = FSlateApplication::Get().PushMenu(PickerButton.ToSharedRef(),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

	return FReply::Handled();
}

void FCollectionReferenceStructCustomization::OnCollectionPicked(const FCollectionNameType& CollectionType, TSharedRef<IPropertyHandle> PropertyHandle)
{
	if (PickerMenu.IsValid())
	{
		PickerMenu->Dismiss();
		PickerMenu.Reset();
	}

	PropertyHandle->SetValue(CollectionType.Name);
}

#undef LOCTEXT_NAMESPACE
