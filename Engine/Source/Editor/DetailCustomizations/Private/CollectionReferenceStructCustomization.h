// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class IMenu;
class IPropertyHandle;
class SButton;
struct FCollectionNameType;

class FCollectionReferenceStructCustomization: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

private:

	/** Delegate for displaying text value of path */
	FText GetDisplayedText(TSharedRef<IPropertyHandle> PropertyHandle) const;

	/** Delegate used to display a directory picker */
	FReply OnPickContent(TSharedRef<IPropertyHandle> PropertyHandle) ;

	/** Called when a path is picked from the path picker */
	void OnCollectionPicked(const FCollectionNameType& CollectionType, TSharedRef<IPropertyHandle> PropertyHandle);

	/** The pick button widget */
	TSharedPtr<SButton> PickerButton;

	/** The pick button popup menu*/
	TSharedPtr<IMenu> PickerMenu;
	};
