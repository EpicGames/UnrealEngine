// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionProfileDetails.h"

#include "BodyInstanceCustomization.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Serialization/Archive.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "CollsiionProfileDetails"

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnValidateChannel, const FCustomChannelSetup* )
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnValidateProfile, const FCollisionResponseTemplate*, int32)

constexpr ECollisionChannel MinCustomCollisionChannel = ECC_GameTraceChannel1;
constexpr ECollisionChannel MaxCustomCollisionChannel = ECC_GameTraceChannel50;
constexpr uint32 MaxNumCustomCollisionChannels = MaxCustomCollisionChannel - MinCustomCollisionChannel + 1;
constexpr uint32 MaxNumCollisionChannels = ECC_OverlapAll_Deprecated;
static_assert(MaxNumCollisionChannels == 64);

#define COLLIISION_COLUMN_WIDTH		50.f

#define PROFILE_WINDOW_WIDTH		300.f
#define PROFILE_WINDOW_HEIGHT		540.f

#define CHANNEL_WINDOW_WIDTH		200.f
#define CHANNEL_WINDOW_HEIGHT		93.f

#define RowWidth_Customization 50.f
//====================================================================================
// SChannelEditDialog 
//=====================================================================================

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

class SChannelEditDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChannelEditDialog)
		: _ChannelSetup(NULL)
		, _CollisionChannel(ECC_MAX)
	{}

		SLATE_ARGUMENT(FCustomChannelSetup*, ChannelSetup)
		SLATE_ARGUMENT(ECollisionChannel, CollisionChannel)
		SLATE_ARGUMENT(bool, bTraceType)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_EVENT(FOnValidateChannel, OnValidateChannel)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	// widget event handlers
	TSharedRef<SWidget> HandleResponseComboBoxGenerateWidget(TSharedPtr<FString> StringItem);
	void HandleResponseComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo);
	FText HandleResponseComboBoxContentText() const;

	FText GetName() const;
	void NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo);
	void OnTextChanged(const FText& NewText);

	// window handler
	FReply OnAccept();
	FReply OnCancel();
	bool	IsAcceptAvailable() const;
	void CloseWindow();

	// utility functions
	FCustomChannelSetup GetChannelSetup()
	{
		return ChannelSetup;
	}

	// data to return
	bool									bApplyChange;
	FCustomChannelSetup						ChannelSetup;

private:
	TWeakPtr<SWindow>						WidgetWindow;
	FOnValidateChannel						OnValidateChannel;
	TSharedPtr<SComboBox< TSharedPtr<FString> >>		ResponseComboBox;
	TArray< TSharedPtr<FString> >			ResponseComboBoxString;
	TSharedPtr<SEditableTextBox>		NameBox;
};

void SChannelEditDialog::Construct(const FArguments& InArgs)
{
	bApplyChange = false;

	if (InArgs._ChannelSetup)
	{
		ChannelSetup = *InArgs._ChannelSetup;
	}
	else
	{
		ChannelSetup.Channel = InArgs._CollisionChannel;
		ChannelSetup.bTraceType = InArgs._bTraceType;
	}

	check(ChannelSetup.Channel >= MinCustomCollisionChannel && ChannelSetup.Channel <= MaxCustomCollisionChannel);

	OnValidateChannel = InArgs._OnValidateChannel;
	WidgetWindow = InArgs._WidgetWindow;

	ResponseComboBoxString.Empty();
	ResponseComboBoxString.Add(MakeShareable(new FString(TEXT("Ignore"))));
	ResponseComboBoxString.Add(MakeShareable(new FString(TEXT("Overlap"))));
	ResponseComboBoxString.Add(MakeShareable(new FString(TEXT("Block"))));

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// channel name
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.VAlign(VAlign_Center)
		.Padding(3.f, 1.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SChannelEditDialog_Name", "Name"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SAssignNew(NameBox, SEditableTextBox)
				.MinDesiredWidth(64.0f)
				.Text(this, &SChannelEditDialog::GetName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.OnTextCommitted(this, &SChannelEditDialog::NewNameEntered)
				.OnTextChanged(this, &SChannelEditDialog::OnTextChanged)
			]
		]

		// default response
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(3.f, 1.f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SChannelEditDialog_DefaultResponse", "Default Response"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SAssignNew(ResponseComboBox, SComboBox< TSharedPtr<FString> >)
				.ContentPadding(FMargin(6.0f, 2.0f))
				.OptionsSource(&ResponseComboBoxString)
				.OnGenerateWidget(this, &SChannelEditDialog::HandleResponseComboBoxGenerateWidget)
				.OnSelectionChanged(this, &SChannelEditDialog::HandleResponseComboBoxSelectionChanged)
				[
					SNew(STextBlock)
					.Text(this, &SChannelEditDialog::HandleResponseComboBoxContentText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]

		// accept or cancel button
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SChannelEditDialog_Accept", "Accept"))
				.OnClicked(this, &SChannelEditDialog::OnAccept)
				.IsEnabled(this, &SChannelEditDialog::IsAcceptAvailable)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SChannelEditDialog_Cancel", "Cancel"))
				.OnClicked(this, &SChannelEditDialog::OnCancel)
			]
		]
	];
}

bool	SChannelEditDialog::IsAcceptAvailable() const
{
	return (ChannelSetup.Name != NAME_None && ChannelSetup.Name.ToString().Find(TEXT(" "))==INDEX_NONE);
}

FReply	SChannelEditDialog::OnAccept()
{
	if (OnValidateChannel.IsBound())
	{
		if (OnValidateChannel.Execute(&ChannelSetup))
		{
			bApplyChange = true;
			CloseWindow();
		}
		else
		{
			// invalid setup
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SChannelEditDialog_InvalidAccept", "Duplicate Name found."));
		}
	}
	else
	{
		// no validate test, just accept
		CloseWindow();
	}

	return FReply::Handled();
}

FReply SChannelEditDialog::OnCancel()
{
	CloseWindow();
	return FReply::Handled();
}

void SChannelEditDialog::CloseWindow()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
}

void SChannelEditDialog::OnTextChanged(const FText& NewText)
{
	FString NewName = NewText.ToString();

	if(NewName.Find(TEXT(" "))!=INDEX_NONE)
	{
		// no white space
		NameBox->SetError(LOCTEXT("ChannelNameValidationWhitespaceError", "No white space is allowed"));
	}
	else
	{
		NameBox->SetError(FText::GetEmpty());
		NewNameEntered(NewText, ETextCommit::Default);
	}
}

void SChannelEditDialog::NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo)
{
	{
		FName NewName = FName(*NewText.ToString());
		// we should accept NAME_None, that will invalidate "accpet" button
		if (NewName != ChannelSetup.Name)
		{
			ChannelSetup.Name = NewName;

			NameBox->SetError(FText::GetEmpty());
		}
	}
}

FText SChannelEditDialog::GetName() const
{
	if (ChannelSetup.Name == NAME_None)
	{
		return FText::GetEmpty();
	}

	return FText::FromName(ChannelSetup.Name);
}
TSharedRef<SWidget> SChannelEditDialog::HandleResponseComboBoxGenerateWidget(TSharedPtr<FString> StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*StringItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SChannelEditDialog::HandleResponseComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo)
{
	for (auto Iter = ResponseComboBoxString.CreateConstIterator(); Iter; ++Iter)
	{
		if (*Iter == StringItem)
		{
			ECollisionResponse NewResponse = (ECollisionResponse) Iter.GetIndex();
			check(NewResponse >= ECR_Ignore && NewResponse <= ECR_Block);
			ChannelSetup.DefaultResponse = NewResponse;
			return;
		}
	}

	// should not get here
	check(false);
}

FText SChannelEditDialog::HandleResponseComboBoxContentText() const
{
	int32 Index = (int32)ChannelSetup.DefaultResponse;
	if (ResponseComboBoxString.IsValidIndex(Index))
	{
		return FText::FromString(*ResponseComboBoxString[Index]);
	}

	return LOCTEXT("ChannelResponseTypeMessage", "Select Response");
}

//====================================================================================
// SProfileEditDialog 
//=====================================================================================

class SProfileEditDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SProfileEditDialog)
		: _ProfileTemplate(NULL)
		, _CollisionProfile(NULL)
		, _ProfileIndex(INDEX_NONE)
	{}

		SLATE_ARGUMENT(FCollisionResponseTemplate*, ProfileTemplate)
		SLATE_ARGUMENT(UCollisionProfile*, CollisionProfile)
		SLATE_ARGUMENT(int32, ProfileIndex)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(TArray<ECollisionChannel>, ObjectTypeMapping)
		SLATE_EVENT(FOnValidateProfile, OnValidateProfile)
		SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	// widget event handlers
	TSharedRef<SWidget> HandleCollisionEnabledComboBoxGenerateWidget(TSharedPtr<FString> StringItem);
	void HandleCollisionEnabledComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo);
	FText HandleCollisionEnabledComboBoxContentText() const;

	TSharedRef<SWidget> HandleObjectTypeComboBoxGenerateWidget(TSharedPtr<FString> StringItem);
	void HandleObjectTypeComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo);
	FText HandleObjectTypeComboBoxContentText() const;

	FText GetName() const;
	void NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo);
	void OnTextChanged(const FText& NewText);

	FText GetDescription() const;
	void NewDescriptionEntered(const FText& NewText, ETextCommit::Type CommitInfo);

	// window handler
	FReply OnAccept();
	FReply OnCancel();
	bool	IsAcceptAvailable() const;
	void CloseWindow();

	bool	CanModify()
	{
		return ProfileTemplate.bCanModify;
	}

	// data to return
	bool									bApplyChange;
	FCollisionResponseTemplate				ProfileTemplate;
	int32									ProfileIndex;

private:
	TWeakPtr<SWindow>						WidgetWindow;
	FOnValidateProfile						OnValidateProfile;

	TSharedPtr<SComboBox< TSharedPtr<FString> >>		CollisionEnabledComboBox;
	TArray< TSharedPtr<FString> >						CollisionEnabledComboBoxString;

	TSharedPtr<SComboBox< TSharedPtr<FString> >>		ObjectTypeComboBox;
	TArray< TSharedPtr<FString> >						ObjectTypeComboBoxString;

	TSharedPtr<SVerticalBox>							SCollisionPanel;

	UCollisionProfile*									CollisionProfile;
	TArray<ECollisionChannel>							ObjectTypeMapping;

	TSharedPtr<SEditableTextBox>						NameBox;	

	void FillCollisionEnabledString();
	void FillObjectTypeString();

	void AddCollisionResponse();
	void AddCollisionChannel(TArray<FCollisionChannelInfo>	ValidCollisionChannels, bool bTraceType);

	// collision channel check boxes
	void OnCollisionChannelChanged(ECheckBoxState InNewValue, int32 ValidIndex, ECollisionResponse InCollisionResponse);
	ECheckBoxState IsCollisionChannelChecked(int32 ValidIndex, ECollisionResponse InCollisionResponse) const;
	// all collision channel check boxes
	void OnAllCollisionChannelChanged(ECheckBoxState InNewValue, ECollisionResponse InCollisionResponse);
	ECheckBoxState IsAllCollisionChannelChecked(ECollisionResponse InCollisionResponse) const;
};

void SProfileEditDialog::Construct(const FArguments& InArgs)
{
	bApplyChange = false;

	check(InArgs._CollisionProfile);
	
	if(InArgs._ProfileTemplate)
	{
		ProfileTemplate = *InArgs._ProfileTemplate;
	}

	CollisionProfile = InArgs._CollisionProfile;
	ProfileIndex = InArgs._ProfileIndex;

	OnValidateProfile = InArgs._OnValidateProfile;
	WidgetWindow = InArgs._WidgetWindow;

	ObjectTypeMapping = InArgs._ObjectTypeMapping;

	FillObjectTypeString();
	FillCollisionEnabledString();

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// Profile name
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SProfileEditDialog_Name", "Name"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SAssignNew(NameBox, SEditableTextBox)
				.MinDesiredWidth(64.0f)
				.Text(this, &SProfileEditDialog::GetName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(SProfileEditDialog::CanModify())
				.OnTextCommitted(this, &SProfileEditDialog::NewNameEntered)
				.OnTextChanged(this, &SProfileEditDialog::OnTextChanged)
			]
		]

		// default CollisionEnabled
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SProfileEditDialog_CollisionEnabled", "CollisionEnabled"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SAssignNew(CollisionEnabledComboBox, SComboBox< TSharedPtr<FString> >)
				.ContentPadding(FMargin(6.0f, 2.0f))
				.OptionsSource(&CollisionEnabledComboBoxString)
				.OnGenerateWidget(this, &SProfileEditDialog::HandleCollisionEnabledComboBoxGenerateWidget)
				.OnSelectionChanged(this, &SProfileEditDialog::HandleCollisionEnabledComboBoxSelectionChanged)
				[
					SNew(STextBlock)
					.Text(this, &SProfileEditDialog::HandleCollisionEnabledComboBoxContentText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]

		// default ObjectType
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SProfileEditDialog_ObjectType", "ObjectType"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SAssignNew(ObjectTypeComboBox, SComboBox< TSharedPtr<FString> >)
				.ContentPadding(FMargin(6.0f, 2.0f))
				.OptionsSource(&ObjectTypeComboBoxString)
				.OnGenerateWidget(this, &SProfileEditDialog::HandleObjectTypeComboBoxGenerateWidget)
				.OnSelectionChanged(this, &SProfileEditDialog::HandleObjectTypeComboBoxSelectionChanged)
				[
					SNew(STextBlock)
					.Text(this, &SProfileEditDialog::HandleObjectTypeComboBoxContentText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]

		// Profile Description
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(100.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SProfileEditDialog_Description", "Description"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Left)
			[
				SNew(SEditableTextBox)
				.MinDesiredWidth(128.0f)
				.Text(this, &SProfileEditDialog::GetDescription)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.OnTextCommitted(this, &SProfileEditDialog::NewDescriptionEntered)
			]
		]

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SAssignNew(SCollisionPanel, SVerticalBox)
			]
		]

		// accept or cancel button
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1.f,3.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SProfileEditDialog_Accept", "Accept"))
				.OnClicked(this, &SProfileEditDialog::OnAccept)
				.IsEnabled(this, &SProfileEditDialog::IsAcceptAvailable)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SProfileEditDialog_Cancel", "Cancel"))
				.OnClicked(this, &SProfileEditDialog::OnCancel)
			]
		]
	];

	AddCollisionResponse();
}

void SProfileEditDialog::FillObjectTypeString()
{
	ObjectTypeComboBoxString.Empty();
	for(auto Iter = ObjectTypeMapping.CreateConstIterator(); Iter; ++Iter)
	{
		ECollisionChannel Channel = *Iter;
		FName ChannelName = CollisionProfile->ReturnChannelNameFromContainerIndex((int32)Channel);

		ObjectTypeComboBoxString.Add(MakeShareable(new FString(ChannelName.ToString())));
	}
	
}

void SProfileEditDialog::FillCollisionEnabledString()
{
	CollisionEnabledComboBoxString.Empty();
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("No Collision"))));
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("Query Only (No Physics Collision)"))));
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("Physics Only (No Query Collision)"))));
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("Collision Enabled (Query and Physics)"))));
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("Probe Only (No Query or Physics Collision)"))));
	CollisionEnabledComboBoxString.Add(MakeShareable(new FString(TEXT("Query and Probe (No Physics Collision)"))));
}

bool	SProfileEditDialog::IsAcceptAvailable() const
{
	return (ProfileTemplate.Name != NAME_None && ProfileTemplate.Name.ToString().Find(TEXT(" "))==INDEX_NONE);
}

FReply	SProfileEditDialog::OnAccept()
{
	if(OnValidateProfile.IsBound())
	{
		if(OnValidateProfile.Execute(&ProfileTemplate, ProfileIndex))
		{
			bApplyChange = true;
			CloseWindow();
		}
		else
		{
			// invalid setup
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SProfileEditDialog_InvalidAccept", "Duplicate Name found."));
		}
	}
	else
	{
		// no validate test, just accept
		CloseWindow();
	}

	return FReply::Handled();
}

FReply SProfileEditDialog::OnCancel()
{
	CloseWindow();
	return FReply::Handled();
}

void SProfileEditDialog::CloseWindow()
{
	if(WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
}

void SProfileEditDialog::OnTextChanged(const FText& NewText)
{
	FString NewName = NewText.ToString();

	if(NewName.Find(TEXT(" "))!=INDEX_NONE)
	{
		// no white space
		NameBox->SetError(TEXT("No white space is allowed"));
	}
	else
	{
		NameBox->SetError(TEXT(""));
		NewNameEntered(NewText, ETextCommit::Default);
	}
}

void SProfileEditDialog::NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// Don't digest the number if we just clicked away from the pop-up
	{
		FName NewName = FName(*NewText.ToString());

		// we should accept NAME_None, that will invalidate "accpet" button
		if(NewName != ProfileTemplate.Name)
		{
			ProfileTemplate.Name = NewName;

			NameBox->SetError(TEXT(""));
		}
	}
}

FText SProfileEditDialog::GetName() const
{
	if(ProfileTemplate.Name == NAME_None)
	{
		return FText::FromString(TEXT(""));
	}

	return FText::FromName(ProfileTemplate.Name);
}

void SProfileEditDialog::NewDescriptionEntered(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// Don't digest the number if we just clicked away from the pop-up
	if((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus))
	{
		ProfileTemplate.HelpMessage= *NewText.ToString();
	}
}

FText SProfileEditDialog::GetDescription() const
{
	return FText::FromString(ProfileTemplate.HelpMessage);
}

TSharedRef<SWidget> SProfileEditDialog::HandleCollisionEnabledComboBoxGenerateWidget(TSharedPtr<FString> StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*StringItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SProfileEditDialog::HandleCollisionEnabledComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo)
{
	for(auto Iter = CollisionEnabledComboBoxString.CreateConstIterator(); Iter; ++Iter)
	{
		if(*Iter == StringItem)
		{
			ECollisionEnabled::Type NewCollisionEnabled = (ECollisionEnabled::Type)Iter.GetIndex();
			check(NewCollisionEnabled >= ECollisionEnabled::NoCollision && NewCollisionEnabled <= ECollisionEnabled::QueryAndProbe);
			ProfileTemplate.CollisionEnabled = NewCollisionEnabled;
			return;
		}
	}

	// should not get here
	check(false);
}

FText SProfileEditDialog::HandleCollisionEnabledComboBoxContentText() const
{
	int32 Index = (int32)ProfileTemplate.CollisionEnabled;
	if(CollisionEnabledComboBoxString.IsValidIndex(Index))
	{
		return FText::FromString(*CollisionEnabledComboBoxString[Index]);
	}

	return LOCTEXT("ProfileCollisionEnabledMessage", "Select CollisionEnabled");
}


TSharedRef<SWidget> SProfileEditDialog::HandleObjectTypeComboBoxGenerateWidget(TSharedPtr<FString> StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*StringItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SProfileEditDialog::HandleObjectTypeComboBoxSelectionChanged(TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo)
{
	for(auto Iter = ObjectTypeComboBoxString.CreateConstIterator(); Iter; ++Iter)
	{
		if(*Iter == StringItem)
		{
			int32 Index = Iter.GetIndex();
			if(ObjectTypeMapping.IsValidIndex(Index))
			{
				FName ObjectTypeName = CollisionProfile->ReturnChannelNameFromContainerIndex(ObjectTypeMapping[Index]);
				ProfileTemplate.ObjectTypeName = ObjectTypeName;
				ProfileTemplate.ObjectType = ObjectTypeMapping[Index];
			}
			else
			{
				// error, warn user?
			}
			return;
		}
	}

	// should not get here
	check(false);
}

FText SProfileEditDialog::HandleObjectTypeComboBoxContentText() const
{
	for(auto Iter = ObjectTypeMapping.CreateConstIterator(); Iter; ++Iter)
	{
		if(*Iter == ProfileTemplate.ObjectType)
		{
			int32 Index = Iter.GetIndex();
			if(ObjectTypeComboBoxString.IsValidIndex(Index))
			{
				return FText::FromString(*ObjectTypeComboBoxString[Index]);
			}
		}
	}

	return LOCTEXT("ProfileObjectTypeMessage", "Select ObjectType");
}

void SProfileEditDialog::AddCollisionResponse()
{
	check(SCollisionPanel.IsValid());

	// find the enum
	TArray<FCollisionChannelInfo>	ValidCollisionChannels;
	UEnum * Enum = StaticEnum<ECollisionChannel>();
	// we need this Enum
	check(Enum);
	const FString KeyName = TEXT("DisplayName");
	const FString TraceType = TEXT("TraceQuery");

	// need to initialize displaynames separate
	int32 NumEnum = Enum->NumEnums();
	ValidCollisionChannels.Empty(NumEnum);

	// first go through enum entry, and add suffix to displaynames
	for(int32 EnumIndex=0; EnumIndex<NumEnum; ++EnumIndex)
	{
		const FString& EnumMetaData = Enum->GetMetaData(*KeyName, EnumIndex);
		if(EnumMetaData.Len() > 0)
		{
			FCollisionChannelInfo Info;
			Info.DisplayName = EnumMetaData;
			Info.CollisionChannel = (ECollisionChannel)EnumIndex;
			if(Enum->GetMetaData(*TraceType, EnumIndex) == TEXT("1"))
			{
				Info.TraceType = true;
			}
			else
			{
				Info.TraceType = false;
			}

			ValidCollisionChannels.Add(Info);
		}
	}

	SCollisionPanel->AddSlot()
	.AutoHeight()
	[
		SNew(SSeparator)
		.Orientation(Orient_Horizontal)
	];

	// Add All check box
	SCollisionPanel->AddSlot()
	.Padding(3.f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1)
			[
				SNew( STextBlock )
				.Text(LOCTEXT("SProfileEditDialog_CR_Label", "Collision Responses"))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
				.ToolTipText(LOCTEXT("SProfileEditDialog_CR_LabelToolTip", "When trace by channel, this information will be used for filtering."))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				IDocumentation::Get()->CreateAnchor( TEXT("Engine/Physics/Collision") )
			]
		]

		+SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(RowWidth_Customization)
				.HAlign( HAlign_Left )
				.Content()
				[
					SNew (STextBlock)
					.Text(LOCTEXT("IgnoreCollisionLabel", "Ignore"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign( HAlign_Left )
				.WidthOverride(RowWidth_Customization)
				.Content()
				[
					SNew (STextBlock)
					.Text(LOCTEXT("OverlapCollisionLabel", "Overlap"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(RowWidth_Customization)
				.HAlign( HAlign_Left )
				.Content()
				[
					SNew (STextBlock)
					.Text(LOCTEXT("BlockCollisionLabel", "Block"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]
			]
		]
	];

	SCollisionPanel->AddSlot()
	.Padding(3.f)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]

		+SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(COLLIISION_COLUMN_WIDTH)
				.Content()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &SProfileEditDialog::OnAllCollisionChannelChanged, ECR_Ignore )
					.IsChecked( this, &SProfileEditDialog::IsAllCollisionChannelChecked, ECR_Ignore )
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(COLLIISION_COLUMN_WIDTH)
				.Content()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &SProfileEditDialog::OnAllCollisionChannelChanged, ECR_Overlap )
					.IsChecked( this, &SProfileEditDialog::IsAllCollisionChannelChecked, ECR_Overlap )
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(COLLIISION_COLUMN_WIDTH)
				.Content()
				[
					SNew(SCheckBox)
					.OnCheckStateChanged( this, &SProfileEditDialog::OnAllCollisionChannelChanged, ECR_Block )
					.IsChecked( this, &SProfileEditDialog::IsAllCollisionChannelChecked, ECR_Block )
				]
			]
		]
	];

	SCollisionPanel->AddSlot()
	.AutoHeight()
	[
		SNew(SSeparator)
		.Orientation(Orient_Horizontal)
	];

	AddCollisionChannel(ValidCollisionChannels, true);
	AddCollisionChannel(ValidCollisionChannels, false);
}

void SProfileEditDialog::AddCollisionChannel(TArray<FCollisionChannelInfo>	ValidCollisionChannels, bool bTraceType)
{

	FText TitleText, TitleToolTip;
	if(bTraceType)
	{
		TitleText = LOCTEXT("SProfileEditDialog_CR_TraceType", "Trace Type");
		TitleToolTip = LOCTEXT("SProfileEditDialog_CR_TraceTypeTooltip", "Trace Type Channels");
	}
	else
	{
		TitleText = LOCTEXT("SProfileEditDialog_CR_ObjectType", "Object Type");
		TitleToolTip = LOCTEXT("SProfileEditDialog_CR_ObjectTypeTooltip", "Object Type Channels");
	}

	SCollisionPanel->AddSlot()
	.Padding(3.f)
	[
		SNew(STextBlock)
		.Text(TitleText)
		.Font(IDetailLayoutBuilder::GetDetailFontBold())
		.ToolTipText(TitleToolTip)
	];
		// Add Title 
	for ( int32 Index=0; Index<ValidCollisionChannels.Num(); ++Index )
	{
		if(ValidCollisionChannels[Index].TraceType == bTraceType)
		{
			const FString& DisplayName = ValidCollisionChannels[Index].DisplayName;
			int32 ChannelIndex = (int32)ValidCollisionChannels[Index].CollisionChannel;

			SCollisionPanel->AddSlot()
			.Padding(6.f, 3.f)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				[
					SNew( STextBlock )
					.Text(FText::FromString(DisplayName))
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(LOCTEXT("SProfileEditDialog_CR_ToolTip", "When trace by channel, this information will be used for filtering."))
				]
				+SHorizontalBox::Slot()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(COLLIISION_COLUMN_WIDTH)
						.Content()
						[
							SNew(SCheckBox)
							.OnCheckStateChanged(this, &SProfileEditDialog::OnCollisionChannelChanged, ChannelIndex, ECR_Ignore)
							.IsChecked(this, &SProfileEditDialog::IsCollisionChannelChecked, ChannelIndex, ECR_Ignore)
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(COLLIISION_COLUMN_WIDTH)
						.Content()
						[
							SNew(SCheckBox)
							.OnCheckStateChanged(this, &SProfileEditDialog::OnCollisionChannelChanged, ChannelIndex, ECR_Overlap)
							.IsChecked(this, &SProfileEditDialog::IsCollisionChannelChecked, ChannelIndex, ECR_Overlap)
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(COLLIISION_COLUMN_WIDTH)
						.Content()
						[
							SNew(SCheckBox)
							.OnCheckStateChanged(this, &SProfileEditDialog::OnCollisionChannelChanged, ChannelIndex, ECR_Block)
							.IsChecked(this, &SProfileEditDialog::IsCollisionChannelChecked, ChannelIndex, ECR_Block)
						]
					]
				]
			];
		}
	}
}

void SProfileEditDialog::OnCollisionChannelChanged(ECheckBoxState InNewValue, int32 ValidIndex, ECollisionResponse InCollisionResponse)
{
	if(ValidIndex >= 0 && ValidIndex < MaxNumCollisionChannels)
	{
		ProfileTemplate.ResponseToChannels.EnumArray[ValidIndex] = static_cast<uint8>(InCollisionResponse);
	}
}

ECheckBoxState SProfileEditDialog::IsCollisionChannelChecked(int32 ValidIndex, ECollisionResponse InCollisionResponse) const
{
	TArray<uint8> CollisionResponses;

	if( ValidIndex >= 0 && ValidIndex < MaxNumCollisionChannels )
	{
		if(ProfileTemplate.ResponseToChannels.EnumArray[ValidIndex] == static_cast<uint8>(InCollisionResponse))
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

void SProfileEditDialog::OnAllCollisionChannelChanged(ECheckBoxState InNewValue, ECollisionResponse InCollisionResponse)
{
	for(int32 Index=0; Index<MaxNumCollisionChannels; ++Index)
	{
		ProfileTemplate.ResponseToChannels.EnumArray[Index] = static_cast<uint8>(InCollisionResponse);
	}
}

ECheckBoxState SProfileEditDialog::IsAllCollisionChannelChecked(ECollisionResponse InCollisionResponse) const
{
	for(int32 Index=0; Index<MaxNumCollisionChannels; ++Index)
	{
		if(ProfileTemplate.ResponseToChannels.EnumArray[Index] != static_cast<uint8>(InCollisionResponse))
		{
			return ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Checked;
}
//====================================================================================
// SChannelListItem 
//=====================================================================================

void SChannelListItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	ChannelSetup = InArgs._ChannelSetup;
	check(ChannelSetup.IsValid());
	SMultiColumnTableRow< TSharedPtr<FChannelListItem> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

FText SChannelListItem::GetDefaultResponse() const
{
	switch (ChannelSetup->DefaultResponse)
	{
	case ECR_Ignore:
		return LOCTEXT("ECR_Ignore", "Ignore");
	case ECR_Overlap:
		return LOCTEXT("ECR_Overlap", "Overlap");
	case ECR_Block:
		return LOCTEXT("ECR_Block", "Block");
	}

	return LOCTEXT("ECR_Error", "ERROR");
}

TSharedRef<SWidget> SChannelListItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == TEXT("Name"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.Padding(FMargin(3.f, 0.f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromName(ChannelSetup->Name))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
	else if (ColumnName == TEXT("DefaultResponse"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.Padding(FMargin(3.f, 0.f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(SChannelListItem::GetDefaultResponse())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	return SNullWidget::NullWidget;
}


//====================================================================================
// SProfileListItem 
//=====================================================================================

void SProfileListItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	ProfileTemplate = InArgs._ProfileTemplate;
	check(ProfileTemplate.IsValid());
	SMultiColumnTableRow< TSharedPtr<FProfileListItem> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

FText SProfileListItem::GetObjectType() const
{
	return FText::FromName(ProfileTemplate->ObjectTypeName);
}

FText SProfileListItem::GetCollsionEnabled() const
{
	switch(ProfileTemplate->CollisionEnabled)
	{
	case ECollisionEnabled::NoCollision:
		return LOCTEXT("ECollisionEnabled_NoCollision", "No Collision");
	case ECollisionEnabled::QueryOnly:
		return LOCTEXT("ECollisionEnabled_QueryOnly", "Query Only (No Physics Collision)");
	case ECollisionEnabled::PhysicsOnly:
		return LOCTEXT("ECollisionEnabled_PhysicsOnly", "Physics Only (No Query Collision)");
	case ECollisionEnabled::QueryAndPhysics:
		return LOCTEXT("ECollisionEnabled_QueryAndPhysics", "Collision Enabled (Query and Physics)");
	case ECollisionEnabled::ProbeOnly:
		return LOCTEXT("ECollisionEnabled_Probe", "Probe Only (No Query or Physics Collision)");
	case ECollisionEnabled::QueryAndProbe:
		return LOCTEXT("ECollisionEnabled_QueryAndProbe", "Query and Probe (No Physics Collision)");
	}

	return LOCTEXT("ECollisionEnabled_Error", "ERROR");
}

TSharedRef<SWidget> SProfileListItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if(ColumnName == TEXT("Engine"))
	{
		if(ProfileTemplate->bCanModify == false)
		{
			return	SNew(SBox)
				.HeightOverride(20.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("SettingsEditor.Collision_Engine"))
					.ToolTipText(LOCTEXT("CantModify_Tooltip", "You can't modify the name of Engine profiles"))
				];
		}
		else
		{
			return	SNew(SBox)
				.HeightOverride(20.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("SettingsEditor.Collision_Game"))
					.ToolTipText(LOCTEXT("CanModify_Tooltip", "This is your custom project profile"))
				];
		}
	}
	else if(ColumnName == TEXT("Name"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromName(ProfileTemplate->Name))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
	else if(ColumnName == TEXT("Collision"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(SProfileListItem::GetCollsionEnabled())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
	else if(ColumnName == TEXT("ObjectType"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(SProfileListItem::GetObjectType())
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
	else if(ColumnName == TEXT("Description"))
	{
		return	SNew(SBox)
			.HeightOverride(20.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ProfileTemplate->HelpMessage))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	return SNullWidget::NullWidget;
}
//====================================================================================
// FCollisionProfileDetails
//=====================================================================================
TSharedRef<IDetailCustomization> FCollisionProfileDetails::MakeInstance()
{
	return MakeShareable( new FCollisionProfileDetails );
}

void FCollisionProfileDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& ObjectChannelCategory = DetailBuilder.EditCategory("Object Channels");
	IDetailCategoryBuilder& TraceChannelCategory = DetailBuilder.EditCategory("Trace Channels");
  	IDetailCategoryBuilder& PresetCategory = DetailBuilder.EditCategory("Preset");

	CollisionProfile= UCollisionProfile::Get();
	check(CollisionProfile);

	// save currently loaded data
	SavedData.Save(CollisionProfile);

	RefreshChannelList(true);
	RefreshChannelList(false);
	RefreshProfileList();

	PresetCategory.InitiallyCollapsed(true);
	PresetCategory.RestoreExpansionState(false);

	const FString ObjectChannelDocLink = TEXT("Shared/Collision");
	const FString TraceChannelDocLink = TEXT("Shared/Collision");
	const FString PresetsDocLink = TEXT("Shared/Collision");

	TSharedPtr<SToolTip> ObjectChannelTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCollisionObject", "Edit collision object types."), NULL, ObjectChannelDocLink, TEXT("ObjectChannel"));
	TSharedPtr<SToolTip> TraceChannelTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCollisionChannel", "Edit collision trace channels."), NULL, TraceChannelDocLink, TEXT("TraceChannel"));
	TSharedPtr<SToolTip> ProfileTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCollisionPreset", "Edit collision presets."), NULL, PresetsDocLink, TEXT("Preset"));

	// Customize collision section
	ObjectChannelCategory.AddCustomRow(LOCTEXT("CustomCollisionObjectChannels", "ObjectChannels"))
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(5.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTip(ObjectChannelTooltip)
				.Text(LOCTEXT("ObjectChannel_Menu_Description", "You can have up to 50 custom channels including object and trace channels. This is the list of object types for your project. If you delete an object type that is being used by the game, any uses of that type will revert to WorldStatic."))
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_NewObject", "New Object Channel..."))
				.OnClicked(this, &FCollisionProfileDetails::OnNewChannel, false)
				.IsEnabled(this, &FCollisionProfileDetails::IsNewChannelAvailable)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_Edit", "Edit..."))
				.OnClicked(this, &FCollisionProfileDetails::OnEditChannel, false)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyChannelSelected, false)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_Delete", "Delete..."))
				.OnClicked(this, &FCollisionProfileDetails::OnDeleteChannel, false)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyChannelSelected, false)
			]
		]

		+SVerticalBox::Slot()
		.Padding(5.f)
		.FillHeight(1.f)
		[
			SNew(SBox)
			.MaxDesiredHeight(200.f)
			[
				SAssignNew(ObjectChannelListView, SChannelListView)
				.ListItemsSource(&ObjectChannelList)
				.OnGenerateRow(this, &FCollisionProfileDetails::HandleGenerateChannelWidget)
				.OnMouseButtonDoubleClick(this, &FCollisionProfileDetails::OnObjectChannelListItemDoubleClicked)
				.SelectionMode(ESelectionMode::Single)

				.HeaderRow(
					SNew(SHeaderRow)
					// Name
					+ SHeaderRow::Column("Name")
					.HAlignCell(HAlign_Left)
					.FillWidth(1.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ChannelListHeader_Name", "Name"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// Default Response
					+ SHeaderRow::Column("DefaultResponse")
					.HAlignCell(HAlign_Left)
					.FillWidth(1.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ChannelListHeader_DefaultResponse", "Default Response"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				)
			]
		]
	];
	
	TraceChannelCategory.AddCustomRow(LOCTEXT("CustomCollisionTraceChannels", "TraceChannels"))
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AutoWrapText(true)
				.ToolTip(TraceChannelTooltip)
				.Text(LOCTEXT("TraceChannel_Menu_Description", "You can have up to 50 custom channels including object and trace channels. This is the list of trace channels for your project. If you delete a trace channel that is being used by the game, the behavior of the trace is undefined."))
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_NewTrace", "New Trace Channel..."))
				.OnClicked(this, &FCollisionProfileDetails::OnNewChannel, true)
				.IsEnabled(this, &FCollisionProfileDetails::IsNewChannelAvailable)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_Edit", "Edit..."))
				.OnClicked(this, &FCollisionProfileDetails::OnEditChannel, true)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyChannelSelected, true)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 10.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ChannelMenu_Delete", "Delete..."))
				.OnClicked(this, &FCollisionProfileDetails::OnDeleteChannel, true)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyChannelSelected, true)
			]
		]

		+SVerticalBox::Slot()
		.Padding(5.f)
		.FillHeight(1.f)
		[
			SNew(SBox)
			.MaxDesiredHeight(400.f)
			[
				SAssignNew(TraceChannelListView, SChannelListView)
				.ListItemsSource(&TraceChannelList)
				.OnGenerateRow(this, &FCollisionProfileDetails::HandleGenerateChannelWidget)
				.OnMouseButtonDoubleClick(this, &FCollisionProfileDetails::OnTraceChannelListItemDoubleClicked)
				.SelectionMode(ESelectionMode::Single)

				.HeaderRow(
					SNew(SHeaderRow)
					// Name
					+ SHeaderRow::Column("Name")
					.HAlignCell(HAlign_Left)
					.FillWidth(1.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ChannelListHeader_Name", "Name"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// Default Response
					+ SHeaderRow::Column("DefaultResponse")
					.HAlignCell(HAlign_Left)
					.FillWidth(1)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ChannelListHeader_DefaultResponse", "Default Response"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				)
			]
		]
	];

	PresetCategory.AddCustomRow(LOCTEXT("CustomCollisionProfiles", "Presets"))
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(5)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(2.f, 2.f)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTip(ProfileTooltip)
				.Text(LOCTEXT("Profile_Menu_Description", "You can modify any of your project profiles. Please note that if you modify profile, it can change collision behavior. Please be careful when you change currently exisiting (used) collision profiles."))
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 2.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ProfileMenu_New", "New..."))
				.OnClicked(this, &FCollisionProfileDetails::OnNewProfile)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 2.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ProfileMenu_Edit", "Edit..."))
				.OnClicked(this, &FCollisionProfileDetails::OnEditProfile)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyProfileSelected)
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, 2.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("ProfileMenu_Delete", "Delete..."))
				.OnClicked(this, &FCollisionProfileDetails::OnDeleteProfile)
				.IsEnabled(this, &FCollisionProfileDetails::IsAnyProfileSelected)
			]
		]

		+SVerticalBox::Slot()
		.Padding(5.f)
		.FillHeight(1.f)
		[
			SNew(SBox)
			.MaxDesiredHeight(600.f)
			[
				SAssignNew(ProfileListView, SProfileListView)
				.ListItemsSource(&ProfileList)
				.OnGenerateRow(this, &FCollisionProfileDetails::HandleGenerateProfileWidget)
				.OnMouseButtonDoubleClick(this, &FCollisionProfileDetails::OnProfileListItemDoubleClicked)
				.SelectionMode(ESelectionMode::Single)

				.HeaderRow(
					SNew(SHeaderRow)
					// Name
					+ SHeaderRow::Column("Engine")
					.HAlignCell(HAlign_Left)
					.FixedWidth(30.f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FText::GetEmpty())
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
					// Name
					+ SHeaderRow::Column("Name")
					.HAlignCell(HAlign_Left)
					.FillWidth(1.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ProfileListHeader_Name", "Name"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// Default Response
					+ SHeaderRow::Column("Collision")
					.HAlignCell(HAlign_Left)
					.FillWidth(1)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ProfileListHeader_Collision", "Collision"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// Trace Type
					+ SHeaderRow::Column("ObjectType")
					.HAlignCell(HAlign_Left)
					.FillWidth(1.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ProfileListHeader_ObjectType", "Object Type"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// Static Object
					+ SHeaderRow::Column("Description")
					.HAlignCell(HAlign_Left)
					.FillWidth(2.f)
					.HeaderContentPadding(FMargin(0.f, 3.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ProfileListHeader_Description", "Description"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				)
			]
		]
	];
}

void FCollisionProfileDetails::CommitProfileChange(int32 ProfileIndex, FCollisionResponseTemplate & NewProfile)
{
	FCollisionResponseTemplate & SourceProfile = CollisionProfile->Profiles[ProfileIndex];

	// if name changed, we need to add redirect
	if(SourceProfile.Name != NewProfile.Name)
	{
		CollisionProfile->AddProfileRedirect(SourceProfile.Name, NewProfile.Name);
	}

	if(SourceProfile.bCanModify)
	{
		// if you can modify, overwrites everything
		CollisionProfile->SaveCustomResponses(NewProfile);
		SourceProfile = (NewProfile);
	}
	else
	{
		// copy everything else but not the response
		// we add that to EditProfile
		SourceProfile.CollisionEnabled = NewProfile.CollisionEnabled;
		SourceProfile.ObjectTypeName = NewProfile.ObjectTypeName;
		SourceProfile.HelpMessage = NewProfile.HelpMessage;

		// now update EditProfiles
		// look at the saved profile, and collect different responses first
		FCollisionResponseTemplate & SavedProfile = SavedData.Profiles[ProfileIndex];
		TArray<FResponseChannel>	NewCustomResponses;

		struct FFindByName
		{
			FName Name;

			FFindByName(FName InName): Name(InName) { }

			bool operator() (const FResponseChannel& Element) const
			{
				return (Name == Element.Channel);
			}

			bool operator() (const FCustomProfile& Element) const
			{
				return (Name == Element.Name);
			}
			
		};
		
		for (int32 Index = 0; Index<MaxNumCollisionChannels; ++Index)
		{
			if (NewProfile.ResponseToChannels.EnumArray[Index] != SavedProfile.ResponseToChannels.EnumArray[Index])
			{
				FName ChannelName = CollisionProfile->ChannelDisplayNames[Index];
				NewCustomResponses.Add( FResponseChannel(ChannelName, (ECollisionResponse) NewProfile.ResponseToChannels.EnumArray[Index]) );
			}
		}

		// we have new list, merge with existing ones
		if ( NewCustomResponses.Num() > 0 )
		{
			FCustomProfile * CurrentProfile = CollisionProfile->EditProfiles.FindByPredicate(FFindByName(NewProfile.Name));
			if ( !CurrentProfile )
			{
				// need to add new one, and just copy NewCustomResponses
				FCustomProfile NewCustomProfile;
				NewCustomProfile.Name = NewProfile.Name;
				NewCustomProfile.CustomResponses = NewCustomResponses;
				CollisionProfile->EditProfiles.Add(NewCustomProfile);
			}
			else
			{
				// need to merge previous list and new list
				for (auto & Iter : NewCustomResponses)
				{
					FResponseChannel * CurrentChannel = CurrentProfile->CustomResponses.FindByPredicate(FFindByName(Iter.Channel));

					if (CurrentChannel)
					{
						if (CurrentChannel->Response != Iter.Response)
						{
							CurrentChannel->Response = Iter.Response;
						}
					}
					else 
					{
						// just add new one
						CurrentProfile->CustomResponses.Add(Iter);
					}
				}
			}
		}	
	}

}

void FCollisionProfileDetails::UpdateChannel(bool bTraceType)
{
	RefreshChannelList(bTraceType);
	if(bTraceType)
	{
		TraceChannelListView->RequestListRefresh();
	}
	else
	{
		ObjectChannelListView->RequestListRefresh();
	}

	UpdateProfile();
}

void FCollisionProfileDetails::UpdateProfile()
{
	CollisionProfile->LoadProfileConfig(true);
	const FString SpecificFileLocation;
	const bool bWarnIfFail = false;
	CollisionProfile->TryUpdateDefaultConfigFile(SpecificFileLocation, bWarnIfFail);
	SavedData.Save(CollisionProfile);

	RefreshProfileList();
	ProfileListView->RequestListRefresh();
}

void FCollisionProfileDetails::RefreshChannelList(bool bTraceType)
{
	if(bTraceType)
	{
		TraceChannelList.Empty();

		for(auto Iter = CollisionProfile->DefaultChannelResponses.CreateIterator(); Iter; ++Iter)
		{
			// only display game channels
			if(Iter->Channel >= MinCustomCollisionChannel && Iter->bTraceType)
			{
				TraceChannelList.Add(MakeShareable(new FChannelListItem(MakeShareable(new FCustomChannelSetup(*Iter)))));
			}
		}
	}
	else
	{
		ObjectChannelList.Empty();

		for(auto Iter = CollisionProfile->DefaultChannelResponses.CreateIterator(); Iter; ++Iter)
		{
			// only display game channels
			if(Iter->Channel >= MinCustomCollisionChannel && !Iter->bTraceType)
			{
				ObjectChannelList.Add(MakeShareable(new FChannelListItem(MakeShareable(new FCustomChannelSetup(*Iter)))));
			}
		}
	}
}

void FCollisionProfileDetails::RefreshProfileList()
{
	ProfileList.Empty();

	for(auto Iter = CollisionProfile->Profiles.CreateIterator(); Iter; ++Iter)
	{
		ProfileList.Add(MakeShareable(new FProfileListItem(MakeShareable(new FCollisionResponseTemplate(*Iter)))));
	}
}

TSharedRef<ITableRow> FCollisionProfileDetails::HandleGenerateChannelWidget(TSharedPtr< FChannelListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SChannelListItem, OwnerTable)
		.ChannelSetup(InItem->ChannelSetup);
}

TSharedRef<ITableRow> FCollisionProfileDetails::HandleGenerateProfileWidget(TSharedPtr< FProfileListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SProfileListItem, OwnerTable)
		.ProfileTemplate(InItem->ProfileTemplate);
}

void FCollisionProfileDetails::RemoveChannel(ECollisionChannel CollisionChannel) const
{
	for(auto Iter = CollisionProfile->DefaultChannelResponses.CreateIterator(); Iter; ++Iter)
	{
		if(Iter->Channel == CollisionChannel)
		{
			CollisionProfile->DefaultChannelResponses.RemoveAt(Iter.GetIndex());
			break;
		}
	}
}

int32 FCollisionProfileDetails::FindProfileIndexFromName(FName Name) const
{
	for(auto Iter = CollisionProfile->Profiles.CreateIterator(); Iter; ++Iter)
	{
		if(Iter->Name == Name)
		{
			return Iter.GetIndex();
		}
	}

	return INDEX_NONE;
}

FCustomChannelSetup * FCollisionProfileDetails::FindFromChannel(ECollisionChannel CollisionChannel) const
{
	for(auto Iter = CollisionProfile->DefaultChannelResponses.CreateIterator(); Iter; ++Iter)
	{
		if(Iter->Channel == CollisionChannel)
		{
			return &(*Iter);
		}
	}

	return NULL;
}

ECollisionChannel	FCollisionProfileDetails::FindAvailableChannel() const
{
	if(CollisionProfile->DefaultChannelResponses.Num() < MaxNumCustomCollisionChannels)
	{
		// this is very inefficient
		for(int32 ChannelIndex = MinCustomCollisionChannel; ChannelIndex < MinCustomCollisionChannel + MaxNumCustomCollisionChannels; ++ChannelIndex)
		{
			if( FindFromChannel((ECollisionChannel)ChannelIndex) == NULL)
			{
				return (ECollisionChannel)ChannelIndex;
			}
		}
	}

	return ECC_MAX;
}

bool	FCollisionProfileDetails::IsValidChannelSetup(const FCustomChannelSetup* Channel) const
{
	for(auto Iter = CollisionProfile->DefaultChannelResponses.CreateConstIterator(); Iter; ++Iter)
	{
		if(Iter->Channel != Channel->Channel)
		{
			// make sure name isn't same
			if(Iter->Name == Channel->Name)
			{
				return false;
			}
		}
	}

	return true;
}

bool	FCollisionProfileDetails::IsValidProfileSetup(const FCollisionResponseTemplate* Template, int32 ProfileIndex) const
{
	for(auto Iter = CollisionProfile->Profiles.CreateConstIterator(); Iter; ++Iter)
	{
		if(ProfileIndex != Iter.GetIndex())
		{
			// make sure name isn't same
			if(Iter->Name == Template->Name)
			{
				return false;
			}
		}
	}

	return true;
}
bool	FCollisionProfileDetails::IsNewChannelAvailable() const
{
	return (CollisionProfile && CollisionProfile->DefaultChannelResponses.Num() < MaxNumCustomCollisionChannels);
}

FReply FCollisionProfileDetails::OnNewChannel(bool bTraceType)
{
	// find empty channel and see if we can add it. 
	ECollisionChannel NewChannel = FindAvailableChannel();

	if (ensure(NewChannel >= MinCustomCollisionChannel && NewChannel <= MaxCustomCollisionChannel))
	{
		// Create modal window for modification
		TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
			.Title(LOCTEXT("CollisionProfileDetail_NewChannelTitle", "New Channel"))
			.ClientSize(FVector2f(CHANNEL_WINDOW_WIDTH, CHANNEL_WINDOW_HEIGHT))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::UserSized);

		TSharedPtr<SChannelEditDialog> ChannelEditor;
		WidgetWindow->SetContent
			(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ChannelEditor, SChannelEditDialog)
				.ChannelSetup(NULL)
				.CollisionChannel(NewChannel)
				.WidgetWindow(WidgetWindow)
				.bTraceType(bTraceType)
				.OnValidateChannel(this, &FCollisionProfileDetails::IsValidChannelSetup)
			]
		);

		GEditor->EditorAddModalWindow(WidgetWindow);

		// add to collision profile
		if(ChannelEditor->bApplyChange && 
			ensure(IsValidChannelSetup(&(ChannelEditor->ChannelSetup))))
		{
			CollisionProfile->DefaultChannelResponses.Add(ChannelEditor->ChannelSetup);
			UpdateChannel(bTraceType);
		}
	}

	return FReply::Handled();
}

FReply	FCollisionProfileDetails::OnEditChannel(bool bTraceType)
{
	TArray< TSharedPtr< FChannelListItem > > SelectedItems = (bTraceType)? TraceChannelListView->GetSelectedItems() : ObjectChannelListView->GetSelectedItems();

	if (SelectedItems.Num() == 1)
	{
		TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
			.Title(LOCTEXT("CollisionProfileDetail_EditChannelTitle", "Edit Channel"))
			.ClientSize(FVector2f(CHANNEL_WINDOW_WIDTH, CHANNEL_WINDOW_HEIGHT))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::UserSized);

		TSharedPtr< FChannelListItem > SelectedItem = SelectedItems[0];
		TSharedPtr<SChannelEditDialog> ChannelEditor;
		WidgetWindow->SetContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ChannelEditor, SChannelEditDialog)
				.ChannelSetup(SelectedItem->ChannelSetup.Get())
				.WidgetWindow(WidgetWindow)
				.OnValidateChannel(this, &FCollisionProfileDetails::IsValidChannelSetup)
			]
		);

		GEditor->EditorAddModalWindow(WidgetWindow);

		// add to collision profile
		if(ChannelEditor->bApplyChange && 
			ensure(IsValidChannelSetup(&(ChannelEditor->ChannelSetup))))
		{
			FCustomChannelSetup * Item = FindFromChannel(ChannelEditor->ChannelSetup.Channel);
			if(Item)
			{
				// if name changed, we need to add to redirect
				if(Item->Name != ChannelEditor->ChannelSetup.Name)
				{
					CollisionProfile->AddChannelRedirect(Item->Name, ChannelEditor->ChannelSetup.Name);
				}

				*Item = ChannelEditor->ChannelSetup;
				// refresh view
				UpdateChannel(Item->bTraceType);
			}
		}
	}

	return FReply::Handled();
}

bool	FCollisionProfileDetails::IsAnyChannelSelected(bool bTraceType) const
{
	return (bTraceType)? TraceChannelListView->GetNumItemsSelected() > 0: ObjectChannelListView->GetNumItemsSelected() > 0;
}

FReply	FCollisionProfileDetails::OnDeleteChannel(bool bInTraceType)
{
	TArray< TSharedPtr< FChannelListItem > > SelectedItems = (bInTraceType) ? TraceChannelListView->GetSelectedItems() : ObjectChannelListView->GetSelectedItems();

	if(SelectedItems.Num() == 1)
	{
		if(FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("FCollisionProfileDetails_DeleteChannel", "If you delete this channel, all the objects that use this channel will be set to default. \nWould you like to continue?")) == EAppReturnType::Yes)
		{
			TSharedPtr< FChannelListItem > SelectedItem = SelectedItems[0];
			bool bTraceType = SelectedItem->ChannelSetup->bTraceType;
			RemoveChannel(SelectedItem->ChannelSetup->Channel);
			UpdateChannel(bTraceType);
		}
	}
	return FReply::Handled();
}

FReply	FCollisionProfileDetails::OnNewProfile()
{
	// Create modal window for modification
	TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
		.Title(LOCTEXT("CollisionProfileDetail_NewProfileTitle", "New Profile"))
		.ClientSize(FVector2f(PROFILE_WINDOW_WIDTH, PROFILE_WINDOW_HEIGHT))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::UserSized);

	TSharedPtr<SProfileEditDialog> ProfileEditor;
	WidgetWindow->SetContent
		(
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(ProfileEditor, SProfileEditDialog)
			.ProfileTemplate(NULL)
			.CollisionProfile(CollisionProfile)
			.ProfileIndex(INDEX_NONE)
			.WidgetWindow(WidgetWindow)
			.ObjectTypeMapping(CollisionProfile->ObjectTypeMapping)
			.OnValidateProfile(this, &FCollisionProfileDetails::IsValidProfileSetup)
		]
	);

	GEditor->EditorAddModalWindow(WidgetWindow);

	// add to collision profile
	if(ProfileEditor->bApplyChange &&
		ensure(IsValidProfileSetup(&(ProfileEditor->ProfileTemplate), ProfileEditor->ProfileIndex)))
	{
		CollisionProfile->SaveCustomResponses(ProfileEditor->ProfileTemplate);
		CollisionProfile->Profiles.Add(ProfileEditor->ProfileTemplate);
		UpdateProfile();
	}

	return FReply::Handled();
}

FReply	FCollisionProfileDetails::OnEditProfile()
{
	TArray< TSharedPtr< FProfileListItem > > SelectedItems = ProfileListView->GetSelectedItems();

	if(SelectedItems.Num() == 1)
	{
		// find which profile it's trying to edit
		TSharedPtr< FProfileListItem > SelectedItem = SelectedItems[0];
		int32 ProfileIndex = FindProfileIndexFromName(SelectedItem->ProfileTemplate->Name);

		if(ProfileIndex != INDEX_NONE)
		{
			// Create modal window for modification
			TSharedRef<SWindow> WidgetWindow = SNew(SWindow)
				.Title(LOCTEXT("CollisionProfileDetail_EditProfileTitle", "Edit Profile"))
				.ClientSize(FVector2f(PROFILE_WINDOW_WIDTH, PROFILE_WINDOW_HEIGHT))
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.SizingRule(ESizingRule::UserSized);


			TSharedPtr<SProfileEditDialog> ProfileEditor;
			WidgetWindow->SetContent
				(
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(ProfileEditor, SProfileEditDialog)
					.ProfileTemplate(&CollisionProfile->Profiles[ProfileIndex])
					.CollisionProfile(CollisionProfile)
					.ProfileIndex(ProfileIndex)
					.WidgetWindow(WidgetWindow)
					.ObjectTypeMapping(CollisionProfile->ObjectTypeMapping)
					.OnValidateProfile(this, &FCollisionProfileDetails::IsValidProfileSetup)
				]
			);

			GEditor->EditorAddModalWindow(WidgetWindow);

			// add to collision profile
			if(ProfileEditor->bApplyChange &&
				ensure(IsValidProfileSetup(&(ProfileEditor->ProfileTemplate), ProfileIndex)))
			{
				CommitProfileChange(ProfileIndex, ProfileEditor->ProfileTemplate);
				UpdateProfile();
			}
		}
		else
		{
			// invalid profile
		}
	}

	return FReply::Handled();
}

FReply	FCollisionProfileDetails::OnDeleteProfile()
{
	TArray< TSharedPtr< FProfileListItem > > SelectedItems = ProfileListView->GetSelectedItems();

	if(SelectedItems.Num() == 1)
	{
		if(FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("FCollisionProfileDetails_DeletePreset", "If you delete this preset, all the objects that use this preset will be set to default. \nWould you like to continue?")) == EAppReturnType::Yes)
		{
			TSharedPtr< FProfileListItem > SelectedItem = SelectedItems[0];
			int32 ProfileIndex = FindProfileIndexFromName(SelectedItem->ProfileTemplate->Name);
			if(ProfileIndex != INDEX_NONE)
			{
				CollisionProfile->Profiles.RemoveAt(ProfileIndex);
				UpdateProfile();
			}
		}
	}
	return FReply::Handled();
}

bool	FCollisionProfileDetails::IsAnyProfileSelected() const
{
	return ProfileListView->GetNumItemsSelected() > 0;
}

/** SListView item double clicked */
void FCollisionProfileDetails::OnObjectChannelListItemDoubleClicked(TSharedPtr< FChannelListItem >)
{
	OnEditChannel(false);
}

void FCollisionProfileDetails::OnTraceChannelListItemDoubleClicked(TSharedPtr< FChannelListItem >)
{
	OnEditChannel(true);
}

void FCollisionProfileDetails::OnProfileListItemDoubleClicked(TSharedPtr< FProfileListItem >)
{
	OnEditProfile();
}

//====================================================================================
// FCollsiionProfileData
//=====================================================================================

void FCollisionProfileDetails::FCollisionProfileData::Save(UCollisionProfile * Profile)
{
	Profiles = Profile->Profiles;
	DefaultChannelResponses = Profile->DefaultChannelResponses;
	EditProfiles = Profile->EditProfiles;
}

//=====================================================================================

#undef LOCTEXT_NAMESPACE
#undef RowWidth_Customization

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
