// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "Engine/CollisionProfile.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class IDetailLayoutBuilder;
class ITableRow;
class STableViewBase;
class SWidget;

// Class containing the friend information - used to build the list view
class FChannelListItem
{
public:

	/**
	* Constructor takes the required details
	*/
	FChannelListItem(TSharedPtr<FCustomChannelSetup> InChannelSetup)
		: ChannelSetup(InChannelSetup)
	{}

	TSharedPtr<FCustomChannelSetup> ChannelSetup;
};

/**
* Implements the FriendsList
*/
class SChannelListItem
	: public SMultiColumnTableRow< TSharedPtr<class FChannelListItem> >
{
public:

	SLATE_BEGIN_ARGS(SChannelListItem) { }
	// for now this is okay to be all argument and it's better for now
	// in the future if we can change this in multiple places, we'll have to change this to be attribute
		SLATE_ARGUMENT(TSharedPtr<FCustomChannelSetup>, ChannelSetup)
	SLATE_END_ARGS()

public:

	/**
	* Constructs the application.
	*
	* @param InArgs - The Slate argument list.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	FText GetDefaultResponse() const;

	TSharedPtr<FCustomChannelSetup> ChannelSetup;
};


// Class containing the friend information - used to build the list view
class FProfileListItem
{
public:

	/**
	* Constructor takes the required details
	*/
	FProfileListItem(TSharedPtr<FCollisionResponseTemplate> InProfileTemplate)
		: ProfileTemplate(InProfileTemplate)
	{}

	TSharedPtr<FCollisionResponseTemplate> ProfileTemplate;
};

/**
* Implements the FriendsList
*/
class SProfileListItem
	: public SMultiColumnTableRow< TSharedPtr<class FProfileListItem> >
{
public:

	SLATE_BEGIN_ARGS(SProfileListItem) { }
	// for now this is okay to be all argument and it's better for now
	// in the future if we can change this in multiple places, we'll have to change this to be attribute
	SLATE_ARGUMENT(TSharedPtr<FCollisionResponseTemplate>, ProfileTemplate)
		SLATE_END_ARGS()

public:

	/**
	* Constructs the application.
	*
	* @param InArgs - The Slate argument list.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	FText GetObjectType() const;
	FText GetCollsionEnabled() const;
	TSharedPtr<FCollisionResponseTemplate> ProfileTemplate;
};

typedef  SListView< TSharedPtr< FChannelListItem > > SChannelListView;
typedef  SListView< TSharedPtr< FProfileListItem > > SProfileListView;

class FCollisionProfileDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/**
	* Generates a widget for a channel item.
	* @param InItem - the ChannelListItem
	* @param OwnerTable - the owning table
	* @return The table row widget
	*/
	TSharedRef<ITableRow> HandleGenerateChannelWidget(TSharedPtr< FChannelListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedRef<ITableRow> HandleGenerateProfileWidget(TSharedPtr< FProfileListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	// button handlers
	FReply	OnNewChannel(bool bTraceType);
	bool	IsNewChannelAvailable() const;
	FReply	OnEditChannel(bool bTraceType);
	bool	IsAnyChannelSelected(bool bTraceType) const;
	FReply	OnDeleteChannel(bool bTraceType);
	bool	IsValidChannelSetup(const FCustomChannelSetup* Channel) const;

	FReply	OnNewProfile();
	FReply	OnEditProfile();
	FReply	OnDeleteProfile();
	bool	IsAnyProfileSelected() const;
	bool	IsValidProfileSetup(const FCollisionResponseTemplate* Template, int32 ProfileIndex) const;

private:
	TSharedPtr<SChannelListView>				ObjectChannelListView;
	TArray< TSharedPtr< FChannelListItem > >	ObjectChannelList;

	TSharedPtr<SChannelListView>				TraceChannelListView;
	TArray< TSharedPtr< FChannelListItem > >	TraceChannelList;

	TSharedPtr<SProfileListView>				ProfileListView;
	TArray< TSharedPtr< FProfileListItem > >	ProfileList;

	UCollisionProfile *							CollisionProfile;

	// functions
	ECollisionChannel	FindAvailableChannel() const;
	void UpdateChannel(bool bTraceType);
	void UpdateProfile();
	void CommitProfileChange(int32 ProfileIndex, FCollisionResponseTemplate & NewProfile);
	void RefreshChannelList(bool bTraceType);
	void RefreshProfileList();
	FCustomChannelSetup * FindFromChannel(ECollisionChannel CollisionChannel) const;
	void RemoveChannel(ECollisionChannel CollisionChannel) const;

	int32 FindProfileIndexFromName(FName Name) const;

	void OnObjectChannelListItemDoubleClicked(TSharedPtr< FChannelListItem >);
	void OnTraceChannelListItemDoubleClicked(TSharedPtr< FChannelListItem >);
	void OnProfileListItemDoubleClicked(TSharedPtr< FProfileListItem >);

	// this is the data that saves before starting, and creates the EditProfiles based on that. 
	// this is needed if we do EditProfiles
	struct FCollisionProfileData
	{
		TArray<FCollisionResponseTemplate>	Profiles;
		TArray<FCustomChannelSetup>			DefaultChannelResponses;
		TArray<FCustomProfile>				EditProfiles;

		void Save(UCollisionProfile * Profile);
	};

	FCollisionProfileData SavedData;
};

