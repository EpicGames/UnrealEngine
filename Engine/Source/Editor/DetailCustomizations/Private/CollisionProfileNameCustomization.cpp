// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionProfileNameCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/CollisionProfile.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformMath.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


#define LOCTEXT_NAMESPACE "CollisionProfileNameCustomization"


FCollisionProfileNameCustomization::FCollisionProfileNameCustomization()
{
	UCollisionProfile::GetProfileNames(NameList);
}


TSharedRef<IPropertyTypeCustomization> FCollisionProfileNameCustomization::MakeInstance()
{
	return MakeShareable(new FCollisionProfileNameCustomization);
}


void FCollisionProfileNameCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}


void FCollisionProfileNameCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	NameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCollisionProfileName, Name));
	check(NameHandle.IsValid());

	TSharedPtr<FName> InitialSelectedName = GetSelectedName();
	if (InitialSelectedName.IsValid())
	{
		SetPropertyWithName(*InitialSelectedName.Get());
	}

	IDetailGroup& CollisionGroup = StructBuilder.AddGroup(TEXT("Collision"), LOCTEXT("CollisionPresetName", "Collision Preset"));
	CollisionGroup.HeaderRow()
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SAssignNew(NameComboBox, SComboBox<TSharedPtr<FName>>)
			.OptionsSource(&NameList)
			.OnGenerateWidget(this, &FCollisionProfileNameCustomization::OnGenerateWidget)
			.OnSelectionChanged(this, &FCollisionProfileNameCustomization::OnSelectionChanged, &CollisionGroup)
			.OnComboBoxOpening(this, &FCollisionProfileNameCustomization::OnComboBoxOpening)
			.InitiallySelectedItem(InitialSelectedName)
			.IsEnabled(StructPropertyHandle->IsEditable())
			.ContentPadding(FMargin(2.0f, 2.0f))
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FCollisionProfileNameCustomization::GetProfileComboBoxContent)
				.Margin(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTipText(this, &FCollisionProfileNameCustomization::GetProfileComboBoxToolTip)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
		]
	];
}


TSharedRef<SWidget> FCollisionProfileNameCustomization::OnGenerateWidget(TSharedPtr<FName> InItem)
{
	return
		SNew(STextBlock)
		.Text(FText::FromName(*InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}


void FCollisionProfileNameCustomization::OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo, IDetailGroup* CollisionGroup)
{
	if (NameItem.IsValid())
	{
		SetPropertyWithName(*NameItem);
	}
}


void FCollisionProfileNameCustomization::OnComboBoxOpening()
{
	TSharedPtr<FName> SelectedName = GetSelectedName();
	if (SelectedName.IsValid())
	{
		check(NameComboBox.IsValid());
		NameComboBox->SetSelectedItem(SelectedName);
	}
}


TSharedPtr<FName> FCollisionProfileNameCustomization::GetSelectedName() const
{
	int32 NameCount = NameList.Num();
	if (NameCount <= 0)
	{
		return NULL;
	}

	FName Name;
	GetPropertyAsName(Name);

	for (int32 NameIndex = 0; NameIndex < NameCount; ++NameIndex)
	{
		if (Name == *NameList[NameIndex].Get())
		{
			return NameList[NameIndex];
		}
	}

	return NameList[0];
}


void FCollisionProfileNameCustomization::SetPropertyWithName(const FName& Name)
{
	check(NameHandle.IsValid());

	FName OldName;
	NameHandle->GetValue(OldName);

	if (OldName != Name)
	{
		NameHandle->SetValue(Name);
	}
}


void FCollisionProfileNameCustomization::GetPropertyAsName(FName& OutName) const
{
	check(NameHandle.IsValid());
	NameHandle->GetValue(OutName);

	UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
	check(CollisionProfile);

	const FName* RedirectName = CollisionProfile->LookForProfileRedirect(OutName);
	if (RedirectName)
	{
		OutName = *RedirectName;
	}
}


FText FCollisionProfileNameCustomization::GetProfileComboBoxContent() const
{
	TSharedPtr<FName> SelectedName = GetSelectedName();
	if (SelectedName.IsValid())
	{
		return FText::FromName(*SelectedName);
	}

	return LOCTEXT("Invalid", "Invalid");
}


FText FCollisionProfileNameCustomization::GetProfileComboBoxToolTip() const
{
	UCollisionProfile* CollisionProfile = UCollisionProfile::Get();
	check(CollisionProfile);

	FName ProfileName;
	GetPropertyAsName(ProfileName);

	FCollisionResponseTemplate ProfileTemplate;
	if (CollisionProfile->GetProfileTemplate(ProfileName, ProfileTemplate))
	{
		return FText::FromString(ProfileTemplate.HelpMessage);
	}

	return LOCTEXT("Invalid", "Invalid");
}


#undef LOCTEXT_NAMESPACE
