// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SComboBox.h"

class IDetailGroup;
class IPropertyHandle;
class SWidget;

/**
 * Customizes a CollisionProfileName property to use a dropdown
 */
class FCollisionProfileNameCustomization : public IPropertyTypeCustomization
{
public:

	FCollisionProfileNameCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FName> InItem);

	void OnSelectionChanged(TSharedPtr<FName> NameItem, ESelectInfo::Type SelectInfo, IDetailGroup* CollisionGroup);
	void OnComboBoxOpening();

	TSharedPtr<FName> GetSelectedName() const;

	void SetPropertyWithName(const FName& Name);
	void GetPropertyAsName(FName& OutName) const;

	FText GetProfileComboBoxContent() const;
	FText GetProfileComboBoxToolTip() const;

protected:

	TSharedPtr<IPropertyHandle> NameHandle;
	TArray<TSharedPtr<FName>> NameList;
	TSharedPtr<SComboBox<TSharedPtr<FName>>> NameComboBox;
};
