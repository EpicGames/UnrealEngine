// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorGradingSettingsCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"

void FColorGradingSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Allow ShowOnlyInnerProperties metadata flag to affect this struct
	const bool bShowInners = PropertyHandle->HasMetaData("ShowOnlyInnerProperties");
	if (!bShowInners)
	{
		HeaderRow.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			PropertyHandle->CreatePropertyValueWidget()
		];
	}
}

void FColorGradingSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TMap<FName, FString> PropertyGroupMap;
	TSet<FString> GroupSet;

	GetPropertyGroups(PropertyHandle, PropertyGroupMap, GroupSet);
	
	uint32 NumChildProperties;
	PropertyHandle->GetNumChildren(NumChildProperties);
	for (uint32 Index = 0; Index < NumChildProperties; ++Index)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);
		if (!ChildHandle.IsValid() || !ChildHandle->IsValidHandle() || ChildHandle->IsCustomized())
		{
			continue;
		}
		
		FProperty* ChildProperty = ChildHandle->GetProperty();

		if (PropertyGroupMap.Contains(ChildProperty->GetFName()))
		{
			// If the property specified an explicit group to be part of, add the handle to that group
			FString GroupName = PropertyGroupMap[ChildProperty->GetFName()];
			IDetailGroup& Group = GetOrAddGroup(ChildBuilder, *GroupName, FText::FromString(GroupName));
			Group.AddPropertyRow(ChildHandle.ToSharedRef());
		}
		else if (GroupSet.Contains(ChildHandle->GetPropertyDisplayName().ToString()))
		{
			// If an explicit group is needed whose name matches that of the property name, we need an explicit group that the property's children will be added to
			AddPropertyChildrenExplicitly(ChildHandle.ToSharedRef(), ChildBuilder);
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

void FColorGradingSettingsCustomization::GetPropertyGroups(TSharedRef<IPropertyHandle> PropertyHandle, TMap<FName, FString>& OutPropertyGroupMap, TSet<FString>& OutGroupSet)
{
	uint32 NumChildProperties;
	PropertyHandle->GetNumChildren(NumChildProperties);
	for (uint32 Index = 0; Index < NumChildProperties; ++Index)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);
		if (!ChildHandle.IsValid() || !ChildHandle->IsValidHandle())
		{
			continue;
		}
		
		FProperty* ChildProperty = ChildHandle->GetProperty();
		FName ChildCategoryFName = FObjectEditorUtils::GetCategoryFName(ChildProperty);
		FString ChildCategoryStr = ChildCategoryFName.ToString();

		if (!ChildCategoryStr.Contains(TEXT("|")))
		{
			continue;
		}

		TArray<FString> ParsedCategories;
		ChildCategoryStr.ParseIntoArray(ParsedCategories, TEXT("|"));

		if (ParsedCategories.Num() < 2)
		{
			continue;
		}

		// For now, only support one level of subcategories
		FString Group = ParsedCategories.Last();
		OutPropertyGroupMap.Add(ChildProperty->GetFName(), Group);
		OutGroupSet.Add(Group);
	}
}

void FColorGradingSettingsCustomization::AddPropertyChildrenExplicitly(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder)
{
	IDetailGroup& PropertyGroup = GetOrAddGroup(ChildBuilder, PropertyHandle->GetProperty()->GetFName(), PropertyHandle->GetPropertyDisplayName());
	
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex);
		if (ChildHandle && ChildHandle->IsValidHandle())
		{
			PropertyGroup.AddPropertyRow(ChildHandle.ToSharedRef());
		}
	}
}

IDetailGroup& FColorGradingSettingsCustomization::GetOrAddGroup(IDetailChildrenBuilder& ChildBuilder, const FName& GroupName, const FText& GroupDisplayName)
{
	IDetailGroup* ExistingGroup = ChildBuilder.GetGroup(GroupName);
	if (!ExistingGroup)
	{
		ExistingGroup = &ChildBuilder.AddGroup(GroupName, GroupDisplayName);
	}

	return *ExistingGroup;
}
