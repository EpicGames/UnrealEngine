// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IDetailGroup;

/**
 * Implements a details view customization for the FColorGradingSettings structure.
 */
class FColorGradingSettingsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FColorGradingSettingsCustomization());
	}

public:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/**
	 * Iterates the property's children and finds all child properties that have a subcategory and need a group made for them
	 * @param PropertyHandle The property handle being customized
	 * @param OutPropertyGroupMap A map of properties and the group they want to be in
	 * @param OutGroupSet The set of all groups that need to be explicitly added
	 */
	void GetPropertyGroups(TSharedRef<IPropertyHandle> PropertyHandle, TMap<FName, FString>& OutPropertyGroupMap, TSet<FString>& OutGroupSet);

	/**
	 * Adds the specified property's children explicitly to the details panel under a custom group that matches the property's name.
	 * Used in case a group is needed to embed additional properties that don't live in the specified property
	 * 
	 * @param PropertyHandle The property handle whose children are added explicitly
	 * @param ChildBuilder The detail children builder
	 */
	void AddPropertyChildrenExplicitly(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder);

	/** Gets an existing group, or adds one if not found */
	IDetailGroup& GetOrAddGroup(IDetailChildrenBuilder& ChildBuilder, const FName& GroupName, const FText& GroupDisplayName);
};
