// Copyright Epic Games, Inc. All Rights Reserved.
#include "ColorGradingVectorCustomization.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Customizations/MathStructCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/NumericLimits.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealType.h"
#include "Util/ColorGradingUtil.h"
#include "Vector4StructCustomization.h"
#include "Widgets/ColorGrading/SColorGradingPicker.h"
#include "Widgets/ColorGrading/SColorGradingComponentViewer.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyTypeCustomizationUtils;

#define LOCTEXT_NAMESPACE "FColorGradingCustomization"


namespace
{
	static FVector4 ClampValueFromMetaData(FVector4 InValue, FProperty* InProperty)
	{
		FVector4 RetVal = InValue;
		if (InProperty)
		{
			//Enforce min
			const FString& MinString = InProperty->GetMetaData(TEXT("ClampMin"));
			if (MinString.Len())
			{
				checkSlow(MinString.IsNumeric());
				double MinValue;
				TTypeFromString<double>::FromString(MinValue, *MinString);
				for (int Index = 0; Index < 4; Index++)
				{
					RetVal[Index] = FMath::Max<double>(MinValue, RetVal[Index]);
				}
			}
			//Enforce max 
			const FString& MaxString = InProperty->GetMetaData(TEXT("ClampMax"));
			if (MaxString.Len())
			{
				checkSlow(MaxString.IsNumeric());
				double MaxValue;
				TTypeFromString<double>::FromString(MaxValue, *MaxString);

				for (int Index = 0; Index < 4; Index++)
				{
					RetVal[Index] = FMath::Min<double>(MaxValue, RetVal[Index]);
				}
			}
		}

		return RetVal;
	}
}

FColorGradingVectorCustomizationBase::FColorGradingVectorCustomizationBase(const FTrackedVector4PropertyHandle& InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray)
	: ColorGradingPropertyHandle(InColorGradingPropertyHandle)
	, SortedChildArray(InSortedChildArray)
	, IsRGBMode(true)
	, bIsUsingSlider(false)
{
	if (ColorGradingPropertyHandle.IsValidHandle())
	{
		FVector4 VectorValue;
		ColorGradingPropertyHandle.GetValue(VectorValue);
		CurrentHSVColor = FLinearColor(static_cast<float>(VectorValue.X), static_cast<float>(VectorValue.Y), static_cast<float>(VectorValue.Z)).LinearRGBToHSV();
	}
}

UE::ColorGrading::EColorGradingModes FColorGradingVectorCustomizationBase::GetColorGradingMode() const
{
	UE::ColorGrading::EColorGradingModes ColorGradingMode = UE::ColorGrading::EColorGradingModes::Invalid;

	if (ColorGradingPropertyHandle.IsValidHandle())
	{
		//Query all meta data we need
		FProperty* Property = ColorGradingPropertyHandle.GetHandle()->GetProperty();
		const FString& ColorGradingModeString = Property->GetMetaData(TEXT("ColorGradingMode"));

		if (ColorGradingModeString.Len() > 0)
		{
			if (ColorGradingModeString.Compare(TEXT("saturation")) == 0)
			{
				ColorGradingMode = UE::ColorGrading::EColorGradingModes::Saturation;
			}
			else if (ColorGradingModeString.Compare(TEXT("contrast")) == 0)
			{
				ColorGradingMode = UE::ColorGrading::EColorGradingModes::Contrast;
			}
			else if (ColorGradingModeString.Compare(TEXT("gamma")) == 0)
			{
				ColorGradingMode = UE::ColorGrading::EColorGradingModes::Gamma;
			}
			else if (ColorGradingModeString.Compare(TEXT("gain")) == 0)
			{
				ColorGradingMode = UE::ColorGrading::EColorGradingModes::Gain;
			}
			else if (ColorGradingModeString.Compare(TEXT("offset")) == 0)
			{
				ColorGradingMode = UE::ColorGrading::EColorGradingModes::Offset;
			}
		}
	}

	return ColorGradingMode;
}

bool FColorGradingVectorCustomizationBase::IsInRGBMode() const
{
	return IsRGBMode;
}

TOptional<float> FColorGradingVectorCustomizationBase::OnGetMaxSliderValue(TOptional<float> DefaultMaxSliderValue, int32 ColorIndex) const
{
	if (ColorIndex == 0 && !IsRGBMode) // Hue value
	{
		return 359.0f;
	}
	else if (ColorIndex == 1 && !IsRGBMode) // Saturation value
	{
		return 1.0f;
	}

	return SpinBoxMinMaxSliderValues.CurrentMaxSliderValue.IsSet() ? SpinBoxMinMaxSliderValues.CurrentMaxSliderValue : DefaultMaxSliderValue;
}

TOptional<float> FColorGradingVectorCustomizationBase::OnGetMinSliderValue(TOptional<float> DefaultMinSliderValue, int32 ColorIndex) const
{
	if (!IsRGBMode)
	{
		return 0.0f;
	}

	return SpinBoxMinMaxSliderValues.CurrentMinSliderValue.IsSet() ? SpinBoxMinMaxSliderValues.CurrentMinSliderValue : DefaultMinSliderValue;
}

float FColorGradingVectorCustomizationBase::OnGetSliderDeltaValue(float DefaultValue, int32 ColorIndex) const
{
	if (ColorIndex == 0 && !IsRGBMode) // Hue value
	{
		return 1.0f;
	}

	return DefaultValue;
}

TOptional<float> FColorGradingVectorCustomizationBase::OnGetMaxValue(TOptional<float> DefaultValue, int32 ColorIndex) const
{
	if (ColorIndex == 0 && !IsRGBMode) // Hue value
	{
		return 359.0f;
	}
	else if (ColorIndex == 1 && !IsRGBMode) // Saturation value
	{
		return 1.0f;
	}

	return DefaultValue;
}

void FColorGradingVectorCustomizationBase::OnBeginSliderMovement()
{
	bIsUsingSlider = true;
	GEditor->BeginTransaction(FText::Format(NSLOCTEXT("ColorGradingVectorCustomization", "SetPropertyValue", "Edit {0}"), ColorGradingPropertyHandle.GetHandle()->GetPropertyDisplayName()));
}

void FColorGradingVectorCustomizationBase::OnEndSliderMovement(float NewValue, int32 ColorIndex)
{
	bIsUsingSlider = false;

	OnValueChanged(NewValue, ColorIndex);

	GEditor->EndTransaction();
}

UE::ColorGrading::EColorGradingComponent FColorGradingVectorCustomizationBase::OnGetColorComponent(int32 ColorIndex) const
{
	return UE::ColorGrading::GetColorGradingComponent(
		IsRGBMode ? UE::ColorGrading::EColorGradingColorDisplayMode::RGB : UE::ColorGrading::EColorGradingColorDisplayMode::HSV,
		ColorIndex
	);
}

bool FColorGradingVectorCustomizationBase::GetCurrentColorGradingValue(FVector4& OutCurrentValue)
{
	return ColorGradingPropertyHandle.GetValue(OutCurrentValue) == FPropertyAccess::Success;
}

void FColorGradingVectorCustomizationBase::OnValueChanged(float NewValue, int32 ColorIndex)
{
	FVector4 CurrentValueVector;
	verifySlow(ColorGradingPropertyHandle.GetValue(CurrentValueVector) == FPropertyAccess::Success);
	ClampValueFromMetaData(CurrentValueVector, ColorGradingPropertyHandle.GetHandle()->GetProperty());
	FVector4 NewValueVector = CurrentValueVector;

	if (IsRGBMode)
	{
		NewValueVector[ColorIndex] = NewValue;

		if (ColorIndex < 3)
		{
			CurrentHSVColor = FLinearColor(static_cast<float>(NewValueVector.X), static_cast<float>(NewValueVector.Y), static_cast<float>(NewValueVector.Z)).LinearRGBToHSV();
		}
	}
	else
	{
		if (ColorIndex < 3) 
		{
			CurrentHSVColor.Component(ColorIndex) = NewValue;
			NewValueVector = (FVector4)CurrentHSVColor.HSVToLinearRGB();
			NewValueVector.W = CurrentValueVector.W;
		}
		else // Luminance
		{
			NewValueVector[ColorIndex] = NewValue;
		}

		OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, true);
	}

	if (ColorGradingPropertyHandle.IsValidHandle())
	{
		ColorGradingPropertyHandle.SetValue(NewValueVector, bIsUsingSlider ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags);
	}
}

TOptional<float> FColorGradingVectorCustomizationBase::OnSliderGetValue(int32 ColorIndex) const
{
	FVector4 ValueVector;
	
	if (ColorGradingPropertyHandle.GetValue(ValueVector) == FPropertyAccess::Success)
	{
		float Value = 0.0f;

		if (IsRGBMode)
		{
			Value = static_cast<float>(ValueVector[ColorIndex]);
		}
		else
		{
			Value = (ColorIndex < 3) ? CurrentHSVColor.Component(ColorIndex) : static_cast<float>(ValueVector.W);
		}

		return Value;
	}

	return TOptional<float>();
}

void FColorGradingVectorCustomizationBase::OnCurrentHSVColorChangedDelegate(FLinearColor NewHSVColor, bool Originator)
{
	CurrentHSVColor = NewHSVColor;

	if (Originator)
	{
		OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, false);
	}
}

void FColorGradingVectorCustomizationBase::PostUndo(bool bSuccess)
{
	if (ColorGradingPropertyHandle.IsValidHandle())
	{
		FVector4 CurrentValueVector;
		if (ColorGradingPropertyHandle.GetValue(CurrentValueVector) == FPropertyAccess::Success)
		{
			CurrentHSVColor = FLinearColor(static_cast<float>(CurrentValueVector.X), static_cast<float>(CurrentValueVector.Y), static_cast<float>(CurrentValueVector.Z)).LinearRGBToHSV();
			OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, true);
		}
	}
}

void FColorGradingVectorCustomizationBase::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher)
{
	if (ComponentViewers.Num() > 0)
	{
		if (!SpinBoxMinMaxSliderValues.CurrentMaxSliderValue.IsSet() || (NewMaxSliderValue > SpinBoxMinMaxSliderValues.CurrentMaxSliderValue.GetValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
		{
			SpinBoxMinMaxSliderValues.CurrentMaxSliderValue = NewMaxSliderValue;
		}

		if (IsOriginator)
		{
			OnNumericEntryBoxDynamicSliderMaxValueChanged.Broadcast(NewMaxSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfHigher);
		}
	}
}

void FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower)
{
	if (ComponentViewers.Num() > 0)
	{
		if (!SpinBoxMinMaxSliderValues.CurrentMinSliderValue.IsSet() || (NewMinSliderValue < SpinBoxMinMaxSliderValues.CurrentMinSliderValue.GetValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
		{
			SpinBoxMinMaxSliderValues.CurrentMinSliderValue = NewMinSliderValue;
		}

		if (IsOriginator)
		{
			OnNumericEntryBoxDynamicSliderMinValueChanged.Broadcast(NewMinSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfLower);
		}
	}
}

bool FColorGradingVectorCustomizationBase::GetSupportDynamicSliderMaxValue(bool DefaultValue, int32 ColorIndex) const
{
	if (DefaultValue)
	{
		if (!IsRGBMode)
		{
			return ColorIndex >= 2;
		}
	}

	return DefaultValue;
}

bool FColorGradingVectorCustomizationBase::GetSupportDynamicSliderMinValue(bool DefaultValue, int32 ColorIndex) const
{
	if (DefaultValue)
	{
		if (!IsRGBMode)
		{
			return ColorIndex >= 2;
		}
	}

	return DefaultValue;
}

bool FColorGradingVectorCustomizationBase::IsEntryBoxEnabled(int32 ColorIndex) const
{
	return OnSliderGetValue(ColorIndex) != TOptional<float>();
}

TSharedRef<UE::ColorGrading::SColorGradingComponentViewer> FColorGradingVectorCustomizationBase::MakeComponentViewer(int32 ColorIndex, TOptional<float>& MinValue, TOptional<float>& MaxValue, TOptional<float>& SliderMinValue, TOptional<float>& SliderMaxValue, float& SliderExponent, float& Delta, float& ShiftMultiplier, float& CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue, bool UseCompactDisplay)
{
	TAttribute<UE::ColorGrading::EColorGradingComponent> ComponentGetter = TAttribute<UE::ColorGrading::EColorGradingComponent>::Create(
		TAttribute<UE::ColorGrading::EColorGradingComponent>::FGetter::CreateSP(this, &FColorGradingVectorCustomizationBase::OnGetColorComponent, ColorIndex)
	);

	return SNew(UE::ColorGrading::SColorGradingComponentViewer)
		.Component(ComponentGetter)
		.ColorGradingMode(GetColorGradingMode())
		.UseCompactDisplay(UseCompactDisplay)
		.Value(this, &FColorGradingVectorCustomizationBase::OnSliderGetValue, ColorIndex)
		.OnValueChanged(this, &FColorGradingVectorCustomizationBase::OnValueChanged, ColorIndex)
		.OnBeginSliderMovement(this, &FColorGradingVectorCustomizationBase::OnBeginSliderMovement)
		.OnEndSliderMovement(this, &FColorGradingVectorCustomizationBase::OnEndSliderMovement, ColorIndex)
		.OnQueryCurrentColor(this, &FColorGradingVectorCustomizationBase::GetCurrentColorGradingValue)
		// Only allow spin on handles with one object.  Otherwise it is not clear what value to spin
		.AllowSpin(ColorGradingPropertyHandle.GetHandle()->GetNumOuterObjects() == 1)
		.ShiftMultiplier(ShiftMultiplier)
		.CtrlMultiplier(CtrlMultiplier)
		.SupportDynamicSliderMaxValue(this, &FColorGradingVectorCustomizationBase::GetSupportDynamicSliderMaxValue, SupportDynamicSliderMaxValue, ColorIndex)
		.SupportDynamicSliderMinValue(this, &FColorGradingVectorCustomizationBase::GetSupportDynamicSliderMinValue, SupportDynamicSliderMinValue, ColorIndex)
		.OnDynamicSliderMaxValueChanged(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged)
		.OnDynamicSliderMinValueChanged(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged)
		.MinValue(MinValue)
		.MaxValue(this, &FColorGradingVectorCustomizationBase::OnGetMaxValue, MaxValue, ColorIndex)
		.MinSliderValue(this, &FColorGradingVectorCustomizationBase::OnGetMinSliderValue, SliderMinValue, ColorIndex)
		.MaxSliderValue(this, &FColorGradingVectorCustomizationBase::OnGetMaxSliderValue, SliderMaxValue, ColorIndex)
		.SliderExponent(SliderExponent)
		.SliderExponentNeutralValue(SliderMinValue.GetValue() + (SliderMaxValue.GetValue() - SliderMinValue.GetValue()) / 2.0f)
		.Delta(this, &FColorGradingVectorCustomizationBase::OnGetSliderDeltaValue, Delta, ColorIndex)
		.IsEnabled(this, &FColorGradingVectorCustomizationBase::IsEntryBoxEnabled, ColorIndex);
}

//////////////////////////////////////////////////////////////////////////
// Color Gradient customization implementation

FColorGradingVectorCustomization::FColorGradingVectorCustomization(TWeakPtr<IPropertyHandle> InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray)
	: FColorGradingVectorCustomizationBase(InColorGradingPropertyHandle, InSortedChildArray)
{
	CustomColorGradingBuilder = nullptr;
}

FColorGradingVectorCustomization::~FColorGradingVectorCustomization()
{	
}

void FColorGradingVectorCustomization::MakeHeaderRow(FDetailWidgetRow& Row, TSharedRef<FVector4StructCustomization> InVector4Customization)
{
	TSharedPtr<SHorizontalBox> ContentHorizontalBox = SNew(SHorizontalBox)
		.IsEnabled(InVector4Customization, &FMathStructCustomization::IsValueEnabled, ColorGradingPropertyHandle.GetHandle().ToWeakPtr());

	Row.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				ColorGradingPropertyHandle.GetHandle()->CreatePropertyNameWidget()
			]
		];

	UE::ColorGrading::EColorGradingModes ColorGradingMode = GetColorGradingMode();

	if (ColorGradingMode == UE::ColorGrading::EColorGradingModes::Offset)
	{
		Row.ValueContent()
			// Make enough space for each child handle
			.MinDesiredWidth(125.0f * static_cast<float>(SortedChildArray.Num()))
			.MaxDesiredWidth(125.0f * static_cast<float>(SortedChildArray.Num()))
			[
				ContentHorizontalBox.ToSharedRef()
			];

		// Make a widget for each property.  The vector component properties  will be displayed in the header

		TSharedRef<IPropertyHandle> ColorGradingPropertyHandleRef = ColorGradingPropertyHandle.GetHandle().ToSharedRef();
		FMathStructCustomization::FNumericMetadata<float> Metadata;
		FMathStructCustomization::ExtractNumericMetadata(ColorGradingPropertyHandleRef, Metadata);

		for (int32 ColorIndex = 0; ColorIndex < SortedChildArray.Num(); ++ColorIndex)
		{
			TWeakPtr<IPropertyHandle> WeakHandlePtr = SortedChildArray[ColorIndex];
			TSharedRef<UE::ColorGrading::SColorGradingComponentViewer> ComponentViewer = MakeComponentViewer(ColorIndex, Metadata.MinValue, Metadata.MaxValue, 
				Metadata.SliderMinValue, Metadata.SliderMaxValue, Metadata.SliderExponent, Metadata.Delta, 
				Metadata.ShiftMultiplier, Metadata.CtrlMultiplier, Metadata.bSupportDynamicSliderMaxValue, Metadata.bSupportDynamicSliderMinValue,
				/*UseCompactDisplay = */true);
			 
			ComponentViewers.Add(ComponentViewer);

			float MinSliderValue = ComponentViewer->GetMinSliderValue();
			float MaxSliderValue = ComponentViewer->GetMaxSliderValue();

			SpinBoxMinMaxSliderValues.CurrentMinSliderValue = MinSliderValue == TNumericLimits<float>::Lowest() ? TOptional<float>() : MinSliderValue;
			SpinBoxMinMaxSliderValues.CurrentMaxSliderValue = MaxSliderValue == TNumericLimits<float>::Max() ? TOptional<float>() : MaxSliderValue;
			SpinBoxMinMaxSliderValues.DefaultMinSliderValue = SpinBoxMinMaxSliderValues.CurrentMinSliderValue;
			SpinBoxMinMaxSliderValues.DefaultMaxSliderValue = SpinBoxMinMaxSliderValues.CurrentMaxSliderValue;

			ContentHorizontalBox->AddSlot()
				.Padding(FMargin((ColorIndex == 0) ? 0.0f : 4.0f, 2.0f, 4.0f, 0.0f))
				.VAlign(VAlign_Top)
				[
					ComponentViewer
				];
		}
	}
	else
	{
		Row.ValueContent()
			.VAlign(VAlign_Center)
			.MinDesiredWidth(125)
			[
				ContentHorizontalBox.ToSharedRef()
			];

		ContentHorizontalBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SBorder)
				.Padding(1)
				.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedSolidBackground"))
				.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.InputOutline"))
				.VAlign(VAlign_Center)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SColorBlock)
						.Color(this, &FColorGradingVectorCustomization::OnGetColorForHeaderColorBlock)
						.ShowBackgroundForAlpha(false)
						.Size(FVector2D(70.0f, 20.0f))
						.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f))
					]
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Visibility(this, &FColorGradingVectorCustomization::GetMultipleValuesTextVisibility)
						.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.MultipleValuesBackground"))
						.VAlign(VAlign_Center)
						.ForegroundColor(FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").ForegroundColor)
						.Padding(FMargin(12.0f, 2.0f))
						[
							SNew(STextBlock)
							.Text(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				]
			];
	}
}

EVisibility FColorGradingVectorCustomization::GetMultipleValuesTextVisibility() const
{
	FVector4 VectorValue;
	return (ColorGradingPropertyHandle.GetValue(VectorValue) == FPropertyAccess::MultipleValues) ? EVisibility::Visible : EVisibility::Collapsed;
}

FLinearColor FColorGradingVectorCustomization::OnGetColorForHeaderColorBlock() const
{
	FLinearColor ColorValue(0.0f, 0.0f, 0.0f);
	FVector4 VectorValue;
	if (ColorGradingPropertyHandle.GetValue(VectorValue) == FPropertyAccess::Success)
	{
		ColorValue.R = static_cast<float>(VectorValue.X * VectorValue.W);
		ColorValue.G = static_cast<float>(VectorValue.Y * VectorValue.W);
		ColorValue.B = static_cast<float>(VectorValue.Z * VectorValue.W);
	}
	else
	{
		ColorValue = FLinearColor::White;
	}

	return ColorValue;
}

void FColorGradingVectorCustomization::OnColorModeChanged(bool InIsRGBMode)
{
	IsRGBMode = InIsRGBMode;
}

void FColorGradingVectorCustomization::CustomizeChildren(IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	ParentGroup = StructBuilder.GetParentGroup();
	CustomColorGradingBuilder = MakeShareable(new FColorGradingCustomBuilder(ColorGradingPropertyHandle, SortedChildArray, StaticCastSharedRef<FColorGradingVectorCustomization>(AsShared()), ParentGroup));

	// Add the individual properties as children as well so the vector can be expanded for more room
	StructBuilder.AddCustomBuilder(CustomColorGradingBuilder.ToSharedRef());

	if (ParentGroup != nullptr)
	{
		TSharedPtr<IDetailPropertyRow> PropertyRow = ParentGroup->FindPropertyRow(ColorGradingPropertyHandle.GetHandle().ToSharedRef());
		verifySlow(PropertyRow.IsValid());

		PropertyRow->OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(CustomColorGradingBuilder.Get(), &FColorGradingCustomBuilder::CanResetToDefault),
			FResetToDefaultHandler::CreateSP(CustomColorGradingBuilder.Get(), &FColorGradingCustomBuilder::ResetToDefault)));
	}
}


//////////////////////////////////////////////////////////////////////////
// Color Gradient custom builder implementation

FColorGradingCustomBuilder::FColorGradingCustomBuilder(const FTrackedVector4PropertyHandle& InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray,
													   TSharedRef<FColorGradingVectorCustomization> InColorGradingCustomization, IDetailGroup* InParentGroup)
	: FColorGradingVectorCustomizationBase(InColorGradingPropertyHandle, InSortedChildArray)
	, ColorGradingCustomization(InColorGradingCustomization)
{
	ParentGroup = InParentGroup;
}

FColorGradingCustomBuilder::~FColorGradingCustomBuilder()
{
	if (ColorGradingCustomization.IsValid())
	{
		OnColorModeChanged.RemoveAll(this);

		ColorGradingCustomization->GetOnCurrentHSVColorChangedDelegate().RemoveAll(this);
		OnCurrentHSVColorChanged.RemoveAll(ColorGradingCustomization.Get());

		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().RemoveAll(this);
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().RemoveAll(this);
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().RemoveAll(ColorGradingPickerWidget.Pin().Get());
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().RemoveAll(ColorGradingPickerWidget.Pin().Get());

		OnNumericEntryBoxDynamicSliderMaxValueChanged.RemoveAll(ColorGradingCustomization.Get());
		OnNumericEntryBoxDynamicSliderMinValueChanged.RemoveAll(ColorGradingCustomization.Get());
	}

	if (ColorGradingPickerWidget.IsValid())
	{
		if (ColorGradingCustomization.IsValid())
		{
			ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().RemoveAll(ColorGradingCustomization.Get());
			ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().RemoveAll(ColorGradingCustomization.Get());
		}

		ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().RemoveAll(this);
		ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().RemoveAll(this);

		OnNumericEntryBoxDynamicSliderMaxValueChanged.RemoveAll(ColorGradingPickerWidget.Pin().Get());
		OnNumericEntryBoxDynamicSliderMinValueChanged.RemoveAll(ColorGradingPickerWidget.Pin().Get());
	}

	if (ParentGroup != nullptr)
	{
		ParentGroup->GetOnDetailGroupReset().RemoveAll(this);
	}

	OnCurrentHSVColorChanged.RemoveAll(this);

	// Deregister for Undo callbacks
	GEditor->UnregisterForUndo(this);

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FColorGradingCustomBuilder::OnDetailGroupReset()
{
	FVector4 CurrentValueVector;
	verifySlow(ColorGradingPropertyHandle.GetValue(CurrentValueVector) == FPropertyAccess::Success);
	CurrentHSVColor = FLinearColor(static_cast<float>(CurrentValueVector.X), static_cast<float>(CurrentValueVector.Y), static_cast<float>(CurrentValueVector.Z)).LinearRGBToHSV();

	OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, true);

	if (SpinBoxMinMaxSliderValues.DefaultMaxSliderValue.IsSet())
	{
		OnDynamicSliderMaxValueChanged(SpinBoxMinMaxSliderValues.DefaultMaxSliderValue.GetValue(), nullptr, true, false);
	}

	if (SpinBoxMinMaxSliderValues.DefaultMinSliderValue.IsSet())
	{
		OnDynamicSliderMinValueChanged(SpinBoxMinMaxSliderValues.DefaultMinSliderValue.GetValue(), nullptr, true, false);
	}
}

void FColorGradingCustomBuilder::ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->ResetToDefault();

	FVector4 CurrentValueVector;
	verifySlow(ColorGradingPropertyHandle.GetValue(CurrentValueVector) == FPropertyAccess::Success);
	CurrentHSVColor = FLinearColor(static_cast<float>(CurrentValueVector.X), static_cast<float>(CurrentValueVector.Y), static_cast<float>(CurrentValueVector.Z)).LinearRGBToHSV();

	OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, true);

	if (SpinBoxMinMaxSliderValues.DefaultMaxSliderValue.IsSet())
	{
		OnDynamicSliderMaxValueChanged(SpinBoxMinMaxSliderValues.DefaultMaxSliderValue.GetValue(), nullptr, true, false);
	}

	if (SpinBoxMinMaxSliderValues.DefaultMinSliderValue.IsSet())
	{
		OnDynamicSliderMinValueChanged(SpinBoxMinMaxSliderValues.DefaultMinSliderValue.GetValue(), nullptr, true, false);
	}
}

bool FColorGradingCustomBuilder::CanResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	return PropertyHandle->DiffersFromDefault();
}

void FColorGradingCustomBuilder::Tick(float DeltaTime)
{
}

void FColorGradingCustomBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	// Make a widget for each property.  The vector component properties  will be displayed in the header

	TSharedRef<IPropertyHandle> ColorGradingPropertyHandleRef = ColorGradingPropertyHandle.GetHandle().ToSharedRef();
	FMathStructCustomization::FNumericMetadata<float> Metadata;
	FMathStructCustomization::ExtractNumericMetadata(ColorGradingPropertyHandleRef, Metadata);
	
	UE::ColorGrading::EColorGradingModes ColorGradingMode = GetColorGradingMode();

	// Add padding on right to compensate for the hidden arrow, which pushes the wheel off-center
	float RightPadding = 0.f;
	if (const FSlateBrush* ArrowBrush = FAppStyle::Get().GetBrush("TreeArrow_Expanded"))
	{
		RightPadding = ArrowBrush->GetImageSize().X;
	}

	NodeRow.NameContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SBox)
		.Padding(FMargin(0.0f, 8.0f, RightPadding, 8.0f))
		.HAlign(HAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(175.0f)
			.HeightOverride(150.0f)
			[
				SAssignNew(ColorGradingPickerWidget, UE::ColorGrading::SColorGradingPicker)
				.ValueMin(Metadata.MinValue)
				.ValueMax(Metadata.MaxValue)
				.SliderValueMin(Metadata.SliderMinValue)
				.SliderValueMax(Metadata.SliderMaxValue)
				.MainDelta(Metadata.Delta)
				.SupportDynamicSliderMaxValue(Metadata.bSupportDynamicSliderMaxValue)
				.SupportDynamicSliderMinValue(Metadata.bSupportDynamicSliderMinValue)
				.MainShiftMultiplier(Metadata.ShiftMultiplier)
				.MainCtrlMultiplier(Metadata.CtrlMultiplier)
				.ColorGradingModes(ColorGradingMode)
				.OnColorCommitted(this, &FColorGradingCustomBuilder::OnColorGradingPickerChanged)
				.OnQueryCurrentColor(this, &FColorGradingCustomBuilder::GetCurrentColorGradingValue)
				.AllowSpin(ColorGradingPropertyHandle.GetHandle()->GetNumOuterObjects() == 1)
				.OnBeginSliderMovement(this, &FColorGradingCustomBuilder::OnBeginMainValueSliderMovement)
				.OnEndSliderMovement(this, &FColorGradingCustomBuilder::OnEndMainValueSliderMovement)
				.OnBeginMouseCapture(this, &FColorGradingCustomBuilder::OnBeginMouseCapture)
				.OnEndMouseCapture(this, &FColorGradingCustomBuilder::OnEndMouseCapture)
			]
		]
	];

	TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	using SDisplayModeControl = SSegmentedControl<UE::ColorGrading::EColorGradingColorDisplayMode>;

	VerticalBox->AddSlot()
	.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Left)
	.AutoHeight()
	[
		SNew(SDisplayModeControl)
		.OnValueChanged(this, &FColorGradingCustomBuilder::OnChangeColorModeClicked)
		.Value(this, &FColorGradingCustomBuilder::OnGetChangeColorMode)
		.UniformPadding(FMargin(16.0f, 2.0f))

		+ SDisplayModeControl::Slot(UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		.Text(this, &FColorGradingCustomBuilder::OnChangeColorModeText, UE::ColorGrading::EColorGradingColorDisplayMode::RGB)
		.ToolTip(this, &FColorGradingCustomBuilder::OnChangeColorModeToolTipText, UE::ColorGrading::EColorGradingColorDisplayMode::RGB)

		+ SDisplayModeControl::Slot(UE::ColorGrading::EColorGradingColorDisplayMode::HSV)
		.Text(this, &FColorGradingCustomBuilder::OnChangeColorModeText, UE::ColorGrading::EColorGradingColorDisplayMode::HSV)
		.ToolTip(this, &FColorGradingCustomBuilder::OnChangeColorModeToolTipText, UE::ColorGrading::EColorGradingColorDisplayMode::HSV)
	];

	for (int32 ColorIndex = 0; ColorIndex < SortedChildArray.Num(); ++ColorIndex)
	{
		TWeakPtr<IPropertyHandle> WeakHandlePtr = SortedChildArray[ColorIndex];

		TSharedRef<UE::ColorGrading::SColorGradingComponentViewer> ComponentViewer = MakeComponentViewer(ColorIndex, Metadata.MinValue, Metadata.MaxValue,
			Metadata.SliderMinValue, Metadata.SliderMaxValue, Metadata.SliderExponent, Metadata.Delta,
			Metadata.ShiftMultiplier, Metadata.CtrlMultiplier, Metadata.bSupportDynamicSliderMaxValue, Metadata.bSupportDynamicSliderMinValue,
			/*UseCompactDisplay = */false);

		ComponentViewers.Add(ComponentViewer);

		float MinSliderValue = ComponentViewer->GetMinSliderValue();
		float MaxSliderValue = ComponentViewer->GetMaxSliderValue();

		SpinBoxMinMaxSliderValues.CurrentMinSliderValue = MinSliderValue == TNumericLimits<float>::Lowest() ? TOptional<float>() : MinSliderValue;
		SpinBoxMinMaxSliderValues.CurrentMaxSliderValue = MaxSliderValue == TNumericLimits<float>::Max() ? TOptional<float>() : MaxSliderValue;
		SpinBoxMinMaxSliderValues.DefaultMinSliderValue = SpinBoxMinMaxSliderValues.CurrentMinSliderValue;
		SpinBoxMinMaxSliderValues.DefaultMaxSliderValue = SpinBoxMinMaxSliderValues.CurrentMaxSliderValue;

		VerticalBox->AddSlot()
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				ComponentViewer
			];
	}	
	
	NodeRow.ValueContent()
		.HAlign(HAlign_Fill)
	[
		VerticalBox.ToSharedRef()
	];

	if (ParentGroup)
	{
		ParentGroup->GetOnDetailGroupReset().AddSP(this, &FColorGradingCustomBuilder::OnDetailGroupReset);
	}

	if (ColorGradingCustomization.IsValid())
	{
		OnColorModeChanged.AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomization::OnColorModeChanged);

		OnCurrentHSVColorChanged.AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomizationBase::OnCurrentHSVColorChangedDelegate);
		ColorGradingCustomization->GetOnCurrentHSVColorChangedDelegate().AddSP(this, &FColorGradingVectorCustomizationBase::OnCurrentHSVColorChangedDelegate);

		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().AddSP(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged);
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().AddSP(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged);
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().AddSP(ColorGradingPickerWidget.Pin().Get(), &UE::ColorGrading::SColorGradingPicker::OnDynamicSliderMaxValueChanged);
		ColorGradingCustomization->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().AddSP(ColorGradingPickerWidget.Pin().Get(), &UE::ColorGrading::SColorGradingPicker::OnDynamicSliderMinValueChanged);

		OnNumericEntryBoxDynamicSliderMaxValueChanged.AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged);
		OnNumericEntryBoxDynamicSliderMinValueChanged.AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged);
	}

	if (ColorGradingPickerWidget.IsValid())
	{
		if (ColorGradingCustomization.IsValid())
		{
			ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged);
			ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().AddSP(ColorGradingCustomization.Get(), &FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged);
		}

		ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate().AddSP(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMaxValueChanged);
		ColorGradingPickerWidget.Pin()->GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate().AddSP(this, &FColorGradingVectorCustomizationBase::OnDynamicSliderMinValueChanged);

		OnNumericEntryBoxDynamicSliderMaxValueChanged.AddSP(ColorGradingPickerWidget.Pin().Get(), &UE::ColorGrading::SColorGradingPicker::OnDynamicSliderMaxValueChanged);
		OnNumericEntryBoxDynamicSliderMinValueChanged.AddSP(ColorGradingPickerWidget.Pin().Get(), &UE::ColorGrading::SColorGradingPicker::OnDynamicSliderMinValueChanged);
	}

	OnCurrentHSVColorChanged.AddSP(this, &FColorGradingVectorCustomizationBase::OnCurrentHSVColorChangedDelegate);

	bool RGBMode = true;

	// Find the highest current value and propagate it to all others so they all matches
	float BestMaxSliderValue = 0.0f;
	float BestMinSliderValue = 0.0f;

	for (TWeakPtr<UE::ColorGrading::SColorGradingComponentViewer>& ComponentViewer : ComponentViewers)
	{
		if (TSharedPtr<UE::ColorGrading::SColorGradingComponentViewer> PinnedComponentViewer = ComponentViewer.Pin())
		{
			if (PinnedComponentViewer->GetMaxSliderValue() > BestMaxSliderValue)
			{
				BestMaxSliderValue = PinnedComponentViewer->GetMaxSliderValue();
			}

			if (PinnedComponentViewer->GetMinSliderValue() < BestMinSliderValue)
			{
				BestMinSliderValue = PinnedComponentViewer->GetMinSliderValue();
			}
		}
	}

	OnDynamicSliderMaxValueChanged(BestMaxSliderValue, nullptr, true, true);
	OnDynamicSliderMinValueChanged(BestMinSliderValue, nullptr, true, true);
	
	FString ParentGroupName = ParentGroup != nullptr ? ParentGroup->GetGroupName().ToString() : TEXT("NoParentGroup");
	ParentGroupName.ReplaceInline(TEXT(" "), TEXT("_"));
	ParentGroupName.ReplaceInline(TEXT("|"), TEXT("_"));

	GConfig->GetBool(TEXT("ColorGrading"), *FString::Printf(TEXT("%s_%s_IsRGB"), *ParentGroupName, *ColorGradingPropertyHandle.GetHandle()->GetPropertyDisplayName().ToString()), RGBMode, GEditorPerProjectIni);
	OnChangeColorModeClicked(RGBMode ? UE::ColorGrading::EColorGradingColorDisplayMode::RGB : UE::ColorGrading::EColorGradingColorDisplayMode::HSV);

	// Register to update when an undo/redo operation has been called to update our list of actors
	GEditor->RegisterForUndo(this);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FColorGradingCustomBuilder::OnPropertyValueChanged);
}

FText FColorGradingCustomBuilder::OnChangeColorModeText(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const
{
	FText Text;

	switch (ModeType)
	{
		case UE::ColorGrading::EColorGradingColorDisplayMode::RGB: Text = LOCTEXT("ChangeColorModeRGB", "RGB"); break;
		case UE::ColorGrading::EColorGradingColorDisplayMode::HSV: Text = LOCTEXT("ChangeColorModeHSV", "HSV"); break;
	}

	return Text;
}

FText FColorGradingCustomBuilder::OnChangeColorModeToolTipText(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const
{
	FText Text;

	switch (ModeType)
	{
		case UE::ColorGrading::EColorGradingColorDisplayMode::RGB: Text = LOCTEXT("ChangeColorModeRGBToolTips", "Change to RGB color mode"); break;
		case UE::ColorGrading::EColorGradingColorDisplayMode::HSV: Text = LOCTEXT("ChangeColorModeHSVToolTips", "Change to HSV color mode"); break;
	}

	return Text;
}

EVisibility FColorGradingCustomBuilder::OnGetRGBHSVButtonVisibility(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const
{
	return GetColorGradingMode() == UE::ColorGrading::EColorGradingModes::Offset ? EVisibility::Hidden : EVisibility::Visible;
}

EVisibility FColorGradingCustomBuilder::OnGetGradientVisibility() const
{
	return GetColorGradingMode() == UE::ColorGrading::EColorGradingModes::Offset ? EVisibility::Hidden : EVisibility::Visible;
}

void FColorGradingCustomBuilder::OnChangeColorModeClicked(UE::ColorGrading::EColorGradingColorDisplayMode ModeType)
{
	FVector4 CurrentValueVector;
	if (ColorGradingPropertyHandle.GetValue(CurrentValueVector) != FPropertyAccess::Success)
	{
		return;
	}

	bool NewIsRGBMode = true;

	NewIsRGBMode = ModeType == UE::ColorGrading::EColorGradingColorDisplayMode::RGB;

	if (NewIsRGBMode != IsRGBMode)
	{
		IsRGBMode = NewIsRGBMode;

		FString ParentGroupName = ParentGroup != nullptr ? ParentGroup->GetGroupName().ToString() : TEXT("NoParentGroup");
		ParentGroupName.ReplaceInline(TEXT(" "), TEXT("_"));
		ParentGroupName.ReplaceInline(TEXT("|"), TEXT("_"));

		GConfig->SetBool(TEXT("ColorGrading"), *FString::Printf(TEXT("%s_%s_IsRGB"), *ParentGroupName, *ColorGradingPropertyHandle.GetHandle()->GetPropertyDisplayName().ToString()), IsRGBMode, GEditorPerProjectIni);

		CurrentHSVColor = FLinearColor(static_cast<float>(CurrentValueVector.X), static_cast<float>(CurrentValueVector.Y), static_cast<float>(CurrentValueVector.Z)).LinearRGBToHSV();

		OnCurrentHSVColorChanged.Broadcast(CurrentHSVColor, true);

		OnColorModeChanged.Broadcast(IsRGBMode);
	}
}

UE::ColorGrading::EColorGradingColorDisplayMode FColorGradingCustomBuilder::OnGetChangeColorMode() const
{
	return IsRGBMode ? UE::ColorGrading::EColorGradingColorDisplayMode::RGB : UE::ColorGrading::EColorGradingColorDisplayMode::HSV;
}

void FColorGradingCustomBuilder::OnColorGradingPickerChanged(FVector4& NewValue, bool ShouldCommitValueChanges)
{
	FScopedTransaction Transaction(LOCTEXT("ColorGradingMainValue", "Color Grading Main Value"), ShouldCommitValueChanges);

	if (ColorGradingPropertyHandle.IsValidHandle())
	{
		// Always perform a purely interactive change. We do this because it won't invoke reconstruction, which may cause that only the first 
		// element gets updated due to its change causing a component reconstruction and the remaining vector element property handles updating 
		// the trashed component.
		ColorGradingPropertyHandle.SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);

		// If not purely interactive, set the value with default flags.
		if (ShouldCommitValueChanges || !bIsUsingSlider)
		{
			ColorGradingPropertyHandle.SetValue(NewValue, EPropertyValueSetFlags::DefaultFlags);
		}
	}

	FLinearColor NewHSVColor(static_cast<float>(NewValue.X), static_cast<float>(NewValue.Y), static_cast<float>(NewValue.Z));
	NewHSVColor = NewHSVColor.LinearRGBToHSV();

	OnCurrentHSVColorChangedDelegate(NewHSVColor, true);
}


void FColorGradingCustomBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
}

void FColorGradingCustomBuilder::OnBeginMainValueSliderMovement()
{
	bIsUsingSlider = true;
	GEditor->BeginTransaction(LOCTEXT("ColorGradingMainValue", "Color Grading Main Value"));
}

void FColorGradingCustomBuilder::OnEndMainValueSliderMovement()
{
	bIsUsingSlider = false;
	GEditor->EndTransaction();
}

void FColorGradingCustomBuilder::OnBeginMouseCapture()
{
	bIsUsingSlider = true;
	GEditor->BeginTransaction(LOCTEXT("ColorGradingMainValue", "Color Grading Main Value"));
}

void FColorGradingCustomBuilder::OnEndMouseCapture()
{
	bIsUsingSlider = false;
	GEditor->EndTransaction();
}

void FColorGradingCustomBuilder::OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ColorGradingPropertyHandle.IsSettingValue())
	{
		// If setting our own value, it's already handled (or will be)
		return;
	}

	if (TSharedPtr<IPropertyHandle> PinnedPropertyHandle = ColorGradingPropertyHandle.GetHandle())
	{
		uint32 NumChildren;
		if (PinnedPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Result::Success)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				if (PinnedPropertyHandle->GetChildHandle(ChildIndex)->GetProperty() == PropertyChangedEvent.Property)
				{
					FVector4 CurrentValueVector;
					if (PinnedPropertyHandle->GetValue(CurrentValueVector) == FPropertyAccess::Success)
					{
						CurrentHSVColor = FLinearColor(static_cast<float>(CurrentValueVector.X), static_cast<float>(CurrentValueVector.Y), static_cast<float>(CurrentValueVector.Z)).LinearRGBToHSV();
					}
					break;
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE