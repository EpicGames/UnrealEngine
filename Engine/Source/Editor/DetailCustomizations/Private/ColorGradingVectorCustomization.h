// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "Framework/ColorGrading/ColorGradingCommon.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector4.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Util/TrackedVector4PropertyHandle.h"

class FDetailWidgetRow;
class FVector4StructCustomization;
class IDetailChildrenBuilder;
class IDetailGroup;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;
class SWidget;

namespace UE::ColorGrading
{
class SColorGradingPicker;
class SColorGradingComponentViewer;
enum class EColorGradingModes;
}

struct FColorGradingMinMaxSliderValue
{
	TOptional<float> CurrentMaxSliderValue;
	TOptional<float> CurrentMinSliderValue;
	TOptional<float> DefaultMaxSliderValue;
	TOptional<float> DefaultMinSliderValue;
};

class FColorGradingVectorCustomizationBase : public TSharedFromThis<FColorGradingVectorCustomizationBase>, public FEditorUndoClient
{
public:
	/** Notification when the max/min slider values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnNumericEntryBoxDynamicSliderMinMaxValueChanged, float, TWeakPtr<SWidget>, bool, bool);
	
	/** Notification the current HSV color was changed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCurrentHSVColorChanged, FLinearColor, bool);

	FColorGradingVectorCustomizationBase(const FTrackedVector4PropertyHandle& InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray);

	/** Return max/min slider value changed delegate (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMaxValueChanged; }
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMinValueChanged; }

	/** Return the current HSV color was changed delegate */
	FOnCurrentHSVColorChanged& GetOnCurrentHSVColorChangedDelegate() { return OnCurrentHSVColorChanged; }

	/** Callback when the max/min slider value are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	void OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher);
	void OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower);

	/** Callback returning the current slider value for a specified color index */
	TOptional<float> OnSliderGetValue(int32 ColorIndex) const;
	
	/** Callback handling HSV color changed */
	void OnCurrentHSVColorChangedDelegate(FLinearColor NewHSVColor, bool Originator);

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient	

protected:
	bool IsInRGBMode() const;
	UE::ColorGrading::EColorGradingModes GetColorGradingMode() const;

	TSharedRef<UE::ColorGrading::SColorGradingComponentViewer> MakeComponentViewer(int32 ColorIndex, TOptional<float>& MinValue, TOptional<float>& MaxValue, TOptional<float>& SliderMinValue, TOptional<float>& SliderMaxValue, float& SliderExponent, float& Delta, float& ShiftMultiplier, float &CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue, bool UseCompactDisplay);

	/** Callback returning the component associated with a specified color index */
	UE::ColorGrading::EColorGradingComponent OnGetColorComponent(int32 ColorIndex) const;

	/** Get the current color value being edited/displayed by this widget */
	bool GetCurrentColorGradingValue(FVector4& OutCurrentValue);
	
	/** Callback returning the min/max slider value for a specified color index */
	TOptional<float> OnGetMaxSliderValue(TOptional<float> DefaultMaxSliderValue, int32 ColorIndex) const;
	TOptional<float> OnGetMinSliderValue(TOptional<float> DefaultMinSliderValue, int32 ColorIndex) const;
	
	/** Callback returning the delta slider value for a specified color index */
	float OnGetSliderDeltaValue(float DefaultValue, int32 ColorIndex) const;
	
	/** Callback returning the max value for a specified color index */
	TOptional<float> OnGetMaxValue(TOptional<float> DefaultValue, int32 ColorIndex) const;
	
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue, int32 ColorIndex);

	/** Callback called when a slider value changed or the user typed into the text box*/
	void OnValueChanged(float NewValue, int32 ColorIndex);

	/** Callback returning if we support dynamic max/min slider value */
	bool GetSupportDynamicSliderMaxValue(bool DefaultValue, int32 ColorIndex) const;
	bool GetSupportDynamicSliderMinValue(bool DefaultValue, int32 ColorIndex) const;

	/** Callback returning if an entry box should be enabled */
	bool IsEntryBoxEnabled(int32 ColorIndex) const;

protected:
	/** Min/Max slider value that can change dynamically */
	FColorGradingMinMaxSliderValue SpinBoxMinMaxSliderValues;
	
	/** List of registered color component viewers */
	TArray<TWeakPtr<UE::ColorGrading::SColorGradingComponentViewer>> ComponentViewers;
	
	/** The color grading property we're editing */
	FTrackedVector4PropertyHandle ColorGradingPropertyHandle;
	
	/** Property for each color value (RGBY) */
	TArray<TWeakPtr<IPropertyHandle>> SortedChildArray;

	/** Tell us if we are in RGB mode or HSV */
	bool IsRGBMode;
	
	/** Represent the current HSV color. This is seperate as we store the value in FVector4 RGB format 
	 *  so during conversion value can be lost, so during editing we always use this variable in HSV mode 
	 */
	FLinearColor CurrentHSVColor;

	/** Callback when the max/min slider value are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMaxValueChanged;
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMinValueChanged;
	
	/** Return the current HSV color was changed delegate */
	FOnCurrentHSVColorChanged OnCurrentHSVColorChanged;

	/** Parent group in the property panel */
	IDetailGroup* ParentGroup;

	/** Whether or not the slider is actively being used */
	bool bIsUsingSlider;
};

class FColorGradingVectorCustomization : public FColorGradingVectorCustomizationBase
{
public:
	FColorGradingVectorCustomization(TWeakPtr<IPropertyHandle> InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray);
	virtual ~FColorGradingVectorCustomization();
	
	void MakeHeaderRow(FDetailWidgetRow& Row, TSharedRef<FVector4StructCustomization> InVector4Customization);
	void CustomizeChildren(IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils);

	/** Callback when the ColorMode changed */
	void OnColorModeChanged(bool InIsRGBMode);	

private:
	/** Will return the color of the color block displayed in the header */
	FLinearColor OnGetColorForHeaderColorBlock() const;

	EVisibility GetMultipleValuesTextVisibility() const;

	/** Represent the custom builder associated with the color grading property */
	TSharedPtr<class FColorGradingCustomBuilder> CustomColorGradingBuilder;
};

class FColorGradingCustomBuilder : public IDetailCustomNodeBuilder, public FColorGradingVectorCustomizationBase
{
public:
	/** Notification when we change color mode (RGB <-> HSV) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnColorModeChanged, bool);

	FColorGradingCustomBuilder(const FTrackedVector4PropertyHandle& InColorGradingPropertyHandle, const TArray<TWeakPtr<IPropertyHandle>>& InSortedChildArray, 
							   TSharedRef<FColorGradingVectorCustomization> InColorGradingCustomization, IDetailGroup* InParentGroup);
	virtual ~FColorGradingCustomBuilder();

	/** Delegate to register for notification when we change color mode (RGB <-> HSV) */
	FOnColorModeChanged& GetOnColorModeChanged() { return OnColorModeChanged; }

	/** Callback when user click the reset button of the color grading property */
	void ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);
	bool CanResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);

private:

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { OnRebuildChildren = InOnRebuildChildren; }
	virtual bool RequiresTick() const override { return false; }
	virtual void Tick(float DeltaTime) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

	/* Local UI Handlers */
	void OnColorGradingPickerChanged(FVector4 &NewValue, bool ShouldCommitValueChanges);

	void OnBeginMainValueSliderMovement();
	void OnEndMainValueSliderMovement();

	void OnBeginMouseCapture();
	void OnEndMouseCapture();

	/** Called when any property changes */
	void OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	/** Callback when user click the Group reset button */
	void OnDetailGroupReset();

	/** Callback returning which color mode text we should display */
	FText OnChangeColorModeText(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const;
	
	/** Callback returning which color mode text tooltip we should display */
	FText OnChangeColorModeToolTipText(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const;

	/** Callback returning which color mode is checked */	
	UE::ColorGrading::EColorGradingColorDisplayMode OnGetChangeColorMode() const;

	/** Callback returning if the RGB/HSV button should be visible */
	EVisibility OnGetRGBHSVButtonVisibility(UE::ColorGrading::EColorGradingColorDisplayMode ModeType) const;

	/** Callback returning if the Gradient should be visible */
	EVisibility OnGetGradientVisibility() const;
	
	/** Callback Called when user click a color mode change checkbox */
	void OnChangeColorModeClicked(UE::ColorGrading::EColorGradingColorDisplayMode ModeType);

	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Color Picker widget */
	TWeakPtr<UE::ColorGrading::SColorGradingPicker> ColorGradingPickerWidget;

	/** Parent of this custom builder (required to communicate with FColorGradingVectorCustomization) */
	TSharedPtr<FColorGradingVectorCustomization> ColorGradingCustomization;
	
	/** Delegate to register for notification when we change color mode (RGB <-> HSV) */
	FOnColorModeChanged OnColorModeChanged;
};