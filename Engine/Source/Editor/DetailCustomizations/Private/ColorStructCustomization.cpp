// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/ColorStructCustomization.h"

#include "Customizations/MathStructCustomizations.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;


#define LOCTEXT_NAMESPACE "FColorStructCustomization"

namespace UE::Editor::DetailCustomizations
{
	bool IsPropertyInBlueprint(const TSharedPtr<IPropertyHandle>& StructHandle)
	{
		if (!StructHandle)
		{
			return false;
		}

		TArray<UObject*> OuterObjects;
		StructHandle->GetOuterObjects(OuterObjects);

		while (!OuterObjects.IsEmpty())
		{
			const UObject* Object = OuterObjects.Pop();
			if (!Object)
			{
				continue;
			}

			const UClass* Class = Object->GetClass();
			if (Class && Class->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				return true;
			}
		
			if (UObject* Outer = Object->GetOuter())
			{
				OuterObjects.Add(Outer);
			}
		}

		return false;
	};
}


TSharedRef<IPropertyTypeCustomization> FColorStructCustomization::MakeInstance() 
{
	return MakeShareable(new FColorStructCustomization);
}


void FColorStructCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	bIsInBlueprint = UE::Editor::DetailCustomizations::IsPropertyInBlueprint(InStructPropertyHandle);
	bIsLinearColor = CastFieldChecked<FStructProperty>(StructPropertyHandle->GetProperty())->Struct->GetFName() == NAME_LinearColor;
	bIgnoreAlpha = TypeSupportsAlpha() == false || StructPropertyHandle->GetProperty()->HasMetaData(TEXT("HideAlphaChannel"));
	
	if (StructPropertyHandle->GetProperty()->HasMetaData(TEXT("sRGB")))
	{
		sRGBOverride = StructPropertyHandle->GetProperty()->GetBoolMetaData(TEXT("sRGB"));
	}

	auto PropertyUtils = StructCustomizationUtils.GetPropertyUtilities();
	bDontUpdateWhileEditing = PropertyUtils.IsValid() ? PropertyUtils->DontUpdateValueWhileEditing() : false;

	FMathStructCustomization::CustomizeHeader(InStructPropertyHandle, InHeaderRow, StructCustomizationUtils);
}


void FColorStructCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row)
{
	TSharedPtr<SWidget> ColorWidget;
	float ContentWidth = 125.0f;

	TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;

	if (InStructPropertyHandle->HasMetaData("InlineColorPicker"))
	{
		ColorWidget = CreateInlineColorPicker(StructWeakHandlePtr);
		ContentWidth = 492.0f;
	}
	else
	{
		ColorWidget = CreateColorWidget(StructWeakHandlePtr);
	}

	Row.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(ContentWidth)
	[
		ColorWidget.ToSharedRef()
	];
}


FColorStructCustomization::~FColorStructCustomization()
{
	if (TransactionIndex.IsSet())
	{
		GEditor->EndTransaction();
	}
}

TSharedRef<SWidget> FColorStructCustomization::CreateColorWidget(TWeakPtr<IPropertyHandle> StructWeakHandlePtr)
{
	return
		SNew(SBox)
		.Padding(FMargin(0,0,4.0f,0.0f))
		.VAlign(VAlign_Center)
		[
			SAssignNew(ColorWidgetBackgroundBorder, SBorder)
			.Padding(1)
			.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedSolidBackground"))
			.BorderBackgroundColor(this, &FColorStructCustomization::GetColorWidgetBorderColor)
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				[
					SAssignNew(ColorPickerParentWidget, SColorBlock)
					.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
					.Color(this, &FColorStructCustomization::OnGetColorForColorBlock)
					.ShowBackgroundForAlpha(true)
					.AlphaDisplayMode(bIgnoreAlpha ? EColorBlockAlphaDisplayMode::Ignore : EColorBlockAlphaDisplayMode::Separate)
					.OnMouseButtonDown(this, &FColorStructCustomization::OnMouseButtonDownColorBlock)
					.Size(FVector2D(70.0f, 20.0f))
					.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f))
					.IsEnabled(this, &FColorStructCustomization::IsValueEnabled, StructWeakHandlePtr)
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.Visibility(this, &FColorStructCustomization::GetMultipleValuesTextVisibility)
					.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.MultipleValuesBackground"))
					.VAlign(VAlign_Center)
					.ForegroundColor(FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").ForegroundColor)
					.Padding(FMargin(12.0f, 2.0f))
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			]
		];
	}


void FColorStructCustomization::GetSortedChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren)
{
	static const FName Red("R");
	static const FName Green("G");
	static const FName Blue("B");
	static const FName Alpha("A");

	// We control the order of the colors via this array so it always ends up R,G,B,A
	TSharedPtr< IPropertyHandle > ColorProperties[4];

	uint32 NumChildren;
	InStructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == Red)
		{
			ColorProperties[0] = ChildHandle;
		}
		else if (PropertyName == Green)
		{
			ColorProperties[1] = ChildHandle;
		}
		else if (PropertyName == Blue)
		{
			ColorProperties[2] = ChildHandle;
		}
		else
		{
			ColorProperties[3] = ChildHandle;
		}
	}

	OutChildren.Add(ColorProperties[0].ToSharedRef());
	OutChildren.Add(ColorProperties[1].ToSharedRef());
	OutChildren.Add(ColorProperties[2].ToSharedRef());

	// Alpha channel may not be used
	if (!bIgnoreAlpha && ColorProperties[3].IsValid())
	{
		OutChildren.Add(ColorProperties[3].ToSharedRef());
	}
}

void FColorStructCustomization::GatherSavedPreColorPickerColors()
{
	SavedPreColorPickerColors.Empty();
	TArray<FString> PerObjectValues;
	StructPropertyHandle->GetPerObjectValues(PerObjectValues);

	for (int32 ObjectIndex = 0; ObjectIndex < PerObjectValues.Num(); ++ObjectIndex)
	{
		if (bIsLinearColor)
		{
			FLinearColor Color;
			Color.InitFromString(PerObjectValues[ObjectIndex]);
			SavedPreColorPickerColors.Add(FLinearOrSrgbColor(Color));
		}
		else
		{
			FColor Color;
			Color.InitFromString(PerObjectValues[ObjectIndex]);
			SavedPreColorPickerColors.Add(FLinearOrSrgbColor(Color));
		}
	}
}

void FColorStructCustomization::CreateColorPicker(bool bUseAlpha)
{
	TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("SetColorProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));

	GatherSavedPreColorPickerColors();

	FLinearColor InitialColor;
	GetColorAsLinear(InitialColor);

	const bool bRefreshOnlyOnOk = bDontUpdateWhileEditing || StructPropertyHandle->HasMetaData("DontUpdateWhileEditing");
	const bool bOnlyRefreshOnMouseUp = StructPropertyHandle->HasMetaData("OnlyUpdateOnInteractionEnd");

	FColorPickerArgs PickerArgs;
	{
		PickerArgs.bUseAlpha = !bIgnoreAlpha;
		PickerArgs.bOnlyRefreshOnMouseUp = bOnlyRefreshOnMouseUp;
		PickerArgs.bOnlyRefreshOnOk = bRefreshOnlyOnOk;
		PickerArgs.sRGBOverride = sRGBOverride;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FColorStructCustomization::OnSetColorFromColorPicker);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FColorStructCustomization::OnColorPickerCancelled);
		PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &FColorStructCustomization::OnColorPickerWindowClosed);
		PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &FColorStructCustomization::OnColorPickerInteractiveBegin);
		PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &FColorStructCustomization::OnColorPickerInteractiveEnd);
		PickerArgs.InitialColor = InitialColor;
		PickerArgs.ParentWidget = ColorPickerParentWidget;
		PickerArgs.OptionalOwningDetailsView = ColorPickerParentWidget;
		FWidgetPath ParentWidgetPath;
		if (FSlateApplication::Get().FindPathToWidget(ColorPickerParentWidget.ToSharedRef(), ParentWidgetPath))
		{
			PickerArgs.bOpenAsMenu = FSlateApplication::Get().FindMenuInWidgetPath(ParentWidgetPath).IsValid();
		}
	}

	OpenColorPicker(PickerArgs);
	
	// Close the color picker when a property is edited on the same blueprint actor 
	// since it will trigger a rerun construction script and duplicate actor and its subobjects
	// and invalidate property handle since details panel refreshes with objects replaced
	if (!FCoreUObjectDelegates::OnObjectPropertyChanged.IsBoundToObject(this))
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddSPLambda(this, [this](UObject* InObject, FPropertyChangedEvent& InEvent)
			{
				const TSharedPtr<SColorPicker> ColorPicker = GetColorPicker();
				if (StructPropertyHandle.IsValid()
					&& ColorPicker.IsValid()
					&& bIsInBlueprint 
					&& InEvent.ChangeType != EPropertyChangeType::Interactive
					&& InEvent.Property != StructPropertyHandle->GetProperty())
				{
					TArray<UObject*> OuterObjects;
					StructPropertyHandle->GetOuterObjects(OuterObjects);
					
					const UObject* RootOwner = InObject->IsA<AActor>() ? InObject : InObject->GetTypedOuter<AActor>();
					for (const UObject* Object : OuterObjects)
					{
						if (Object == RootOwner || Object->GetTypedOuter<AActor>() == RootOwner)
						{
							DestroyColorPicker();
							return;
						}
					}
				}
			}
		);
	}
}


TSharedRef<SColorPicker> FColorStructCustomization::CreateInlineColorPicker(TWeakPtr<IPropertyHandle> StructWeakHandlePtr)
{
	TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("SetColorProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));

	GatherSavedPreColorPickerColors();

	FLinearColor InitialColor;
	GetColorAsLinear(InitialColor);

	const bool bRefreshOnlyOnOk = /*bDontUpdateWhileEditing ||*/ StructPropertyHandle->HasMetaData("DontUpdateWhileEditing");
	const bool bOnlyRefreshOnMouseUp = StructPropertyHandle->HasMetaData("OnlyUpdateOnInteractionEnd");

	return SNew(SColorPicker)
		.DisplayInlineVersion(true)
		.OnlyRefreshOnMouseUp(bOnlyRefreshOnMouseUp)
		.OnlyRefreshOnOk(bRefreshOnlyOnOk)
		.DisplayGamma(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma)))
		.OnColorCommitted(FOnLinearColorValueChanged::CreateSP(this, &FColorStructCustomization::OnSetColorFromColorPicker))
		.OnColorPickerCancelled(FOnColorPickerCancelled::CreateSP(this, &FColorStructCustomization::OnColorPickerCancelled))
		.OnColorPickerWindowClosed(FOnWindowClosed::CreateSP(this, &FColorStructCustomization::OnColorPickerWindowClosed))
		.OnInteractivePickBegin(FSimpleDelegate::CreateSP(this, &FColorStructCustomization::OnColorPickerInteractiveBegin))
		.OnInteractivePickEnd(FSimpleDelegate::CreateSP(this, &FColorStructCustomization::OnColorPickerInteractiveEnd))
		.sRGBOverride(sRGBOverride)
		.TargetColorAttribute(InitialColor)
		.IsEnabled(this, &FColorStructCustomization::IsValueEnabled, StructWeakHandlePtr);
}

void FColorStructCustomization::SetLastPickerColorString(const FLinearColor NewColor)
{
	if (bIsLinearColor)
	{
		LastPickerColorString = NewColor.ToString();
	}
	else
	{
		const bool bSRGB = true;
		FColor NewFColor = NewColor.ToFColor(bSRGB);
		LastPickerColorString = NewFColor.ToString();
	}
}

void FColorStructCustomization::OnSetColorFromColorPicker(FLinearColor NewColor)
{
	SetLastPickerColorString(NewColor);
	EPropertyValueSetFlags::Type PropertyFlags = EPropertyValueSetFlags::NotTransactable;
	PropertyFlags |= (bIsInteractive || bIsInBlueprint) ? EPropertyValueSetFlags::InteractiveChange : 0;
	StructPropertyHandle->SetValueFromFormattedString(LastPickerColorString, PropertyFlags);
}

TArray<FString> FColorStructCustomization::ConvertToPerObjectColors(const TArray<FLinearOrSrgbColor>& Colors) const
{
	TArray<FString> PerObjectColors;

	for (int32 ColorIndex = 0; ColorIndex < SavedPreColorPickerColors.Num(); ++ColorIndex)
	{
		if (bIsLinearColor)
		{
			PerObjectColors.Add(SavedPreColorPickerColors[ColorIndex].GetLinear().ToString());
		}
		else
		{
			FColor Color = SavedPreColorPickerColors[ColorIndex].GetSrgb();
			PerObjectColors.Add(Color.ToString());
		}
	}

	return PerObjectColors;
}

void FColorStructCustomization::ResetColors()
{
	TArray<FString> PerObjectColors = ConvertToPerObjectColors(SavedPreColorPickerColors);

	if (PerObjectColors.Num() > 0)
	{
		// See @TODO in FColorStructCustomization::OnColorPickerWindowClosed
		// ensure(StructPropertyHandle->SetPerObjectValues(PerObjectColors, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Success);
		StructPropertyHandle->SetPerObjectValues(PerObjectColors, EPropertyValueSetFlags::NotTransactable);
	}
}

void FColorStructCustomization::OnColorPickerCancelled(FLinearColor OriginalColor)
{
	ResetColors();
	LastPickerColorString.Reset();

	GEditor->CancelTransaction(0);
	TransactionIndex.Reset();

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void FColorStructCustomization::OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window)
{
	// Transact only at the end to avoid opening a lingering transaction. Reset value before transacting.
	if (!LastPickerColorString.IsEmpty())
	{
		//@TODO: Not using reset & apply instant scoped transition pattern since certain property nodes are 
		// returning nullptr when finding objects to modify on reset, so we can't reset correctly for those.
		// ResetColors();
		{
			// FScopedTransaction Transaction(FText::Format(LOCTEXT("SetColorProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));
			StructPropertyHandle->SetValueFromFormattedString(LastPickerColorString);
		}
	}

	GEditor->EndTransaction();
	TransactionIndex.Reset();
	
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}


void FColorStructCustomization::OnColorPickerInteractiveBegin()
{
	bIsInteractive = true;
}


void FColorStructCustomization::OnColorPickerInteractiveEnd()
{
	if (bIsInteractive)
	{
		bIsInteractive = false;

		if (!LastPickerColorString.IsEmpty())
		{
			EPropertyValueSetFlags::Type PropertyFlags = EPropertyValueSetFlags::NotTransactable;
			PropertyFlags |= bIsInBlueprint ? EPropertyValueSetFlags::InteractiveChange : 0;
			StructPropertyHandle->SetValueFromFormattedString(LastPickerColorString, PropertyFlags);
		}
	}
}


FLinearColor FColorStructCustomization::OnGetColorForColorBlock() const
{
	FLinearColor Color;
	GetColorAsLinear(Color);
	return Color;
}


FSlateColor FColorStructCustomization::OnGetSlateColorForBlock() const
{
	FLinearColor Color = OnGetColorForColorBlock();
	Color.A = 1;
	return FSlateColor(Color);
}

FSlateColor FColorStructCustomization::GetColorWidgetBorderColor() const
{
	static const FSlateColor HoveredColor = FAppStyle::Get().GetSlateColor("Colors.Hover");
	static const FSlateColor DefaultColor = FAppStyle::Get().GetSlateColor("Colors.InputOutline");
	return ColorWidgetBackgroundBorder->IsHovered() ? HoveredColor : DefaultColor;
}

FPropertyAccess::Result FColorStructCustomization::GetColorAsLinear(FLinearColor& OutColor) const
{
	// Default to full alpha in case the alpha component is disabled.
	OutColor.A = 1.0f;

	FString StringValue;
	FPropertyAccess::Result Result = StructPropertyHandle->GetValueAsFormattedString(StringValue);

	if(Result == FPropertyAccess::Success)
	{
		if (bIsLinearColor)
		{
			OutColor.InitFromString(StringValue);
		}
		else
		{
			FColor SrgbColor;
			SrgbColor.InitFromString(StringValue);
			OutColor = FLinearColor(SrgbColor);
		}
	}
	else if(Result == FPropertyAccess::MultipleValues)
	{
		OutColor = FLinearColor::White;
	}

	return Result;
}


EVisibility FColorStructCustomization::GetMultipleValuesTextVisibility() const
{
	FLinearColor Color;
	const FPropertyAccess::Result ValueResult = GetColorAsLinear(Color);
	return (ValueResult == FPropertyAccess::MultipleValues) ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}


FReply FColorStructCustomization::OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	
	bool CanShowColorPicker = true;
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->GetProperty() != nullptr)
	{
		CanShowColorPicker = !StructPropertyHandle->IsEditConst();
	}
	if (CanShowColorPicker)
	{
		CreateColorPicker(true /*bUseAlpha*/);
	}

	return FReply::Handled();
}


FReply FColorStructCustomization::OnOpenFullColorPickerClicked()
{
	CreateColorPicker(true /*bUseAlpha*/);
	bIsInlineColorPickerVisible = false;

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
