// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentMaterialCategory.h"

#include "AI/NavigationSystemBase.h"
#include "AssetRegistry/AssetData.h"
#include "Components/ActorComponent.h"
#include "Components/DecalComponent.h"
#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Components/TextRenderComponent.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "JsonObjectConverter.h"
#include "LandscapeComponent.h"
#include "LandscapeProxy.h"
#include "MaterialList.h"
#include "Misc/App.h"
#include "Misc/NotifyHook.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectHash.h"
#include "UObject/UnrealType.h"
#include "UnrealEdGlobals.h"
#include "PropertyCustomizationHelpers.h"

class AActor;
class UWorld;

#define LOCTEXT_NAMESPACE "SMaterialList"

/**
 * Specialized iterator for stepping through used materials on actors
 * Iterates through all materials on the provided list of actors by examining each actors component for materials
 */
class FMaterialIterator
{
public:
	FMaterialIterator( const TArray< TWeakObjectPtr<USceneComponent> >& InSelectedComponents )
		: SelectedComponents( InSelectedComponents )
		, CurMaterial( NULL )
		, CurComponent( NULL )
		, CurComponentIndex( 0 )
		, CurMaterialIndex( -1 )
		, bReachedEnd( false )
	{
		// Step to the first material
		++(*this);
	}

	/**
	 * Advances to the next material
	 */
	void operator++()
	{
		// Advance to the next material
		++CurMaterialIndex;

		// Examine each component until we are out of them
		while( SelectedComponents.IsValidIndex(CurComponentIndex) )
		{
			USceneComponent* TestComponent = SelectedComponents[CurComponentIndex].Get();

			if( TestComponent != NULL )
			{
				CurComponent = TestComponent;
				int32 NumMaterials = 0;

				// Primitive components and some actor components have materials
				UPrimitiveComponent* PrimitiveComp = Cast<UPrimitiveComponent>( CurComponent );
				UDecalComponent* DecalComponent = (PrimitiveComp ? NULL : Cast<UDecalComponent>( CurComponent ));
				TArray<FName> MaterialSlotNames;

				if( PrimitiveComp )
				{
					if (PrimitiveComp->IsCompiling())
					{
						++CurComponentIndex;
						continue;
					}

					NumMaterials = PrimitiveComp->GetNumMaterials();
					MaterialSlotNames = PrimitiveComp->GetMaterialSlotNames();
				}
				else if( DecalComponent )
				{
					// DecalComponent isn't a primitive component so we must get the materials directly from it
					NumMaterials = DecalComponent->GetNumMaterials();
				}

				// Check materials
				while( CurMaterialIndex < NumMaterials )
				{
					UMaterialInterface* Material = NULL;

					if( PrimitiveComp )
					{
						Material = PrimitiveComp->GetEditorMaterial(CurMaterialIndex);
						if (MaterialSlotNames.IsValidIndex(CurMaterialIndex))
						{
							CurMaterialSlotName = MaterialSlotNames[CurMaterialIndex];
						}
						else
						{
							CurMaterialSlotName = FName();
						}
					}
					else if( DecalComponent )
					{
						Material = DecalComponent->GetMaterial(CurMaterialIndex);
						CurMaterialSlotName = FName();
					}

					CurMaterial = Material;
							
					// We step only once in the iterator if we have a material.  Note: A null material is considered valid
					
					// PVS-Studio has noticed that this is a fairly unorthodox return statement. Typically a break would 
					// be the expected control sequence in a nested while loop. This may be correct, however, so for now
					// we have disabled the warning.
					return; //-V612
				}
				// Out of materials on this component, reset for the next component
				CurMaterialIndex = 0;
				CurMaterialSlotName = FName();
			}
			// Advance to the next compoment
			++CurComponentIndex;
		}


		// Out of components to check, reset to an invalid state
		CurComponentIndex = INDEX_NONE;
		bReachedEnd = true;
		CurComponent = NULL;
		CurMaterial = NULL;
		CurMaterialIndex = INDEX_NONE;
		CurMaterialSlotName = FName();
	}

	/**
	 * @return Whether or not the iterator is valid
	 */
	explicit operator bool() const
	{
		return !bReachedEnd;	
	}

	/**
	 * @return The current material the iterator is stopped on
	 */
	UMaterialInterface* GetMaterial() const { return CurMaterial; }

	/**
	 * @return The index of the material in the current component
	 */
	int32 GetMaterialIndex() const { return CurMaterialIndex; }

	/**
	 * @return The material slot name of the material in the current component, if applicable.
	 */
	FName GetMaterialSlotName() const { return CurMaterialSlotName; }

	/**
	 * @return The current component using the current material
	 */
	UActorComponent* GetComponent() const { return CurComponent; }

private:
	/** Reference to the selected components */
	const TArray< TWeakObjectPtr<USceneComponent> >& SelectedComponents;
	/** The current material the iterator is stopped on */
	UMaterialInterface* CurMaterial;
	/** The current component using the current material */
	UActorComponent* CurComponent;
	/** The index of the component we are stopped on */
	int32 CurComponentIndex;
	/** The index of the material we are stopped on */
	int32 CurMaterialIndex;
	/** The slot name of the material we are stopped on, if applicable */
	FName CurMaterialSlotName;
	/** Whether or not we've reached the end of the components */
	uint32 bReachedEnd:1;
};

FComponentMaterialCategory::FComponentMaterialCategory( TArray< TWeakObjectPtr<USceneComponent> >& InSelectedComponents )
	: SelectedComponents( InSelectedComponents )
	, NotifyHook( nullptr )
	, MaterialCategory( nullptr )
{
}

void FComponentMaterialCategory::Create( IDetailLayoutBuilder& DetailBuilder )
{
	NotifyHook = DetailBuilder.GetPropertyUtilities()->GetNotifyHook();

	FMaterialListDelegates MaterialListDelegates;
	MaterialListDelegates.OnGetMaterials.BindSP( this, &FComponentMaterialCategory::OnGetMaterialsForView );
	MaterialListDelegates.OnMaterialChanged.BindSP( this, &FComponentMaterialCategory::OnMaterialChanged );
	MaterialListDelegates.OnCanCopyMaterialItem.BindSP( this, &FComponentMaterialCategory::OnCanCopyMaterialItem );
	MaterialListDelegates.OnCopyMaterialItem.BindSP( this, &FComponentMaterialCategory::OnCopyMaterialItem );
	MaterialListDelegates.OnPasteMaterialItem.BindSP(this, &FComponentMaterialCategory::OnPasteMaterialItem);
	MaterialListDelegates.OnGenerateCustomMaterialWidgets.BindSP(this, &FComponentMaterialCategory::OnGenerateWidgetsForMaterial);
	
	//Pass an empty material list owner (owner can be use by the asset picker filter. In this case we do not need it)
	TArray<FAssetData> MaterialListOwner;

	TSharedRef<FMaterialList> MaterialList = MakeShareable( new FMaterialList( DetailBuilder, MaterialListDelegates, MaterialListOwner) );

	bool bAnyMaterialsToDisplay = false;

	for( FMaterialIterator It( SelectedComponents ); It; ++It )
	{	
		UActorComponent* CurrentComponent = It.GetComponent();

		if( !bAnyMaterialsToDisplay )
		{
			bAnyMaterialsToDisplay = true;
			break;
		}
	}

	// Make a category for the materials.
	MaterialCategory = &DetailBuilder.EditCategory("Materials", FText::GetEmpty(), ECategoryPriority::TypeSpecific );

	if (bAnyMaterialsToDisplay)
	{
		MaterialCategory->AddCustomBuilder( MaterialList );
		MaterialCategory->SetCategoryVisibility( bAnyMaterialsToDisplay );
		return;
	}

	// Check again - if there are any properties in the category, we don't want to hide it
	TArray<TSharedRef<IPropertyHandle>> DefaultMaterialProperties;
	MaterialCategory->GetDefaultProperties(DefaultMaterialProperties);
	
	if (!DefaultMaterialProperties.IsEmpty())
	{
		bAnyMaterialsToDisplay = true;
	}
	
	MaterialCategory->SetCategoryVisibility( bAnyMaterialsToDisplay );
}

void FComponentMaterialCategory::OnGetMaterialsForView( IMaterialListBuilder& MaterialList )
{
	const bool bAllowNullEntries = true;
	bool bAnyMaterialsToDisplay = false;

	// Iterate over every material on the actors
	for( FMaterialIterator It( SelectedComponents ); It; ++It )
	{	
		int32 MaterialIndex = It.GetMaterialIndex();

		UActorComponent* CurrentComponent = It.GetComponent();

		if( CurrentComponent )
		{
			UMaterialInterface* Material = It.GetMaterial();

			AActor* Actor = CurrentComponent->GetOwner();

			// Component materials can be replaced if the component supports material overrides
			const bool bCanBeReplaced =
				( CurrentComponent->IsA( UMeshComponent::StaticClass() ) ||
				CurrentComponent->IsA( UTextRenderComponent::StaticClass() ) ||
				CurrentComponent->IsA( ULandscapeComponent::StaticClass() ) );

			// Add the material if we allow null materials to be added or we have a valid material
			if( bAllowNullEntries || Material )
			{
				MaterialList.AddMaterial( MaterialIndex, Material, bCanBeReplaced, CurrentComponent );
				bAnyMaterialsToDisplay = true;
			}
		}
	}

	if (bAnyMaterialsToDisplay)
	{
		MaterialCategory->SetCategoryVisibility(true);
		return;
	}

	// Check again - if there are any properties in the category, we don't want to hide it
	TArray<TSharedRef<IPropertyHandle>> DefaultMaterialProperties;
	MaterialCategory->GetDefaultProperties(DefaultMaterialProperties);

	if (!DefaultMaterialProperties.IsEmpty())
	{
		bAnyMaterialsToDisplay = true;
	}

	MaterialCategory->SetCategoryVisibility(bAnyMaterialsToDisplay);
}

void FComponentMaterialCategory::OnMaterialChanged( UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll )
{
	// Whether or not we should begin a transaction on swap
	// Note we only begin a transaction on the first swap
	bool bShouldMakeTransaction = true;

	// Whether or not we made a transaction and need to end it
	bool bMadeTransaction = false;

	// Lambda to swap materials on a given component at the given slot index
	auto SwapMaterialLambda = []( UActorComponent* InComponent, int32 InElementIndex, UMaterialInterface* InNewMaterial )
	{
		UPrimitiveComponent* PrimitiveComp = Cast<UPrimitiveComponent>( InComponent );
		UDecalComponent* DecalComponent = Cast<UDecalComponent>( InComponent );

		if( PrimitiveComp )
		{
			PrimitiveComp->SetMaterial( InElementIndex, InNewMaterial );
		}
		else if( DecalComponent )
		{
			DecalComponent->SetMaterial( InElementIndex, InNewMaterial );
		}
	};

	struct FObjectAndProperty
	{
		UObject* Object;
		FProperty* PropertyThatChanged;
	};
	TArray<FObjectAndProperty> ObjectsThatChanged;
	// Scan the selected actors mesh components for the old material and swap it with the new material 
	for( FMaterialIterator It( SelectedComponents ); It; ++It )
	{
		int32 MaterialIndex = It.GetMaterialIndex();

		UActorComponent* CurrentComponent = It.GetComponent();

		if( CurrentComponent )
		{
			// Component materials can be replaced if they are not created from a blueprint (not exposed to the user) and have material overrides on the component
			bool bCanBeReplaced = 
				( CurrentComponent->IsA( UMeshComponent::StaticClass() ) ||
				CurrentComponent->IsA( UDecalComponent::StaticClass() ) ||
				CurrentComponent->IsA( UTextRenderComponent::StaticClass() ) ||
				CurrentComponent->IsA( ULandscapeComponent::StaticClass() ) );

			UMaterialInterface* Material = It.GetMaterial();
			// Check if the material is the same as the previous material or we are replaceing all in the same slot.  If so we will swap it with the new material
			if( bCanBeReplaced && ( Material == PrevMaterial || bReplaceAll ) && It.GetMaterialIndex() == SlotIndex )
			{
				// Begin a transaction for undo/redo the first time we encounter a material to replace.  
				// There is only one transaction for all replacement
				if( bShouldMakeTransaction && !bMadeTransaction )
				{
					GEditor->BeginTransaction( NSLOCTEXT("UnrealEd", "ReplaceComponentUsedMaterial", "Replace component used material") );

					bMadeTransaction = true;
				}

				FProperty* MaterialProperty = NULL;
				UObject* EditChangeObject = CurrentComponent;
				if( CurrentComponent->IsA( UMeshComponent::StaticClass() ) )
				{
					MaterialProperty = FindFProperty<FProperty>( UMeshComponent::StaticClass(), "OverrideMaterials" );
				}
				else if( CurrentComponent->IsA( UDecalComponent::StaticClass() ) )
				{
					MaterialProperty = FindFProperty<FProperty>( UDecalComponent::StaticClass(), "DecalMaterial" );
				}
				else if( CurrentComponent->IsA( UTextRenderComponent::StaticClass() ) )
				{
					MaterialProperty = FindFProperty<FProperty>( UTextRenderComponent::StaticClass(), "TextMaterial" );
				}
				else if (CurrentComponent->IsA<ULandscapeComponent>() )
				{
					MaterialProperty = FindFProperty<FProperty>( ALandscapeProxy::StaticClass(), "LandscapeMaterial" );
					EditChangeObject = CastChecked<ULandscapeComponent>(CurrentComponent)->GetLandscapeProxy();
				}

				// Add a navigation update lock only if the component world is valid
				TSharedPtr<FNavigationLockContext> NavUpdateLock;
				UWorld* World = CurrentComponent->GetWorld();
				if( World )
				{
					NavUpdateLock = MakeShareable( new FNavigationLockContext(World, ENavigationLockReason::MaterialUpdate) );
				}

				{
					FEditPropertyChain EditPropertyChain;
					EditPropertyChain.AddHead(MaterialProperty);
					EditPropertyChain.SetActivePropertyNode(MaterialProperty);
					// Use a property chain so that FCoreUObjectDelegates::OnPreObjectPropertyChanged is emitted
					EditChangeObject->PreEditChange(EditPropertyChain);
				}

				if( NotifyHook && MaterialProperty )
				{
					NotifyHook->NotifyPreChange( MaterialProperty );
				}
				FObjectAndProperty ObjectAndProperty;
				ObjectAndProperty.Object = EditChangeObject;
				ObjectAndProperty.PropertyThatChanged = MaterialProperty;

				ObjectsThatChanged.Add(ObjectAndProperty);

				FPropertyChangedEvent PropertyChangedEvent(MaterialProperty);

				SwapMaterialLambda( CurrentComponent, It.GetMaterialIndex(), NewMaterial );

				// Propagate material change to instances of the edited component template
				if( !FApp::IsGame() )
				{
					TArray<UObject*> ComponentArchetypeInstances;
					if( CurrentComponent->HasAnyFlags(RF_ArchetypeObject) )
					{
						CurrentComponent->GetArchetypeInstances(ComponentArchetypeInstances);
					}
					else if( UObject* Outer = CurrentComponent->GetOuter() )
					{
						TArray<UObject*> OuterArchetypeInstances;
						Outer->GetArchetypeInstances(OuterArchetypeInstances);
						for( auto OuterArchetypeInstance : OuterArchetypeInstances )
						{
							if( UObject* ArchetypeInstance = static_cast<UObject*>(FindObjectWithOuter(OuterArchetypeInstance, CurrentComponent->GetClass(), CurrentComponent->GetFName())) )
							{
								ComponentArchetypeInstances.Add(ArchetypeInstance);
							}
						}
					}

					for( auto ComponentArchetypeInstance : ComponentArchetypeInstances )
					{
						CurrentComponent = CastChecked<UActorComponent>( ComponentArchetypeInstance );
						if( CurrentComponent->IsA<ULandscapeComponent>() )
						{
							ComponentArchetypeInstance = CastChecked<ULandscapeComponent>(CurrentComponent)->GetLandscapeProxy();
						}

						// Reset the navigation update lock if necessary
						UWorld* PreviousWorld = World;
						World = CurrentComponent->GetWorld();
						if( PreviousWorld != World )
						{
							NavUpdateLock = MakeShareable( new FNavigationLockContext(World, ENavigationLockReason::MaterialUpdate) );
						}

						ComponentArchetypeInstance->PreEditChange( MaterialProperty );

						SwapMaterialLambda( CurrentComponent, It.GetMaterialIndex(), NewMaterial );

						ComponentArchetypeInstance->PostEditChangeProperty( PropertyChangedEvent );
					}
				}
			}
		}
	}

	// Route post edit change after all components have had their values changed.  This is to avoid 
	// construction scripts from re-running in the middle of setting values and wiping out components we need to modify
	for( FObjectAndProperty& ObjectData : ObjectsThatChanged)
	{
		FPropertyChangedEvent PropertyChangeEvent(ObjectData.PropertyThatChanged, EPropertyChangeType::ValueSet);
		ObjectData.Object->PostEditChangeProperty(PropertyChangeEvent);

		if(NotifyHook && ObjectData.PropertyThatChanged)
		{
			NotifyHook->NotifyPostChange(PropertyChangeEvent, ObjectData.PropertyThatChanged);
		}
	}

	
	if( bMadeTransaction )
	{
		// End the transation if we created one
		GEditor->EndTransaction();
		// Redraw viewports to reflect the material changes 
		GUnrealEd->RedrawLevelEditingViewports();
	}
}

void FComponentMaterialCategory::OnCopyMaterialItem(int32 CurrentSlot)
{
	// For now we don't support multi-select
	// TODO: Pass through more data to these delegates that allows us to know which
	// material should be copied, or allow reaching into the generated material list from here
	if (SelectedComponents.Num() != 1)
	{
		return;
	}

	for (FMaterialIterator It(SelectedComponents); It; ++It)
	{
		if (It.GetMaterialIndex() == CurrentSlot)
		{
			TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

			const FStaticMaterial StaticMaterial(It.GetMaterial());
			FJsonObjectConverter::UStructToJsonObject(FStaticMaterial::StaticStruct(), &StaticMaterial, RootJsonObject, 0, 0);

			typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
			typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

			FString CopyStr;
			TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
			FJsonSerializer::Serialize(RootJsonObject, Writer);

			if (!CopyStr.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
			}

			break;
		}
	}
}

bool FComponentMaterialCategory::OnCanCopyMaterialItem(int32 CurrentSlot) const
{
	// For now we don't support multi-select
	if (SelectedComponents.Num() != 1)
	{
		return false;
	}

	for (FMaterialIterator It(SelectedComponents); It; ++It)
	{
		const UActorComponent* CurrentComponent = It.GetComponent();

		// We need to handle the same various component types as OnMaterialChanged,
		// which are all primitive components except decals, which only have a single material slot anyway
		if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(CurrentComponent))
		{
			if (CurrentSlot >= PrimitiveComponent->GetNumMaterials())
			{
				return false;
			}
		}
		else if (CurrentComponent->IsA<UDecalComponent>())
		{
			return CurrentSlot == 0;
		}
	}

	return true;
}

void FComponentMaterialCategory::OnPasteMaterialItem(int32 CurrentSlot)
{
	// For now we don't support multi-select
	if (SelectedComponents.Num() != 1)
	{
		return;
	}

	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if (RootJsonObject.IsValid())
	{
		FStaticMaterial TmpStaticMaterial;
		FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FStaticMaterial::StaticStruct(), &TmpStaticMaterial, 0, 0);
		UMaterialInterface* NewMaterial = TmpStaticMaterial.MaterialInterface;

		// Try to find the previous material used for this slot so we can piggy-back on the existing material change method
		for (FMaterialIterator It(SelectedComponents); It; ++It)
		{
			if (It.GetMaterialIndex() == CurrentSlot)
			{
				UMaterialInterface* PreviousMaterial = It.GetMaterial();
				OnMaterialChanged(NewMaterial, PreviousMaterial, CurrentSlot, /* bReplaceAll */ false);
				break;
			}
		}
	}
}

TSharedRef<SWidget> FComponentMaterialCategory::OnGenerateWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex)
{
	return
		SNew(SMaterialSlotWidget, SlotIndex, true)
		.MaterialName(this, &FComponentMaterialCategory::GetMaterialNameText, SlotIndex)
		.IsMaterialSlotNameReadOnly(true)
		.DeleteMaterialSlotVisibility(EVisibility::Collapsed);
}

FText FComponentMaterialCategory::GetMaterialNameText(int32 MaterialIndex) const
{
	// For now we don't support multi-select
	if (SelectedComponents.Num() != 1)
	{
		return FText::FromName(NAME_None);
	}
	for (FMaterialIterator It(SelectedComponents); It; ++It)
	{
		if (It.GetMaterialIndex() == MaterialIndex)
		{
			return FText::FromName(It.GetMaterialSlotName());
		}
	}

	return FText::FromName(NAME_None);
}

#undef LOCTEXT_NAMESPACE
