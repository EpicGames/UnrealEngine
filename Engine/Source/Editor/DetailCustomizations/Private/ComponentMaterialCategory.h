// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FNotifyHook;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class UActorComponent;
class UMaterialInterface;
class USceneComponent;
class SWidget;

/**
 * Encapsulates functionality for the ActorDetails material category
 */
class FComponentMaterialCategory : public TSharedFromThis<FComponentMaterialCategory>
{
public:
	/**
	 * Constructs a category in the details panel for displaying used materials
	 */
	FComponentMaterialCategory( TArray< TWeakObjectPtr<USceneComponent> >& InSelectedComponents );

	/**
	 * Creates the category
	 */
	void Create( IDetailLayoutBuilder& DetailBuilder );
private:
	/**
	 * Called by the material list widget when we need to get new materials for the list
	 *
	 * @param OutMaterials	Handle to a material list builder that materials should be added to
	 */
	void OnGetMaterialsForView( class IMaterialListBuilder& OutMaterials );

	/**
	 * Called when a user drags a new material over a list item to replace it
	 *
	 * @param NewMaterial	The material that should replace the existing material
	 * @param PrevMaterial	The material that should be replaced
	 * @param SlotIndex		The index of the slot on the component where materials should be replaces
	 * @param bReplaceAll	If true all materials in the slot should be replaced not just ones using PrevMaterial
	 */
	void OnMaterialChanged( UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll );

	/**
	 * Called when a user executes the copy command on a material list item
	 *
	 * @param CurrentSlot	The slot index that was copied
	 */
	void OnCopyMaterialItem(int32 CurrentSlot);

	/**
	 * Called to determine if the copy command should be enabled for a particular a material list item
	 *
	 * @param CurrentSlot	The slot index that was copied
	 * @return	whether this slot index can be copied
	 */
	bool OnCanCopyMaterialItem(int32 CurrentSlot) const;

	/**
	 * Called when a user executes the paste command on a material list item
	 *
	 * @param CurrentSlot	The slot index that was pasted on
	 */
	void OnPasteMaterialItem(int32 CurrentSlot);

	/**
	 * Called by the material list widget to customize the material slot widget
	 *
	 * @param Material	The material for the slot widget to be customized
	 * @param SlotIndex The slot index for the slot widget to be customized
	 */
	TSharedRef<SWidget> OnGenerateWidgetsForMaterial(UMaterialInterface* Material, int32 SlotIndex);

	/** 
	 * @return true if a component is editable (and visible in the view)
	 */
	bool IsComponentEditable( UActorComponent* InComponent ) const;

private:
	/** Reference to the list of selected actors */
	TArray< TWeakObjectPtr<USceneComponent> > SelectedComponents;

	/** Notify hook to use */
	FNotifyHook* NotifyHook;

	IDetailCategoryBuilder* MaterialCategory;

	FText GetMaterialNameText(int32 MaterialIndex) const;
};
