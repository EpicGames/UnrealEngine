// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentReferenceCustomization.h"

#include "ActorPickerMode.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "Brushes/SlateNoResource.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "Engine/LevelScriptActor.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ComponentReferenceCustomization"

static const FLazyName NAME_AllowAnyActor = FLazyName("AllowAnyActor");
static const FLazyName NAME_AllowedClasses = FLazyName("AllowedClasses");
static const FLazyName NAME_DisallowedClasses = FLazyName("DisallowedClasses");
static const FLazyName NAME_UseComponentPicker = FLazyName("UseComponentPicker");
static const FLazyName NAME_DisallowLevelActorReference = FLazyName("DisallowLevelActorReference");

TSharedRef<IPropertyTypeCustomization> FComponentReferenceCustomization::MakeInstance()
{
	return MakeShareable( new FComponentReferenceCustomization);
}

void FComponentReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	CachedComponent.Reset();
	CachedFirstOuterActor.Reset();
	CachedPropertyAccess = FPropertyAccess::Fail;

	bAllowClear = false;
	bAllowAnyActor = false;
	bUseComponentPicker = PropertyHandle->HasMetaData(NAME_UseComponentPicker);
	bIsSoftReference = false;

	if (bUseComponentPicker)
	{
		FProperty* Property = InPropertyHandle->GetProperty();
		check(CastField<FStructProperty>(Property) &&
				(FComponentReference::StaticStruct() == CastFieldChecked<const FStructProperty>(Property)->Struct ||
				FSoftComponentReference::StaticStruct() == CastFieldChecked<const FStructProperty>(Property)->Struct));

		bAllowClear = !(InPropertyHandle->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);
		bAllowAnyActor = InPropertyHandle->HasMetaData(NAME_AllowAnyActor);
		bIsSoftReference = FSoftComponentReference::StaticStruct() == CastFieldChecked<const FStructProperty>(Property)->Struct;

		BuildClassFilters();
		BuildComboBox();

		InPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FComponentReferenceCustomization::OnPropertyValueChanged));

		UpdateCachedValues(/*bResetValueIfInvalid*/ false);

		HeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			ComponentComboButton.ToSharedRef()
		]
		.IsEnabled(MakeAttributeSP(this, &FComponentReferenceCustomization::CanEdit));
	}
	else
	{
		HeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InPropertyHandle->CreatePropertyValueWidget()
		]
		.IsEnabled(MakeAttributeSP(this, &FComponentReferenceCustomization::CanEdit));
	}
}

void FComponentReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumberOfChild;
	if (InStructPropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			TSharedRef<IPropertyHandle> ChildPropertyHandle = InStructPropertyHandle->GetChildHandle(Index).ToSharedRef();
			if (bUseComponentPicker)
			{
				ChildPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FComponentReferenceCustomization::OnPropertyValueChanged));
				StructBuilder.AddProperty(ChildPropertyHandle)
					.ShowPropertyButtons(true)
					.IsEnabled(MakeAttributeSP(this, &FComponentReferenceCustomization::CanEditChildren));
			}
			else
			{
				StructBuilder.AddProperty(ChildPropertyHandle)
					.ShowPropertyButtons(true)
					.IsEnabled(MakeAttributeSP(this, &FComponentReferenceCustomization::CanEditChildren));
			}
		}
	}
}

void FComponentReferenceCustomization::BuildClassFilters()
{
	auto AddToClassFilters = [this](const UClass* Class, TArray<const UClass*>& ActorList, TArray<const UClass*>& ComponentList)
	{
		if (bAllowAnyActor && Class->IsChildOf(AActor::StaticClass()))
		{
			ActorList.Add(Class);
		}
		else if (Class->IsChildOf(UActorComponent::StaticClass()))
		{
			ComponentList.Add(Class);
		}
	};

	auto ParseClassFilters = [this, AddToClassFilters](const FString& MetaDataString, TArray<const UClass*>& ActorList, TArray<const UClass*>& ComponentList)
	{
		if (!MetaDataString.IsEmpty())
		{
			TArray<FString> ClassFilterNames;
			MetaDataString.ParseIntoArrayWS(ClassFilterNames, TEXT(","), true);

			for (const FString& ClassName : ClassFilterNames)
			{
				UClass* Class = UClass::TryFindTypeSlow<UClass>(ClassName);
				if (!Class)
				{
					Class = LoadObject<UClass>(nullptr, *ClassName);
				}

				if (Class)
				{
					// If the class is an interface, expand it to be all classes in memory that implement the class.
					if (Class->HasAnyClassFlags(CLASS_Interface))
					{
						for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
						{
							UClass* const ClassWithInterface = (*ClassIt);
							if (ClassWithInterface->ImplementsInterface(Class))
							{
								AddToClassFilters(ClassWithInterface, ActorList, ComponentList);
							}
						}
					}
					else
					{
						AddToClassFilters(Class, ActorList, ComponentList);
					}
				}
			}
		}
	};

	// Account for the allowed classes specified in the property metadata
	const FString& AllowedClassesFilterString = PropertyHandle->GetMetaData(NAME_AllowedClasses);
	ParseClassFilters(AllowedClassesFilterString, AllowedActorClassFilters, AllowedComponentClassFilters);

	const FString& DisallowedClassesFilterString = PropertyHandle->GetMetaData(NAME_DisallowedClasses);
	ParseClassFilters(DisallowedClassesFilterString, DisallowedActorClassFilters, DisallowedComponentClassFilters);
}

void FComponentReferenceCustomization::BuildComboBox()
{
	TAttribute<bool> IsEnabledAttribute(this, &FComponentReferenceCustomization::CanEdit);
	TAttribute<FText> TooltipAttribute;

	TArray<UObject*> ObjectList;
	PropertyHandle->GetOuterObjects(ObjectList);

	if (PropertyHandle->GetMetaDataProperty()->HasAnyPropertyFlags(CPF_EditConst | CPF_DisableEditOnTemplate))
	{

		// If there is no objects, that means we must have a struct asset managing this property
		if (ObjectList.Num() == 0)
		{
			IsEnabledAttribute.Set(false);
			TooltipAttribute.Set(LOCTEXT("VariableHasDisableEditOnTemplate", "Editing this value in structure's defaults is not allowed"));
		}
		else
		{
			// Go through all the found objects and see if any are a CDO, we can't set an actor in a CDO default.
			for (const UObject* Obj : ObjectList)
			{
				if (Obj->IsTemplate() && !Obj->IsA<ALevelScriptActor>())
				{
					IsEnabledAttribute.Set(false);
					TooltipAttribute.Set(LOCTEXT("VariableHasDisableEditOnTemplateTooltip", "Editing this value in a Class Default Object is not allowed"));
					break;
				}

			}
		}
	}

	if (IsEnabledAttribute.Get())
	{
		for (const UObject* Obj : ObjectList)
		{
			if (const UClass* Class = Obj->GetClass())
			{
				if (Class->GetBoolMetaDataHierarchical(NAME_DisallowLevelActorReference))
				{
					IsEnabledAttribute.Set(false);
					TooltipAttribute.Set(LOCTEXT("ObjectDisallowLevelActorReference", "Object doesn't allow level actor reference"));
					break;
				}
			}
		}
	}

	TSharedRef<SVerticalBox> ObjectContent = SNew(SVerticalBox);
	if (bAllowAnyActor)
	{
		ObjectContent->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SImage)
				.Image(this, &FComponentReferenceCustomization::GetActorIcon)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FComponentReferenceCustomization::OnGetActorName)
			]
		];
	}

	ObjectContent->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SImage)
			.Image(this, &FComponentReferenceCustomization::GetComponentIcon)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			// Show the name of the asset or actor
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FComponentReferenceCustomization::OnGetComponentName)
		]
	];

	TSharedRef<SHorizontalBox> ComboButtonContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(this, &FComponentReferenceCustomization::GetStatusIcon)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		[
			ObjectContent
		];

	ComponentComboButton = SNew(SComboButton)
		.ToolTipText(TooltipAttribute)
		.OnGetMenuContent(this, &FComponentReferenceCustomization::OnGetMenuContent)
		.OnMenuOpenChanged(this, &FComponentReferenceCustomization::OnMenuOpenChanged)
		.IsEnabled(IsEnabledAttribute)
		.ContentPadding(0.0f)
		.ButtonContent()
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FComponentReferenceCustomization::OnGetComboContentWidgetIndex)
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MultipleValuesText", "<multiple values>"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SWidgetSwitcher::Slot()
			[
				ComboButtonContent
			]
		];
}


AActor* FComponentReferenceCustomization::GetFirstOuterActor() const
{
	TArray<UObject*> ObjectList;
	PropertyHandle->GetOuterObjects(ObjectList);
	for (UObject* Obj : ObjectList)
	{
		while (Obj)
		{
			if (AActor* Actor = Cast<AActor>(Obj))
			{
				return Actor;
			}
			if (UActorComponent* Component = Cast<UActorComponent>(Obj))
			{
				if (Component->HasAnyFlags(RF_ArchetypeObject))
				{
					if (UClass* OuterClass = Cast<UClass>(Component->GetOuter()))
					{
						return Cast<AActor>(OuterClass->GetDefaultObject());
					}
				}

				if (Component->GetOwner())
				{
					return Component->GetOwner();
				}
			}
			Obj = Obj->GetOuter();
		}
	}
	return nullptr;
}

void FComponentReferenceCustomization::SetValue(const FComponentReference& Value)
{
	ComponentComboButton->SetIsOpen(false);

	const bool bIsEmpty = Value == FComponentReference();
	const bool bAllowedToSetBasedOnFilter = IsComponentReferenceValid(Value);
	if (bIsEmpty || bAllowedToSetBasedOnFilter)
	{
		FString TextValue;
		if (bIsSoftReference)
		{
			FSoftComponentReference SoftValue;
			if (Value.OtherActor.IsValid())
			{
				SoftValue.OtherActor = Value.OtherActor.Get();
				SoftValue.ComponentProperty = Value.ComponentProperty;
				SoftValue.PathToComponent = Value.PathToComponent;
			}
			CastFieldChecked<const FStructProperty>(PropertyHandle->GetProperty())->Struct->ExportText(TextValue, &SoftValue, &SoftValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
			ensure(PropertyHandle->SetValueFromFormattedString(TextValue) == FPropertyAccess::Result::Success);
		}
		else
		{
			CastFieldChecked<const FStructProperty>(PropertyHandle->GetProperty())->Struct->ExportText(TextValue, &Value, &Value, nullptr, EPropertyPortFlags::PPF_None, nullptr);
			ensure(PropertyHandle->SetValueFromFormattedString(TextValue) == FPropertyAccess::Result::Success);
		}
	}
}

FPropertyAccess::Result FComponentReferenceCustomization::GetValue(FComponentReference& OutValue) const
{
	// Potentially accessing the value while garbage collecting or saving the package could trigger a crash.
	// so we fail to get the value when that is occurring.
	if (UE::IsSavingPackage() || IsGarbageCollecting())
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);
		UActorComponent* CurrentComponent = nullptr;
		AActor* CurrentActor = CachedFirstOuterActor.Get();
		for (const void* RawPtr : RawData)
		{
			if (RawPtr)
			{
				FComponentReference ThisReference;
				if (bIsSoftReference)
				{
					FSoftComponentReference SoftReference = *reinterpret_cast<const FSoftComponentReference*>(RawPtr);
					if (SoftReference.OtherActor.IsValid())
					{
						ThisReference.OtherActor = SoftReference.OtherActor.Get();
						ThisReference.ComponentProperty = SoftReference.ComponentProperty;
						ThisReference.PathToComponent = SoftReference.PathToComponent;
					}
				}
				else
				{
					ThisReference = *reinterpret_cast<const FComponentReference*>(RawPtr);
				}
				if (Result == FPropertyAccess::Success)
				{
					if (ThisReference.GetComponent(CurrentActor) != CurrentComponent)
					{
						Result = FPropertyAccess::MultipleValues;
						break;
					}
				}
				else
				{
					OutValue = ThisReference;
					CurrentComponent = OutValue.GetComponent(CurrentActor);
					Result = FPropertyAccess::Success;
				}
			}
			else if (Result == FPropertyAccess::Success)
			{
				Result = FPropertyAccess::MultipleValues;
				break;
			}
		}
	}
	return Result;
}

bool FComponentReferenceCustomization::IsComponentReferenceValid(const FComponentReference& Value) const
{
	if (!bAllowAnyActor && Value.OtherActor.IsValid())
	{
		return false;
	}

	AActor* CachedActor = CachedFilterActor.Get();
	if (UActorComponent* NewComponent = Value.GetComponent(CachedActor))
	{
		if (!IsFilteredComponent(NewComponent))
		{
			return false;
		}


		if (bAllowAnyActor)
		{
			if (NewComponent->GetOwner() == nullptr)
			{
				return false;
			}


			TArray<UObject*> ObjectList;
			PropertyHandle->GetOuterObjects(ObjectList);

			// Is the Outer object in the same world/level
			for (UObject* Obj : ObjectList)
			{
				AActor* Actor = Cast<AActor>(Obj);
				if (Actor == nullptr)
				{
					if (UActorComponent* ActorComponent = Cast<UActorComponent>(Obj))
					{
						Actor = ActorComponent->GetOwner();
					}
				}

				if (Actor)
				{
					if (NewComponent->GetOwner()->GetLevel() != Actor->GetLevel())
					{
						return false;
					}
				}
			}
		}
	}

	return true;
}

void FComponentReferenceCustomization::OnPropertyValueChanged()
{
	UpdateCachedValues(/*bResetValueIfInvalid*/ true);
}

void FComponentReferenceCustomization::UpdateCachedValues(bool bResetValueIfInvalid)
{
	CachedComponent.Reset();
	CachedFirstOuterActor = GetFirstOuterActor();
	CachedFilterActor = CachedFirstOuterActor;
	UWorld* PickerWorld = nullptr;
	RedirectActorToBPPreviewActor(CachedFilterActor, PickerWorld);

	FComponentReference TmpComponentReference;
	CachedPropertyAccess = GetValue(TmpComponentReference);
	if (CachedPropertyAccess != FPropertyAccess::Success)
	{
		return;
	}

	// If the reference is null, we don't want to update CachedComponent (the further code would
	//  otherwise try to set it to the root component of our actor)
	if (TmpComponentReference.PathToComponent.IsEmpty()
		&& TmpComponentReference.ComponentProperty == NAME_None
		&& TmpComponentReference.OverrideComponent.IsExplicitlyNull())
	{
		return;
	}

	CachedComponent = TmpComponentReference.GetComponent(CachedFilterActor.Get());
	if (!IsComponentReferenceValid(TmpComponentReference))
	{
		CachedComponent.Reset();
		if (bResetValueIfInvalid && !(TmpComponentReference == FComponentReference()))
		{
			SetValue(FComponentReference());
		}
	}
}

UActorComponent* FComponentReferenceCustomization::GetCachedComponent() const
{
	UActorComponent* Component = CachedComponent.Get();
	if (Component == nullptr)
	{
		const_cast<FComponentReferenceCustomization*>(this)->UpdateCachedValues(false);
	}
	return CachedComponent.Get();
}

int32 FComponentReferenceCustomization::OnGetComboContentWidgetIndex() const
{
	switch (CachedPropertyAccess)
	{
	case FPropertyAccess::MultipleValues: return 0;
	case FPropertyAccess::Success:
	default:
		return 1;
	}
}

bool FComponentReferenceCustomization::CanEdit() const
{
	return PropertyHandle.IsValid() ? !PropertyHandle->IsEditConst() : true;
}

bool FComponentReferenceCustomization::CanEditChildren() const
{
	return CanEdit() && (!bUseComponentPicker || !CachedFirstOuterActor.IsValid());
}

const FSlateBrush* FComponentReferenceCustomization::GetActorIcon() const
{
	if (UActorComponent* Component = GetCachedComponent())
	{
		if (AActor* Owner = Component->GetOwner())
		{
			return FSlateIconFinder::FindIconBrushForClass(Owner->GetClass());
		}
	}
	return FSlateIconFinder::FindIconBrushForClass(AActor::StaticClass());;
}

FText FComponentReferenceCustomization::OnGetActorName() const
{
	if (UActorComponent* Component = GetCachedComponent())
	{
		if (AActor* Owner = Component->GetOwner())
		{
			return FText::AsCultureInvariant(Owner->GetActorLabel());
		}
	}
	return LOCTEXT("NoActor", "None");
}

const FSlateBrush* FComponentReferenceCustomization::GetComponentIcon() const
{
	if (const UActorComponent* ActorComponent = GetCachedComponent())
	{
		return FSlateIconFinder::FindIconBrushForClass(ActorComponent->GetClass());
	}
	return FSlateIconFinder::FindIconBrushForClass(UActorComponent::StaticClass());
}

FText FComponentReferenceCustomization::OnGetComponentName() const
{
	if (CachedPropertyAccess == FPropertyAccess::Success)
	{
		if (UActorComponent* ActorComponent = GetCachedComponent())
		{
			const FName ComponentName = FComponentEditorUtils::FindVariableNameGivenComponentInstance(ActorComponent);
			const bool bIsArrayVariable = !ComponentName.IsNone() && ActorComponent->GetOwner() != nullptr && FindFProperty<FArrayProperty>(ActorComponent->GetOwner()->GetClass(), ComponentName);

			if (!ComponentName.IsNone() && !bIsArrayVariable)
			{
				return FText::FromName(ComponentName);
			}
			return FText::AsCultureInvariant(ActorComponent->GetName());
		}
	}
	else if (CachedPropertyAccess == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	return LOCTEXT("NoComponent", "None"); 
}

const FSlateBrush* FComponentReferenceCustomization::GetStatusIcon() const
{
	static FSlateNoResource EmptyBrush = FSlateNoResource();

	if (CachedPropertyAccess == FPropertyAccess::Fail)
	{
		return FAppStyle::GetBrush("Icons.Error");
	}
	return &EmptyBrush;
}

bool FComponentReferenceCustomization::RedirectActorToBPPreviewActor(TWeakObjectPtr<AActor>& InOutActor, UWorld*& OutWorld) const
{
	bool bShowTransient = false;
	if (AActor* Actor = InOutActor.Get())
	{
		if (Actor->HasAnyFlags(RF_ArchetypeObject))
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy))
			{
				FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
				for (TSharedRef<IBlueprintEditor>& EditorInterface : BlueprintEditorModule.GetBlueprintEditors())
				{
					TSharedRef<FBlueprintEditor> Editor = StaticCastSharedRef<FBlueprintEditor>(EditorInterface);
					if (Editor->GetBlueprintObj() == Blueprint)
					{
						if(AActor* PreviewActor = Editor->GetPreviewActor())
						{
							InOutActor = PreviewActor;
							OutWorld = PreviewActor->GetWorld();
							bShowTransient = true;
						}
						break;
					}
				}
			}
		}
	}
	return bShowTransient;
}

TSharedRef<SWidget> FComponentReferenceCustomization::OnGetMenuContent()
{
	UActorComponent* InitialComponent = CachedComponent.Get();

	// If the actor we are looking at is a BP, use its BP editor preview world, if any
	CachedFilterActor = CachedFirstOuterActor;
	UWorld* PickerWorld = nullptr;
	bool bShowTransient = RedirectActorToBPPreviewActor(CachedFilterActor, PickerWorld);

	return PropertyCustomizationHelpers::MakeComponentPickerWithMenu(InitialComponent
		, bAllowClear
		, FOnShouldFilterActor::CreateSP(this, &FComponentReferenceCustomization::IsFilteredActor)
		, FOnShouldFilterComponent::CreateSP(this, &FComponentReferenceCustomization::IsFilteredComponent)
		, FOnComponentSelected::CreateSP(this, &FComponentReferenceCustomization::OnComponentSelected)
		, FSimpleDelegate::CreateSP(this, &FComponentReferenceCustomization::CloseComboButton)
		, PickerWorld
		, bShowTransient);
}

void FComponentReferenceCustomization::OnMenuOpenChanged(bool bOpen)
{
	if (!bOpen)
	{
		ComponentComboButton->SetMenuContent(SNullWidget::NullWidget);
	}
}

bool FComponentReferenceCustomization::IsFilteredActor(const AActor* const Actor) const
{
	return bAllowAnyActor || Actor == CachedFilterActor.Get();
}

bool FComponentReferenceCustomization::IsFilteredComponent(const UActorComponent* const Component) const
{
	const USceneComponent* SceneComp = Cast<USceneComponent>(Component);
	const USceneComponent* ParentSceneComp = SceneComp != nullptr ? SceneComp->GetAttachParent() : nullptr;
	const AActor* FilterActor = CachedFilterActor.Get();

	return Component->GetOwner()
		&& (bAllowAnyActor || Component->GetOwner() == FilterActor)
		&& (!bAllowAnyActor || (FilterActor && Component->GetOwner()->GetLevel() == FilterActor->GetLevel()))
		&& FComponentEditorUtils::CanEditComponentInstance(Component, SceneComp, false)
		&& IsFilteredObject(Component, AllowedComponentClassFilters, DisallowedComponentClassFilters)
		&& IsFilteredObject(Component->GetOwner(), AllowedActorClassFilters, DisallowedActorClassFilters);
}

bool FComponentReferenceCustomization::IsFilteredObject(const UObject* const Object, const TArray<const UClass*>& AllowedFilters, const TArray<const UClass*>& DisallowedFilters)
{
	bool bAllowedToSetBasedOnFilter = true;

	const UClass* ObjectClass = Object->GetClass();
	if (AllowedFilters.Num() > 0)
	{
		bAllowedToSetBasedOnFilter = false;
		for (const UClass* AllowedClass : AllowedFilters)
		{
			const bool bAllowedClassIsInterface = AllowedClass->HasAnyClassFlags(CLASS_Interface);
			if (ObjectClass->IsChildOf(AllowedClass) || (bAllowedClassIsInterface && ObjectClass->ImplementsInterface(AllowedClass)))
			{
				bAllowedToSetBasedOnFilter = true;
				break;
			}
		}
	}

	if (DisallowedFilters.Num() > 0 && bAllowedToSetBasedOnFilter)
	{
		for (const UClass* DisallowedClass : DisallowedFilters)
		{
			const bool bDisallowedClassIsInterface = DisallowedClass->HasAnyClassFlags(CLASS_Interface);
			if (ObjectClass->IsChildOf(DisallowedClass) || (bDisallowedClassIsInterface && ObjectClass->ImplementsInterface(DisallowedClass)))
			{
				bAllowedToSetBasedOnFilter = false;
				break;
			}
		}
	}

	return bAllowedToSetBasedOnFilter;
}

void FComponentReferenceCustomization::OnComponentSelected(const UActorComponent* InComponent)
{
	ComponentComboButton->SetIsOpen(false);

	FComponentReference ComponentReference = FComponentEditorUtils::MakeComponentReference(CachedFilterActor.Get(), InComponent);
	SetValue(ComponentReference);
}

void FComponentReferenceCustomization::CloseComboButton()
{
	ComponentComboButton->SetIsOpen(false);
}

#undef LOCTEXT_NAMESPACE
