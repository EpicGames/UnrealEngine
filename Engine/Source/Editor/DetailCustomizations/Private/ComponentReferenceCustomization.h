// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class IPropertyHandle;
class SComboButton;
class SWidget;
class UActorComponent;
class UClass;
class UObject;
struct FComponentReference;
struct FSoftComponentReference;
struct FSlateBrush;

class FComponentReferenceCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this customization for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

private:
	/** From the property metadata, build the list of allowed and disallowed class. */
	void BuildClassFilters();

	/** Build the combobox widget. */
	void BuildComboBox();

	/**
	 * From the Detail panel outer hierarchy, find the first actor or component owner we find.
	 * This is use in case we want only component on the Self actor and to check if we did a cross-level reference.
	 */
	AActor* GetFirstOuterActor() const;

	/**
	 * Set the value of the asset referenced by this property editor.
	 * Will set the underlying property handle if there is one.
	 */
	void SetValue(const FComponentReference& Value);

	/** Get the value referenced by this widget. */
	FPropertyAccess::Result GetValue(FComponentReference& OutValue) const;

	/** Is the Value valid */
	bool IsComponentReferenceValid(const FComponentReference& Value) const;

	/** Callback when the property value changed. */
	void OnPropertyValueChanged();

private:
	/**
	 * Return 0, if we have multiple values to edit.
	 * Return 1, if we display the widget normally.
	 */
	int32 OnGetComboContentWidgetIndex() const;

	bool CanEdit() const;
	bool CanEditChildren() const;

	const FSlateBrush* GetActorIcon() const;
	FText OnGetActorName() const;
	const FSlateBrush* GetComponentIcon() const;
	FText OnGetComponentName() const;
	const FSlateBrush* GetStatusIcon() const;

	/**
	 * Get the content to be displayed in the asset/actor picker menu
	 * @returns the widget for the menu content
	 */
	TSharedRef<SWidget> OnGetMenuContent();

	/**
	 * Called when the asset menu is closed, we handle this to force the destruction of the asset menu to
	 * ensure any settings the user set are saved.
	 */
	void OnMenuOpenChanged(bool bOpen);

	/**
	 * Returns whether the actor/component should be filtered out from selection.
	 */
	bool IsFilteredActor(const AActor* const Actor) const;
	bool IsFilteredComponent(const UActorComponent* const Component) const;
	static bool IsFilteredObject(const UObject* const Object, const TArray<const UClass*>& AllowedFilters, const TArray<const UClass*>& DisallowedFilters);

	/**
	 * Delegate for handling selection in the scene outliner.
	 * @param	InActor	The chosen component
	 */
	void OnComponentSelected(const UActorComponent* InComponent);

	/**
	 * Closes the combo button.
	 */
	void CloseComboButton();

	/**
	 * Updates CachedFirstOuterActor, CachedComponent, and CachedPropertyAccess
	 * @param bResetValueIfInvalid If true and the resulting component doesn't match the filters, do
	 *  a SetValue call to reset the reference to null.
	 */
	void UpdateCachedValues(bool bResetValueIfInvalid);

	// Redirects the found actor to a BP preview actor to be able to choose components in a BP editor context
	bool RedirectActorToBPPreviewActor(TWeakObjectPtr<AActor>& InOutActor, UWorld*& OutWorld) const;

	// Gets (or updates) the cached component
	UActorComponent* GetCachedComponent() const;

private:
	/** The property handle we are customizing */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Main combo button */
	TSharedPtr<SComboButton> ComponentComboButton;

	/** Classes that can be used with this property */
	TArray<const UClass*> AllowedActorClassFilters;
	TArray<const UClass*> AllowedComponentClassFilters;

	/** Classes that can NOT be used with this property */
	TArray<const UClass*> DisallowedActorClassFilters;
	TArray<const UClass*> DisallowedComponentClassFilters;

	/** Whether the asset can be 'None' in this case */
	bool bAllowClear;

	/** Can the actor be different/selected */
	bool bAllowAnyActor;

	/** Do it has the UseComponentPicker metadata */
	bool bUseComponentPicker;

	/** Whether or not the component reference is a FSoftComponentReference */
	bool bIsSoftReference;

	/** Cached ComponentReference */
	TWeakObjectPtr<AActor> CachedFirstOuterActor;
	TWeakObjectPtr<AActor> CachedFilterActor;
	TWeakObjectPtr<UActorComponent> CachedComponent;
	FPropertyAccess::Result CachedPropertyAccess;
};
