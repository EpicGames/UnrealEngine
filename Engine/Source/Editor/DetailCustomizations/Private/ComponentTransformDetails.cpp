// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentTransformDetails.h"

#include "Algo/Transform.h"
#include "Components/SceneComponent.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "IPropertyUtilities.h"
#include "PropertyEditorArchetypePolicy.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Layout/Margin.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnitConversion.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/NotifyHook.h"
#include "PropertyEditorCopyPaste.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Settings/EditorProjectSettings.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "UnrealEdGlobals.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include <utility>

class SWidget;
class UWorld;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FComponentTransformDetails"

namespace UE::DetailsCustomizations::Internal
{
	/** Lookup to get the property name for the given TransformField. */
	static TMap<ETransformField::Type, FString> TransformFieldToPropertyNameString = {
		{ ETransformField::Location, USceneComponent::GetRelativeLocationPropertyName().ToString() },
		{ ETransformField::Rotation, USceneComponent::GetRelativeRotationPropertyName().ToString() },
		{ ETransformField::Scale, USceneComponent::GetRelativeScale3DPropertyName().ToString() }		
	};
}

class FScopedSwitchWorldForObject
{
public:
	FScopedSwitchWorldForObject( UObject* Object )
		: PrevWorld( NULL )
	{
		bool bRequiresPlayWorld = false;
		if( GUnrealEd->PlayWorld && !GIsPlayInEditorWorld )
		{
			UPackage* ObjectPackage = Object->GetOutermost();
			bRequiresPlayWorld = ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor);
		}

		if( bRequiresPlayWorld )
		{
			PrevWorld = SetPlayInEditorWorld( GUnrealEd->PlayWorld );
		}
	}

	~FScopedSwitchWorldForObject()
	{
		if( PrevWorld )
		{
			RestoreEditorWorld( PrevWorld );
		}
	}

private:
	UWorld* PrevWorld;
};

static USceneComponent* GetSceneComponentFromDetailsObject(UObject* InObject)
{
	AActor* Actor = Cast<AActor>(InObject);
	if(Actor)
	{
		return Actor->GetRootComponent();
	}
	
	return Cast<USceneComponent>(InObject);
}

namespace ComponentTransformDetails::Private
{
	bool AreRotationsEqual(const FVector& Lhs, const FVector& Rhs)
	{
		constexpr double RotationEpsilon = 1.e-4;
		
		const double AbsDiffX = FMath::Abs(Lhs.X - Rhs.X);
		const double AbsDiffY = FMath::Abs(Lhs.Y - Rhs.Y);
		const double AbsDiffZ = FMath::Abs(Lhs.Z - Rhs.Z);
							
		return AbsDiffX < RotationEpsilon && AbsDiffY < RotationEpsilon && AbsDiffZ < RotationEpsilon;
	}
}

FComponentTransformDetails::FComponentTransformDetails( const TArray< TWeakObjectPtr<UObject> >& InSelectedObjects, const FSelectedActorInfo& InSelectedActorInfo, IDetailLayoutBuilder& DetailBuilder )
	: TNumericUnitTypeInterface(GetDefault<UEditorProjectAppearanceSettings>()->bDisplayUnitsOnComponentTransforms ? EUnit::Centimeters : EUnit::Unspecified)
	, SelectedActorInfo( InSelectedActorInfo )
	, SelectedObjects( InSelectedObjects )
	, NotifyHook( DetailBuilder.GetPropertyUtilities()->GetNotifyHook() )
	, bPreserveScaleRatio( false )
	, bEditingRotationInUI( false )
	, bIsSliderTransaction( false )
	, HiddenFieldMask( 0 )
	, bIsEnabledCache( false )
	, bIsAxisDisplayLeftUpForward( AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward )
{
	GConfig->GetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FComponentTransformDetails::OnObjectsReplaced);
}

FComponentTransformDetails::~FComponentTransformDetails()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

TSharedRef<SWidget> FComponentTransformDetails::BuildTransformFieldLabel( ETransformField::Type TransformField )
{
	FText Label;
	switch( TransformField )
	{
	case ETransformField::Rotation:
		Label = LOCTEXT( "RotationLabel", "Rotation");
		break;
	case ETransformField::Scale:
		Label = LOCTEXT( "ScaleLabel", "Scale" );
		break;
	case ETransformField::Location:
	default:
		Label = LOCTEXT("LocationLabel", "Location");
		break;
	}

	FMenuBuilder MenuBuilder( true, NULL, NULL );

	FUIAction SetRelativeLocationAction
	(
		FExecuteAction::CreateSP( this, &FComponentTransformDetails::OnSetAbsoluteTransform, TransformField, false ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &FComponentTransformDetails::IsAbsoluteTransformChecked, TransformField, false )
	);

	FUIAction SetWorldLocationAction
	(
		FExecuteAction::CreateSP( this, &FComponentTransformDetails::OnSetAbsoluteTransform, TransformField, true ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &FComponentTransformDetails::IsAbsoluteTransformChecked, TransformField, true )
	);

	MenuBuilder.BeginSection( TEXT("TransformType"), FText::Format( LOCTEXT("TransformType", "{0} Type"), Label ) );

	MenuBuilder.AddMenuEntry
	(
		FText::Format( LOCTEXT( "RelativeLabel", "Relative"), Label ),
		FText::Format( LOCTEXT( "RelativeLabel_ToolTip", "{0} is relative to its parent"), Label ),
		FSlateIcon(),
		SetRelativeLocationAction,
		NAME_None, 
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		FText::Format( LOCTEXT( "WorldLabel", "World"), Label ),
		FText::Format( LOCTEXT( "WorldLabel_ToolTip", "{0} is relative to the world"), Label ),
		FSlateIcon(),
		SetWorldLocationAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.EndSection();

	TSharedRef<SHorizontalBox> NameContent =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.ContentPadding(0.f)
			.IsEnabled(this, &FComponentTransformDetails::CanChangeAbsoluteFlag, TransformField)
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
			.ButtonContent()
			[
				SNew( SBox )
				.Padding( FMargin( 0.0f, 0.0f, 2.0f, 0.0f ) )
				.MinDesiredWidth(50.f)
				[
					SNew(STextBlock)
					.Text(this, &FComponentTransformDetails::GetTransformFieldText, TransformField)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		];
	
	if (TransformField == ETransformField::Scale)
	{
		NameContent->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			[
				// Add a checkbox to toggle between preserving the ratio of x,y,z components of scale when a value is entered
				SNew(SCheckBox)
				.IsChecked(this, &FComponentTransformDetails::IsPreserveScaleRatioChecked)
				.IsEnabled(this, &FComponentTransformDetails::GetIsScaleEnabled)
				.OnCheckStateChanged(this, &FComponentTransformDetails::OnPreserveScaleRatioToggled)
				.Style(FAppStyle::Get(), "TransparentCheckBox")
				.ToolTipText(LOCTEXT("PreserveScaleToolTip", "When locked, all axis values scale together so the object maintains its proportions in all directions."))
				[
					SNew(SImage)
					.Image(this, &FComponentTransformDetails::GetPreserveScaleRatioImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return NameContent;
}

FText FComponentTransformDetails::GetTransformFieldText( ETransformField::Type TransformField ) const
{
	switch (TransformField)
	{
	case ETransformField::Location:
		return GetLocationText();
	case ETransformField::Rotation:
		return GetRotationText();
	case ETransformField::Scale:
		return GetScaleText();
	default:
		return FText::GetEmpty();
	}
}

bool FComponentTransformDetails::OnCanCopy( ETransformField::Type TransformField ) const
{
	// We can only copy values if the whole field is set.  If multiple values are defined we do not copy since we are unable to determine the value
	switch (TransformField)
	{
	case ETransformField::Location:
		return CachedLocation.IsSet();
	case ETransformField::Rotation:
		return CachedRotation.IsSet();
	case ETransformField::Scale:
		return CachedScale.IsSet();
	default:
		return false;
	}
}

void FComponentTransformDetails::OnCopy( ETransformField::Type TransformField )
{
	CacheDetails();

	FString CopyStr;

	CopyToText(CopyStr, TransformField);

	if( !CopyStr.IsEmpty() )
	{
		FPlatformApplicationMisc::ClipboardCopy( *CopyStr );
	}
}

void FComponentTransformDetails::OnCopyToText(TSharedPtr<IPropertyHandle> PropertyHandle, FString& Text, const TOptional<FGuid>& InOperationId,
	ETransformField::Type InTransformField)
{
	const FString PropertyPath = UE::PropertyEditor::GetPropertyPath(PropertyHandle);

	// ensure that if tag is specified, that it matches the subscriber
	if (!PropertyPath.Equals(UE::DetailsCustomizations::Internal::TransformFieldToPropertyNameString[InTransformField]))
	{
		return;
	}

	CopyToText(Text, InTransformField);
}

void FComponentTransformDetails::CopyToText(FString& CopyStr, ETransformField::Type InTransformField)
{
	switch (InTransformField)
	{
	case ETransformField::Location:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), GetLocationX().GetValue(), GetLocationY().GetValue(), GetLocationZ().GetValue());
		break;
	case ETransformField::Rotation:
		CopyStr = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), CachedRotation.Y.GetValue(), CachedRotation.Z.GetValue(), CachedRotation.X.GetValue());
		break;
	case ETransformField::Scale:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), CachedScale.X.GetValue(), CachedScale.Y.GetValue(), CachedScale.Z.GetValue());
		break;
	default:
		break;
	}

}

void FComponentTransformDetails::OnPaste( ETransformField::Type TransformField )
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	PasteFromText(TEXT(""), PastedText, TransformField);
}

void FComponentTransformDetails::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	ETransformField::Type InTransformField)
{
	PasteFromText(InTag, InText, InTransformField);
}

void FComponentTransformDetails::PasteFromText(
	const FString& InTag,
	const FString& InText,
	ETransformField::Type InTransformField)
{
	if (InText.IsEmpty())
	{
		return;
	}

	FString Text = InText;
	if (!InTag.IsEmpty())
	{
		const FString PropertyPath = UE::PropertyEditor::GetPropertyPath(GetPropertyHandle());

		// ensure that if tag is specified, that it matches the subscriber
		if (!InTag.Equals(UE::DetailsCustomizations::Internal::TransformFieldToPropertyNameString[InTransformField]))
		{
			return;
		}
	}

	switch (InTransformField)
	{
	case ETransformField::Location:
		{
			FVector Location;
			if (Location.InitFromString(Text))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteLocation", "Paste Location"));
				OnSetTransform(ETransformField::Location, EAxisList::All, Location, false, true);
			}
		}
		break;
	case ETransformField::Rotation:
		{
			FRotator Rotation;
			Text.ReplaceInline(TEXT("Pitch="), TEXT("P="));
			Text.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
			Text.ReplaceInline(TEXT("Roll="), TEXT("R="));
			if (Rotation.InitFromString(Text))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
				OnSetTransform(ETransformField::Rotation, EAxisList::All, Rotation.Euler(), false, true);
			}
		}
		break;
	case ETransformField::Scale:
		{
			FVector Scale;
			if (Scale.InitFromString(Text))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteScale", "Paste Scale"));
				OnSetTransform(ETransformField::Scale, EAxisList::All, Scale, false, true);
			}
		}
		break;
	default:
		break;
	}
}

FUIAction FComponentTransformDetails::CreateCopyAction( ETransformField::Type TransformField ) const
{
	return
		FUIAction
		(
			FExecuteAction::CreateSP(const_cast<FComponentTransformDetails*>(this), &FComponentTransformDetails::OnCopy, TransformField ),
			FCanExecuteAction::CreateSP(const_cast<FComponentTransformDetails*>(this), &FComponentTransformDetails::OnCanCopy, TransformField )
		);
}

FUIAction FComponentTransformDetails::CreatePasteAction( ETransformField::Type TransformField ) const
{
	return 
		 FUIAction( FExecuteAction::CreateSP(const_cast<FComponentTransformDetails*>(this), &FComponentTransformDetails::OnPaste, TransformField ) );
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FComponentTransformDetails::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	UClass* SceneComponentClass = USceneComponent::StaticClass();

	FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

	const bool bHideLocationField = ( HiddenFieldMask & ( 1 << ETransformField::Location ) ) != 0;
	const bool bHideRotationField = ( HiddenFieldMask & ( 1 << ETransformField::Rotation ) ) != 0;
	const bool bHideScaleField = ( HiddenFieldMask & ( 1 << ETransformField::Scale ) ) != 0;

	IDetailCategoryBuilder& ParentCategory = ChildrenBuilder.GetParentCategory();
	
	IDetailLayoutBuilder& LayoutBuilder = ParentCategory.GetParentLayout();
	TSharedPtr<IPropertyHandle> LocationPropertyHandle = LayoutBuilder.GetProperty(USceneComponent::GetRelativeLocationPropertyName(), USceneComponent::StaticClass());
	TSharedPtr<IPropertyHandle> RotationPropertyHandle = LayoutBuilder.GetProperty(USceneComponent::GetRelativeRotationPropertyName(), USceneComponent::StaticClass());
	TSharedPtr<IPropertyHandle> ScalePropertyHandle = LayoutBuilder.GetProperty(USceneComponent::GetRelativeScale3DPropertyName(), USceneComponent::StaticClass());

	const FString& MetaLocationDeltaString = LocationPropertyHandle->GetMetaData("Delta");
	const FString& MetaRotationDeltaString = RotationPropertyHandle->GetMetaData("Delta");
	const FString& MetaRotationMinString = RotationPropertyHandle->GetMetaData("UIMin");
	const FString& MetaRotationMaxString = RotationPropertyHandle->GetMetaData("UIMax");
	const FString& MetaScaleDeltaString = ScalePropertyHandle->GetMetaData("Delta");

	float LocationSpinDelta = !MetaLocationDeltaString.IsEmpty() ? FCString::Atof(*MetaLocationDeltaString) : 1.f;
	float RotationSpinDelta = !MetaRotationDeltaString.IsEmpty() ? FCString::Atof(*MetaRotationDeltaString) : 1.f;
	TOptional<FRotator::FReal> RotationMin = !MetaRotationMinString.IsEmpty() ? FCString::Atof(*MetaRotationMinString) : TOptional<FRotator::FReal>();
	TOptional<FRotator::FReal> RotationMax = !MetaRotationMaxString.IsEmpty() ? FCString::Atof(*MetaRotationMaxString) : TOptional<FRotator::FReal>();
	float ScaleSpinDelta = !MetaScaleDeltaString.IsEmpty() ? FCString::Atof(*MetaScaleDeltaString) : 0.0025f;

	// Location
	if(!bHideLocationField)
	{
		TSharedPtr<INumericTypeInterface<FVector::FReal>> TypeInterface;
		if( FUnitConversion::Settings().ShouldDisplayUnits() )
		{
			TypeInterface = SharedThis(this);
		}

		ParentCategory.OnPasteFromText()->AddSP(this, &FComponentTransformDetails::OnPasteFromText, ETransformField::Location);
		ParentCategory.OnCopyToText()->AddSP(this, &FComponentTransformDetails::OnCopyToText, ETransformField::Location);

		FindOrCreatePropertyHandle(USceneComponent::GetAbsoluteLocationPropertyName(), ChildrenBuilder);

		TSharedPtr<IPropertyHandle> PropertyHandle = FindOrCreatePropertyHandle(USceneComponent::GetRelativeLocationPropertyName(), ChildrenBuilder);

		ChildrenBuilder.AddCustomRow( LOCTEXT("LocationFilter", "Location") )
		.RowTag("Location")
		.CopyAction( CreateCopyAction( ETransformField::Location ) )
		.PasteAction( CreatePasteAction( ETransformField::Location ) )
		.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>(this, &FComponentTransformDetails::GetLocationResetVisibility), FSimpleDelegate::CreateSP(this, &FComponentTransformDetails::OnLocationResetClicked)))
		.PropertyHandleList({ PropertyHandle })
		.IsEnabled(TAttribute<bool>(this, &FComponentTransformDetails::GetIsEnabled))
		.NameContent()
		.VAlign(VAlign_Center)
		[
			BuildTransformFieldLabel( ETransformField::Location )
		]
		.ValueContent()
		.MinDesiredWidth(125.0f * 3.0f)
		.MaxDesiredWidth(125.0f * 3.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SNumericVectorInputBox<FVector::FReal>)
			.X(this, &FComponentTransformDetails::GetLocationX)
			.Y(this, &FComponentTransformDetails::GetLocationY)
			.Z(this, &FComponentTransformDetails::GetLocationZ)
			.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
			.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
			.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
			.bColorAxisLabels(true)
			.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
			.IsEnabled(this, &FComponentTransformDetails::GetIsLocationEnabled)
			.OnXChanged(this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Location, EAxisList::X, false)
			.OnYChanged(this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Location, EAxisList::Y, false)
			.OnZChanged(this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Location, EAxisList::Z, false)
			.OnXCommitted(this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Location, EAxisList::X, true)
			.OnYCommitted(this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Location, EAxisList::Y, true)
			.OnZCommitted(this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Location, EAxisList::Z, true)
			.Font(FontInfo)
			.TypeInterface(TypeInterface)
			.AllowSpin(SelectedObjects.Num() == 1)
			.SpinDelta(LocationSpinDelta)
			.OnBeginSliderMovement(this, &FComponentTransformDetails::OnBeginLocationSlider)
			.OnEndSliderMovement(this, &FComponentTransformDetails::OnEndLocationSlider)
			.PreventThrottling(true)
		];
	}
	
	// Rotation
	if(!bHideRotationField)
	{
		TSharedPtr<INumericTypeInterface<FRotator::FReal>> TypeInterface;
		if( FUnitConversion::Settings().ShouldDisplayUnits() )
		{
			TypeInterface = MakeShareable( new TNumericUnitTypeInterface<FRotator::FReal>(EUnit::Degrees) );
			if (bIsAxisDisplayLeftUpForward)
			{
				TypeInterface->SetMaxFractionalDigits(3);
				TypeInterface->SetIndicateNearlyInteger(false);
			}
		}

		ParentCategory.OnPasteFromText()->AddSP(this, &FComponentTransformDetails::OnPasteFromText, ETransformField::Rotation);
		ParentCategory.OnCopyToText()->AddSP(this, &FComponentTransformDetails::OnCopyToText, ETransformField::Rotation);

		FindOrCreatePropertyHandle(USceneComponent::GetAbsoluteRotationPropertyName(), ChildrenBuilder);

		TSharedPtr<IPropertyHandle> PropertyHandle = FindOrCreatePropertyHandle(USceneComponent::GetRelativeRotationPropertyName(), ChildrenBuilder);

		ChildrenBuilder.AddCustomRow( LOCTEXT("RotationFilter", "Rotation") )
		.RowTag("Rotation")
		.CopyAction( CreateCopyAction(ETransformField::Rotation) )
		.PasteAction( CreatePasteAction(ETransformField::Rotation) )
		.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>(this, &FComponentTransformDetails::GetRotationResetVisibility), FSimpleDelegate::CreateSP(this, &FComponentTransformDetails::OnRotationResetClicked)))
		.PropertyHandleList({ PropertyHandle })
		.IsEnabled(TAttribute<bool>(this, &FComponentTransformDetails::GetIsEnabled))
		.NameContent()
		.VAlign(VAlign_Center)
		[
			BuildTransformFieldLabel(ETransformField::Rotation)
		]
		.ValueContent()
		.MinDesiredWidth(125.0f * 3.0f)
		.MaxDesiredWidth(125.0f * 3.0f)
		.VAlign(VAlign_Center)
		[
			SNew( SNumericRotatorInputBox<FRotator::FReal> )
			.AllowSpin( SelectedObjects.Num() == 1 ) 
			.SpinDelta(RotationSpinDelta)
			.MinSliderValue(RotationMin)
			.MaxSliderValue(RotationMax)
			.Roll( this, &FComponentTransformDetails::GetRotationX )
			.Pitch( this, &FComponentTransformDetails::GetRotationY )
			.Yaw( this, &FComponentTransformDetails::GetRotationZ )
			.RollDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Forward))
			.PitchDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Left))
			.YawDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Up))
			.bColorAxisLabels( true )
			.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
			.IsEnabled( this, &FComponentTransformDetails::GetIsRotationEnabled )
			.OnPitchBeginSliderMovement( this, &FComponentTransformDetails::OnBeginRotationSlider )
			.OnYawBeginSliderMovement( this, &FComponentTransformDetails::OnBeginRotationSlider )
			.OnRollBeginSliderMovement( this, &FComponentTransformDetails::OnBeginRotationSlider )
			.OnPitchEndSliderMovement( this, &FComponentTransformDetails::OnEndRotationSlider )
			.OnYawEndSliderMovement( this, &FComponentTransformDetails::OnEndRotationSlider )
			.OnRollEndSliderMovement( this, &FComponentTransformDetails::OnEndRotationSlider )
			.OnRollChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Rotation, EAxisList::X, false )
			.OnPitchChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Rotation, EAxisList::Y, false )
			.OnYawChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Rotation, EAxisList::Z, false )
			.OnRollCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Rotation, EAxisList::X, true )
			.OnPitchCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Rotation, EAxisList::Y, true )
			.OnYawCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Rotation, EAxisList::Z, true )
			.TypeInterface( TypeInterface )
			.Font( FontInfo )
			.PreventThrottling(true)
		];
	}
	
	// Scale
	if(!bHideScaleField)
	{
		ParentCategory.OnPasteFromText()->AddSP(this, &FComponentTransformDetails::OnPasteFromText, ETransformField::Scale);
		ParentCategory.OnCopyToText()->AddSP(this, &FComponentTransformDetails::OnCopyToText, ETransformField::Scale);

		FindOrCreatePropertyHandle(USceneComponent::GetAbsoluteScalePropertyName(), ChildrenBuilder);

		TSharedPtr<IPropertyHandle> PropertyHandle = FindOrCreatePropertyHandle(USceneComponent::GetRelativeScale3DPropertyName(), ChildrenBuilder);

		ChildrenBuilder.AddCustomRow( LOCTEXT("ScaleFilter", "Scale") )
		.RowTag("Scale")
		.CopyAction( CreateCopyAction(ETransformField::Scale) )
		.PasteAction( CreatePasteAction(ETransformField::Scale) )
		.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>(this, &FComponentTransformDetails::GetScaleResetVisibility), FSimpleDelegate::CreateSP(this, &FComponentTransformDetails::OnScaleResetClicked)))
		.PropertyHandleList({ PropertyHandle })
		.IsEnabled(TAttribute<bool>(this, &FComponentTransformDetails::GetIsEnabled))
		.NameContent()
		.VAlign(VAlign_Center)
		[
			BuildTransformFieldLabel(ETransformField::Scale)
		]
		.ValueContent()
		.MinDesiredWidth(125.0f * 3.0f)
		.MaxDesiredWidth(125.0f * 3.0f)
		.VAlign(VAlign_Center)
		[
			SNew( SNumericVectorInputBox<FVector::FReal> )
			.X( this, &FComponentTransformDetails::GetScaleX )
			.Y( this, &FComponentTransformDetails::GetScaleY )
			.Z( this, &FComponentTransformDetails::GetScaleZ )
			.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
			.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
			.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
			.bColorAxisLabels( true )
			.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
			.IsEnabled( this, &FComponentTransformDetails::GetIsScaleEnabled )
			.OnXChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Scale, EAxisList::X, false )
			.OnYChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Scale, EAxisList::Y, false )
			.OnZChanged( this, &FComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, ETransformField::Scale, EAxisList::Z, false )
			.OnXCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Scale, EAxisList::X, true )
			.OnYCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Scale, EAxisList::Y, true )
			.OnZCommitted( this, &FComponentTransformDetails::OnSetTransformAxis, ETransformField::Scale, EAxisList::Z, true )
			.ContextMenuExtenderX( this, &FComponentTransformDetails::ExtendXScaleContextMenu )
			.ContextMenuExtenderY( this, &FComponentTransformDetails::ExtendYScaleContextMenu )
			.ContextMenuExtenderZ( this, &FComponentTransformDetails::ExtendZScaleContextMenu )
			.Font( FontInfo )
			.AllowSpin( SelectedObjects.Num() == 1 )
			.SpinDelta( ScaleSpinDelta )
			.OnBeginSliderMovement( this, &FComponentTransformDetails::OnBeginScaleSlider )
			.OnEndSliderMovement(this, &FComponentTransformDetails::OnEndScaleSlider)
			.PreventThrottling(true)
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FComponentTransformDetails::Tick( float DeltaTime ) 
{
	CacheDetails();
	if (!FixedDisplayUnits.IsSet())
	{
		CacheCommonLocationUnits();
	}
}

void FComponentTransformDetails::CacheCommonLocationUnits()
{
	const TOptional<FVector::FReal> LocationX = GetLocationX();
	const TOptional<FVector::FReal> LocationY = GetLocationY();
	const TOptional<FVector::FReal> LocationZ = GetLocationZ();
	FVector::FReal LargestValue = 0.0;
	if (LocationX.IsSet() && LocationX.GetValue() > LargestValue)
	{
		LargestValue = LocationX.GetValue();
	}
	if (LocationY.IsSet() && LocationY.GetValue() > LargestValue)
	{
		LargestValue = LocationY.GetValue();
	}
	if (LocationZ.IsSet() && LocationZ.GetValue() > LargestValue)
	{
		LargestValue = LocationZ.GetValue();
	}

	SetupFixedDisplay(LargestValue);
}

TSharedPtr<IPropertyHandle> FComponentTransformDetails::FindOrCreatePropertyHandle(FName PropertyName, IDetailChildrenBuilder& ChildrenBuilder)
{
	if (TSharedPtr<IPropertyHandle>* HandlePtr = PropertyHandles.Find(PropertyName))
	{
		return *HandlePtr;
	}
	
	// Try finding the property handle in the details panel's property map first.
	IDetailLayoutBuilder& LayoutBuilder = ChildrenBuilder.GetParentCategory().GetParentLayout();
	TSharedPtr<IPropertyHandle> PropertyHandle = LayoutBuilder.GetProperty(PropertyName, USceneComponent::StaticClass());
	if (!PropertyHandle || !PropertyHandle->IsValidHandle())
	{
		// If it wasn't found, add a collapsed row which contains the property node.
		TArray<UObject*> SceneComponents;
		Algo::Transform(SelectedObjects, SceneComponents, [](TWeakObjectPtr<UObject> Obj) { return GetSceneComponentFromDetailsObject(Obj.Get()); });
		PropertyHandle = LayoutBuilder.AddObjectPropertyData(SceneComponents, PropertyName);
		CachedHandlesObjects.Append(SceneComponents);
	}

	if (PropertyHandle && PropertyHandle->IsValidHandle())
	{
		PropertyHandles.Add(PropertyName, PropertyHandle);
	}
	return PropertyHandle;
}

void FComponentTransformDetails::UpdatePropertyHandlesObjects(const TArray<UObject*> NewSceneComponents)
{
	// Cached the old handles objects.
	CachedHandlesObjects.Reset(NewSceneComponents.Num());
	Algo::Transform(NewSceneComponents, CachedHandlesObjects, [](UObject* Obj) { return TWeakObjectPtr<UObject>(Obj); });

	for (TMap<FName, TSharedPtr<IPropertyHandle>>::TIterator It(PropertyHandles); It; ++It)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = It.Value();
		if (PropertyHandle && PropertyHandle->IsValidHandle())
		{
			PropertyHandle->ReplaceOuterObjects(NewSceneComponents);
		}
	}
}

bool FComponentTransformDetails::GetIsEnabled() const
{
	return bIsEnabledCache;
}

bool FComponentTransformDetails::GetIsLocationEnabled() const
{
	return GetIsTransformComponentEnabled(USceneComponent::GetRelativeLocationPropertyName());
}

bool FComponentTransformDetails::GetIsRotationEnabled() const
{
	return GetIsTransformComponentEnabled(USceneComponent::GetRelativeRotationPropertyName());
}

bool FComponentTransformDetails::GetIsScaleEnabled() const
{
	return GetIsTransformComponentEnabled(USceneComponent::GetRelativeScale3DPropertyName());
}

bool FComponentTransformDetails::GetIsTransformComponentEnabled(FName ComponentName) const
{
	if (GetIsEnabled())
	{
		if (const TSharedPtr<IPropertyHandle>* PropertyHandle = PropertyHandles.Find(ComponentName))
		{
			return (*PropertyHandle)->IsEditable();
		}
	}
	return false;
}

const FSlateBrush* FComponentTransformDetails::GetPreserveScaleRatioImage() const
{
	return bPreserveScaleRatio ? FAppStyle::GetBrush( TEXT("Icons.Lock") ) : FAppStyle::GetBrush( TEXT("Icons.Unlock") ) ;
}

ECheckBoxState FComponentTransformDetails::IsPreserveScaleRatioChecked() const
{
	return bPreserveScaleRatio ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FComponentTransformDetails::OnPreserveScaleRatioToggled( ECheckBoxState NewState )
{
	bPreserveScaleRatio = (NewState == ECheckBoxState::Checked) ? true : false;
	GConfig->SetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
}

FText FComponentTransformDetails::GetLocationText() const
{
	return bAbsoluteLocation ? LOCTEXT( "AbsoluteLocation", "Absolute Location" ) : LOCTEXT( "Location", "Location" );
}

FText FComponentTransformDetails::GetRotationText() const
{
	return bAbsoluteRotation ? LOCTEXT( "AbsoluteRotation", "Absolute Rotation" ) : LOCTEXT( "Rotation", "Rotation" );
}

FText FComponentTransformDetails::GetScaleText() const
{
	return bAbsoluteScale ? LOCTEXT( "AbsoluteScale", "Absolute Scale" ) : LOCTEXT( "Scale", "Scale" );
}

void FComponentTransformDetails::OnSetAbsoluteTransform(ETransformField::Type TransformField, bool bAbsoluteEnabled)
{
	FBoolProperty* AbsoluteProperty = nullptr;
	FText TransactionText;

	switch (TransformField)
	{
	case ETransformField::Location:
		AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteLocationPropertyName());
		TransactionText = LOCTEXT("ToggleAbsoluteLocation", "Toggle Absolute Location");
		break;
	case ETransformField::Rotation:
		AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteRotationPropertyName());
		TransactionText = LOCTEXT("ToggleAbsoluteRotation", "Toggle Absolute Rotation");
		break;
	case ETransformField::Scale:
		AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteScalePropertyName());
		TransactionText = LOCTEXT("ToggleAbsoluteScale", "Toggle Absolute Scale");
		break;
	default:
		return;
	}

	bool bBeganTransaction = false;
	TArray<UObject*> ModifiedObjects;
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				bool bOldValue = TransformField == ETransformField::Location ? SceneComponent->IsUsingAbsoluteLocation() : (TransformField == ETransformField::Rotation ? SceneComponent->IsUsingAbsoluteRotation() : SceneComponent->IsUsingAbsoluteScale());

				if (bOldValue == bAbsoluteEnabled)
				{
					// Already the desired value
					continue;
				}

				if (!bBeganTransaction)
				{
					// NOTE: One transaction per change, not per actor
					GEditor->BeginTransaction(TransactionText);
					bBeganTransaction = true;
				}

				FScopedSwitchWorldForObject WorldSwitcher(Object);

				if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
				{
					// Default subobjects must be included in any undo/redo operations
					SceneComponent->SetFlags(RF_Transactional);
				}

				SceneComponent->PreEditChange(AbsoluteProperty);

				if (NotifyHook)
				{
					NotifyHook->NotifyPreChange(AbsoluteProperty);
				}

				TOptional<FTransform> TransformToPreserve;
				if (SceneComponent->GetAttachParent())
				{
					if (bAbsoluteEnabled)
					{
						TransformToPreserve = SceneComponent->GetComponentTransform();
					}
					else
					{
						FTransform ParentToWorld = SceneComponent->GetAttachParent()->GetSocketTransform(SceneComponent->GetAttachSocketName());
						TransformToPreserve = SceneComponent->GetComponentTransform().GetRelativeTransform(ParentToWorld);
					}
				}

				switch (TransformField)
				{
				case ETransformField::Location:
					SceneComponent->SetUsingAbsoluteLocation(bAbsoluteEnabled);

					if (TransformToPreserve.IsSet())
					{
						SceneComponent->SetRelativeLocation_Direct(TransformToPreserve->GetTranslation());
					}
					break;
				case ETransformField::Rotation:
					SceneComponent->SetUsingAbsoluteRotation(bAbsoluteEnabled);

					if (TransformToPreserve.IsSet())
					{
						SceneComponent->SetRelativeRotation_Direct(FRotator(TransformToPreserve->GetRotation()));
					}
					break;
				case ETransformField::Scale:
					SceneComponent->SetUsingAbsoluteScale(bAbsoluteEnabled);

					if (TransformToPreserve.IsSet())
					{
						SceneComponent->SetRelativeScale3D_Direct(TransformToPreserve->GetScale3D());
					}
					break;
				}

				ModifiedObjects.Add(Object);
			}
		}
	}

	if (bBeganTransaction)
	{
		FPropertyChangedEvent PropertyChangedEvent(AbsoluteProperty, EPropertyChangeType::ValueSet, MakeArrayView(ModifiedObjects));

		for (UObject* Object : ModifiedObjects)
		{
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				SceneComponent->PostEditChangeProperty(PropertyChangedEvent);

				// If it's a template, propagate the change out to any current instances of the object
				if (SceneComponent->IsTemplate())
				{
					bool NewValue = bAbsoluteEnabled;
					bool OldValue = !NewValue;
					TSet<USceneComponent*> UpdatedInstances;
					FComponentEditorUtils::PropagateDefaultValueChange(SceneComponent, AbsoluteProperty, OldValue, NewValue, UpdatedInstances);
				}
			}
		}

		if (NotifyHook)
		{
			NotifyHook->NotifyPostChange(PropertyChangedEvent, AbsoluteProperty);
		}

		GEditor->EndTransaction();

		GUnrealEd->RedrawLevelEditingViewports();
	}
}

bool FComponentTransformDetails::IsAbsoluteTransformChecked(ETransformField::Type TransformField, bool bAbsoluteEnabled) const
{
	switch (TransformField)
	{
	case ETransformField::Location:
		return bAbsoluteLocation == bAbsoluteEnabled;
	case ETransformField::Rotation:
		return bAbsoluteRotation == bAbsoluteEnabled;
	case ETransformField::Scale:
		return bAbsoluteScale == bAbsoluteEnabled;
	default:
		return false;
	}
}

bool FComponentTransformDetails::CanChangeAbsoluteFlag(ETransformField::Type TransformField) const
{
	FName PropertyName;

	switch (TransformField)
	{
	case ETransformField::Location:
		PropertyName = USceneComponent::GetAbsoluteLocationPropertyName();
		break;
	case ETransformField::Rotation:
		PropertyName = USceneComponent::GetAbsoluteRotationPropertyName();
		break;
	case ETransformField::Scale:
		PropertyName = USceneComponent::GetAbsoluteScalePropertyName();
		break;
	default:
		break;
	}

	if (!PropertyName.IsNone())
	{
		if (const TSharedPtr<IPropertyHandle>* HandlePtr = PropertyHandles.Find(PropertyName))
		{
			return (*HandlePtr)->IsEditable();
		}
	}

	return false;
}

struct FGetRootComponentArchetype
{
	static USceneComponent* Get(UObject* Object)
	{
		auto RootComponent = Object ? GetSceneComponentFromDetailsObject(Object) : nullptr;
		return RootComponent ? Cast<USceneComponent>(PropertyEditorPolicy::GetArchetype(RootComponent)) : nullptr;
	}
};

TOptional<FVector::FReal> FComponentTransformDetails::GetLocationX() const
{
	return CachedLocation.X;
}

TOptional<FVector::FReal> FComponentTransformDetails::GetLocationY() const
{
	if (bIsAxisDisplayLeftUpForward && CachedLocation.Y.IsSet())
	{
		return TOptional<FVector::FReal>(-CachedLocation.Y.GetValue());
	}
	return CachedLocation.Y;
}

TOptional<FVector::FReal> FComponentTransformDetails::GetLocationZ() const
{
	return CachedLocation.Z;
}

TOptional<FRotator::FReal> FComponentTransformDetails::GetRotationX() const
{
	return CachedRotation.X;
}

TOptional<FRotator::FReal> FComponentTransformDetails::GetRotationY() const
{
	return CachedRotation.Y;
}

TOptional<FRotator::FReal> FComponentTransformDetails::GetRotationZ() const
{
	return CachedRotation.Z;
}

TOptional<FVector::FReal> FComponentTransformDetails::GetScaleX() const
{
	return CachedScale.X;
}

TOptional<FVector::FReal> FComponentTransformDetails::GetScaleY() const
{
	return CachedScale.Y;
}

TOptional<FVector::FReal> FComponentTransformDetails::GetScaleZ() const
{
	return CachedScale.Z;
}

bool FComponentTransformDetails::GetLocationResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeLocation() : FVector::ZeroVector;

	// unset means multiple differing values, so show "Reset to Default" in that case
	return CachedLocation.IsSet() && CachedLocation.X.GetValue() == Data.X && CachedLocation.Y.GetValue() == Data.Y && CachedLocation.Z.GetValue() == Data.Z ? false : true;
}

void FComponentTransformDetails::OnLocationResetClicked()
{
	if (GetIsLocationEnabled())
	{
		const FText TransactionName = LOCTEXT("ResetLocation", "Reset Location");
		FScopedTransaction Transaction(TransactionName);

		const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
		const FVector Data = Archetype ? Archetype->GetRelativeLocation() : FVector::ZeroVector;

		OnSetTransform(ETransformField::Location, EAxisList::All, Data, false, true);
	}
}

bool FComponentTransformDetails::GetRotationResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());

	// unset means multiple differing values, so show "Reset to Default" in that case
	if (!CachedRotation.IsSet())
	{
		return true;
	}
	
	if (!bIsAxisDisplayLeftUpForward)
	{
		const FVector Data = Archetype ? Archetype->GetRelativeRotation().Euler() : FVector::ZeroVector;
		// unset means multiple differing values, so show "Reset to Default" in that case
		return CachedRotation.X.GetValue() != Data.X || CachedRotation.Y.GetValue() != Data.Y || CachedRotation.Z.GetValue() != Data.Z;
	}
	else
	{
		const FVector Data = Archetype ? ConvertFromUnrealSpace_EulerDeg(Archetype->GetRelativeRotation()) : FVector::ZeroVector;
		return !ComponentTransformDetails::Private::AreRotationsEqual(Data, CachedRotation.ToVector());
	}
}

void FComponentTransformDetails::OnRotationResetClicked()
{
	if (GetIsRotationEnabled())
	{
		const FText TransactionName = LOCTEXT("ResetRotation", "Reset Rotation");
		FScopedTransaction Transaction(TransactionName);

		const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
		const FVector Data = Archetype ? ConvertFromUnrealSpace_EulerDeg(Archetype->GetRelativeRotation()) : FVector::ZeroVector;

		OnSetTransform(ETransformField::Rotation, EAxisList::All, Data, false, true);
	}
}

bool FComponentTransformDetails::GetScaleResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeScale3D() : FVector(1.0f);

	// unset means multiple differing values, so show "Reset to Default" in that case
	return CachedScale.IsSet() && CachedScale.X.GetValue() == Data.X && CachedScale.Y.GetValue() == Data.Y && CachedScale.Z.GetValue() == Data.Z ? false : true;
}

void FComponentTransformDetails::OnScaleResetClicked()
{
	if (GetIsScaleEnabled())
	{
		const FText TransactionName = LOCTEXT("ResetScale", "Reset Scale");
		FScopedTransaction Transaction(TransactionName);

		const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
		const FVector Data = Archetype ? Archetype->GetRelativeScale3D() : FVector(1.0f);

		OnSetTransform(ETransformField::Scale, EAxisList::All, Data, false, true);
	}
}

void FComponentTransformDetails::ExtendXScaleContextMenu( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.BeginSection( "ScaleOperations", LOCTEXT( "ScaleOperations", "Scale Operations" ) );
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)),
		FSlateIcon(), 		
		FUIAction( FExecuteAction::CreateSP( this, &FComponentTransformDetails::OnXScaleMirrored ), FCanExecuteAction::CreateSP( this, &FComponentTransformDetails::GetIsScaleEnabled ) )
	);
	MenuBuilder.EndSection();
}

void FComponentTransformDetails::ExtendYScaleContextMenu( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.BeginSection( "ScaleOperations", LOCTEXT( "ScaleOperations", "Scale Operations" ) );
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)),
		FSlateIcon(), 		
		FUIAction( FExecuteAction::CreateSP( this, &FComponentTransformDetails::OnYScaleMirrored ), FCanExecuteAction::CreateSP( this, &FComponentTransformDetails::GetIsScaleEnabled ) )
	);
	MenuBuilder.EndSection();
}

void FComponentTransformDetails::ExtendZScaleContextMenu( FMenuBuilder& MenuBuilder )
{
	MenuBuilder.BeginSection( "ScaleOperations", LOCTEXT( "ScaleOperations", "Scale Operations" ) );
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)),
		FSlateIcon(), 		
		FUIAction( FExecuteAction::CreateSP( this, &FComponentTransformDetails::OnZScaleMirrored ), FCanExecuteAction::CreateSP( this, &FComponentTransformDetails::GetIsScaleEnabled ) )
	);
	MenuBuilder.EndSection();
}

void FComponentTransformDetails::OnXScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)));
	OnSetTransform(ETransformField::Scale, EAxisList::X, FVector(1.0f), true, true);
}

void FComponentTransformDetails::OnYScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)));
	OnSetTransform(ETransformField::Scale, EAxisList::Y, FVector(1.0f), true, true);
}

void FComponentTransformDetails::OnZScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)));
	OnSetTransform(ETransformField::Scale, EAxisList::Z, FVector(1.0f), true, true);
}

void FComponentTransformDetails::CacheDetails()
{
	FVector CurLoc = FVector::ZeroVector;
	FRotator CurRot = FRotator::ZeroRotator;
	FVector CurScale = FVector::ZeroVector;
	bIsEnabledCache = true;
	for( int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex )
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];
		if( ObjectPtr.IsValid() )
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject( Object );

			FVector Loc;
			FRotator Rot;
			FVector Scale;
			if( SceneComponent )
			{
				if (AActor* Owner = SceneComponent->GetOwner(); Owner && Owner->GetRootComponent() == SceneComponent)
				{
					bIsEnabledCache &= !Owner->IsLockLocation();
				}
				
				Loc = SceneComponent->GetRelativeLocation();
				FRotator* FoundRotator = ObjectToRelativeRotationMap.Find(SceneComponent);
				Rot = (bEditingRotationInUI && !Object->IsTemplate() && FoundRotator) ? *FoundRotator : SceneComponent->GetRelativeRotation();
				if (bIsAxisDisplayLeftUpForward)
				{
					FVector Euler = ConvertFromUnrealSpace_EulerDeg(Rot);
					Rot = FRotator(Euler.X, Euler.Y, Euler.Z);
					
					
				}
				Scale = SceneComponent->GetRelativeScale3D();

				if( ObjectIndex == 0 )
				{
					// Cache the current values from the first actor to see if any values differ among other actors
					CurLoc = Loc;
					CurRot = Rot;
					CurScale = Scale;

					CachedLocation.Set( Loc );
					CachedRotation.Set( Rot );
					CachedScale.Set( Scale );

					bAbsoluteLocation = SceneComponent->IsUsingAbsoluteLocation();
					bAbsoluteScale = SceneComponent->IsUsingAbsoluteScale();
					bAbsoluteRotation = SceneComponent->IsUsingAbsoluteRotation();
				}
				else if( CurLoc != Loc || CurRot != Rot || CurScale != Scale )
				{
					// Check which values differ and unset the different values
					CachedLocation.X = Loc.X == CurLoc.X && CachedLocation.X.IsSet() ? Loc.X : TOptional<FVector::FReal>();
					CachedLocation.Y = Loc.Y == CurLoc.Y && CachedLocation.Y.IsSet() ? Loc.Y : TOptional<FVector::FReal>();
					CachedLocation.Z = Loc.Z == CurLoc.Z && CachedLocation.Z.IsSet() ? Loc.Z : TOptional<FVector::FReal>();

					CachedRotation.X = Rot.Roll == CurRot.Roll && CachedRotation.X.IsSet() ? Rot.Roll : TOptional<FRotator::FReal>();
					CachedRotation.Y = Rot.Pitch == CurRot.Pitch && CachedRotation.Y.IsSet() ? Rot.Pitch : TOptional<FRotator::FReal>();
					CachedRotation.Z = Rot.Yaw == CurRot.Yaw && CachedRotation.Z.IsSet() ? Rot.Yaw : TOptional<FRotator::FReal>();

					CachedScale.X = Scale.X == CurScale.X && CachedScale.X.IsSet() ? Scale.X : TOptional<FVector::FReal>();
					CachedScale.Y = Scale.Y == CurScale.Y && CachedScale.Y.IsSet() ? Scale.Y : TOptional<FVector::FReal>();
					CachedScale.Z = Scale.Z == CurScale.Z && CachedScale.Z.IsSet() ? Scale.Z : TOptional<FVector::FReal>();

					// If all values are unset all values are different and we can stop looking
					const bool bAllValuesDiffer = !CachedLocation.IsSet() && !CachedRotation.IsSet() && !CachedScale.IsSet();
					if( bAllValuesDiffer )
					{
						break;
					}
				}
			}
		}
	}
}

FVector FComponentTransformDetails::GetAxisFilteredVector(EAxisList::Type Axis, const FVector& NewValue, const FVector& OldValue)
{
	return FVector((Axis & EAxisList::X) ? NewValue.X : OldValue.X,
		(Axis & EAxisList::Y) ? NewValue.Y : OldValue.Y,
		(Axis & EAxisList::Z) ? NewValue.Z : OldValue.Z);
}

void FComponentTransformDetails::OnSetTransform(ETransformField::Type TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted)
{
	if (!bCommitted && SelectedObjects.Num() > 1)
	{
		// Ignore interactive changes when we have more than one selected object
		return;
	}

	FText TransactionText;
	FProperty* ValueProperty = nullptr;
	FProperty* AxisProperty = nullptr;
	
	switch (TransformField)
	{
	case ETransformField::Location:
		TransactionText = LOCTEXT("OnSetLocation", "Set Location");
		ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
		
		// Only set axis property for single axis set
		if (Axis == EAxisList::X)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
			check(AxisProperty != nullptr);
		}
		else if (Axis == EAxisList::Y)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
			check(AxisProperty != nullptr);
		}
		else if (Axis == EAxisList::Z)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
			check(AxisProperty != nullptr);
		}
		break;
	case ETransformField::Rotation:
		TransactionText = LOCTEXT("OnSetRotation", "Set Rotation");
		ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());
		
		// Only set axis property for single axis set
		if (Axis == EAxisList::X)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Roll));
			check(AxisProperty != nullptr);
		}
		else if (Axis == EAxisList::Y)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Pitch));
			check(AxisProperty != nullptr);
		}
		else if (Axis == EAxisList::Z)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Yaw));
			check(AxisProperty != nullptr);
		}
		break;
	case ETransformField::Scale:
		TransactionText = LOCTEXT("OnSetScale", "Set Scale");
		ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());

		// If keep scale is set, don't set axis property
		if (!bPreserveScaleRatio && Axis == EAxisList::X)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
			check(AxisProperty != nullptr);
		}
		else if (!bPreserveScaleRatio && Axis == EAxisList::Y)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
			check(AxisProperty != nullptr);
		}
		else if (!bPreserveScaleRatio && Axis == EAxisList::Z)
		{
			AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
			check(AxisProperty != nullptr);
		}
		break;
	default:
		return;
	}

	bool bBeganTransaction = false;
	TArray<UObject*> ModifiedObjects;

	FPropertyChangedEvent PropertyChangedEvent(ValueProperty, !bCommitted ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet, MakeArrayView(ModifiedObjects));
	FEditPropertyChain PropertyChain;

	if (AxisProperty)
	{
		PropertyChain.AddHead(AxisProperty);
	}
	PropertyChain.AddHead(ValueProperty);
	FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

	EAxisList::Type RemappedAxis = Axis;
	FVector SwizzledNewValue = NewValue;

	if (bIsAxisDisplayLeftUpForward)
	{
		if (TransformField == ETransformField::Location)
    	{
    		SwizzledNewValue.Y = -NewValue.Y;
    	}

		if (TransformField == ETransformField::Rotation)
		{
			// Need to convert from Right-handed Y-Up to UE's Left-handed Z-Up however...
			// NewValue is not the full set of Euler values to be applied, it will only contain
			// the single value that was changed as specified by Axis
			// Therefore it is not yet safe to convert it over.

			// However we do need to swizzle the NewValue since the rotation widgets Axis values are set assuming
			// normal Unreal rotations
			RemappedAxis = [&]()
			{
				switch (Axis)
				{
				case EAxisList::X: return EAxisList::Z;
				case EAxisList::Y: return EAxisList::X;
				case EAxisList::Z: return EAxisList::Y;
				case EAxisList::All: return EAxisList::All;
				default: return EAxisList::X;
				}
			}();

			SwizzledNewValue = FVector(NewValue.Y, NewValue.Z, NewValue.X);
			
			// Next step is to run SwizzledNewValue through GetAxisFilteredValue() to compose it
			// with the converted rotator to right-hand coordinate space and get the full set of euler angles.
			// Finally, these euler angles will be converted back to Unreal Rotator space and applied.
			
			// ObjectToRelativeRotationMap stores the rotations in Unreal Rotator space always - this may need to change though
			// CachedRotation is stored in Right handed Y-up space as this is used to read back the values into the widget for display purposes
			// See GetRotationY, GetRotationZ
		}
	}
	

	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				AActor* EditedActor = SceneComponent->GetOwner();
				const bool bIsEditingTemplateObject = Object->IsTemplate();

				FRotator OldComponentRotator;
				FVector OldComponentValue;
				FVector NewComponentValue;

				switch (TransformField)
				{
				case ETransformField::Location:
					OldComponentValue = SceneComponent->GetRelativeLocation();
					break;
				case ETransformField::Rotation:
					// Pull from the actual component or from the cache
					if (bEditingRotationInUI && !bIsEditingTemplateObject && ObjectToRelativeRotationMap.Find(SceneComponent))
					{
						OldComponentRotator = *ObjectToRelativeRotationMap.Find(SceneComponent);
					}
					else
					{
						OldComponentRotator = SceneComponent->GetRelativeRotation();
					}
					OldComponentValue = ConvertFromUnrealSpace_EulerDeg(OldComponentRotator);
					break;
				case ETransformField::Scale:
					OldComponentValue = SceneComponent->GetRelativeScale3D();
					break;
				}

				// Set the incoming value
				if (bMirror)
				{
					NewComponentValue = GetAxisFilteredVector(RemappedAxis, -OldComponentValue, OldComponentValue);
				}
				else
				{
					NewComponentValue = GetAxisFilteredVector(RemappedAxis, SwizzledNewValue, OldComponentValue);
				}

				// Clamp location to world size
				if (TransformField == ETransformField::Location)
				{
					double MaxMagnitude = HALF_WORLD_MAX;
					if (UWorld* World = SceneComponent->GetWorld())
					{
						AWorldSettings* WorldSettings = World->GetWorldSettings(false);
						bool bEnableWorldBoundsChecks = WorldSettings ? WorldSettings->bEnableWorldBoundsChecks : true;
						if (!bEnableWorldBoundsChecks)
						{
							MaxMagnitude = UE_DOUBLE_HUGE_DISTANCE;
						}
					}

					NewComponentValue = FVector
					{
						FMath::Clamp(NewComponentValue.X, -MaxMagnitude, MaxMagnitude),
						FMath::Clamp(NewComponentValue.Y, -MaxMagnitude, MaxMagnitude),
						FMath::Clamp(NewComponentValue.Z, -MaxMagnitude, MaxMagnitude)
					};
				}
				
				auto AreValuesEqual = [this, TransformField](const FVector& NewComponentValue_, const FVector& OldComponentValue_)
				{
					if (!bIsAxisDisplayLeftUpForward || TransformField != ETransformField::Rotation)
					{
						// Bit-wise identical check
						return NewComponentValue_ == OldComponentValue_;
					}
					else
					{
						// LeftUpForward uses alternative XYZ intrinsic rotation but rotation is stored
						// still as FRotator in Yaw-Pitch-Roll intrinsic convention.  The conversion between
						// these two conventions prevents bit-exact comparisons.  If values set are close enough
						// to what exists on the component, then skip the setting rotation
						// This prevents accidental small errors accumulating due to automatic conversion from the cached
						// euler rotation representations and the underlying data
						return ComponentTransformDetails::Private::AreRotationsEqual(NewComponentValue_, OldComponentValue_);
					}
				};

				// If we're committing during a slider transaction then we need to force it, in order that PostEditChangeChainProperty be called.
				// Note: this will even happen if the slider hasn't changed the value.
				if (!AreValuesEqual(NewComponentValue, OldComponentValue) || (bCommitted && bIsSliderTransaction))
				{

					if (!bBeganTransaction && bCommitted)
					{
						// NOTE: One transaction per change, not per actor
						GEditor->BeginTransaction(TransactionText);
						bBeganTransaction = true;
					}

					FScopedSwitchWorldForObject WorldSwitcher(Object);

					if (bCommitted)
					{
						if (!bIsEditingTemplateObject)
						{
							// Broadcast the first time an actor is about to move
							GEditor->BroadcastBeginObjectMovement(*SceneComponent);
							if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
							{
								GEditor->BroadcastBeginObjectMovement(*EditedActor);
							}
						}

						if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
						{
							// Default subobjects must be included in any undo/redo operations
							SceneComponent->SetFlags(RF_Transactional);
						}
					}

					// Have to downcast here because of function overloading and inheritance not playing nicely
					((UObject*)SceneComponent)->PreEditChange(PropertyChain);
					if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
					{
						((UObject*)EditedActor)->PreEditChange(PropertyChain);
					}

					if (NotifyHook)
					{
						NotifyHook->NotifyPreChange(ValueProperty);
					}

					switch (TransformField)
					{
					case ETransformField::Location:
						{
							if (!bIsEditingTemplateObject)
							{
								// Update local cache for restoring later
								ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = SceneComponent->GetRelativeRotation();
							}

							SceneComponent->SetRelativeLocation(NewComponentValue);

							// Also forcibly set it as the cache may have changed it slightly
							SceneComponent->SetRelativeLocation_Direct(NewComponentValue);

							// If it's a template, propagate the change out to any current instances of the object
							if (bIsEditingTemplateObject)
							{
								TSet<USceneComponent*> UpdatedInstances;
								FComponentEditorUtils::PropagateDefaultValueChange(SceneComponent, ValueProperty, OldComponentValue, NewComponentValue, UpdatedInstances);
							}

							break;
						}
					case ETransformField::Rotation:
						{
							FRotator NewRotation = ConvertToUnrealSpace_EulerDeg(NewComponentValue);

							if (!bIsEditingTemplateObject)
							{
								// Update local cache for restoring later
								ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = NewRotation;
							}

							SceneComponent->SetRelativeRotationExact(NewRotation);

							// If it's a template, propagate the change out to any current instances of the object
							if (bIsEditingTemplateObject)
							{
								TSet<USceneComponent*> UpdatedInstances;
								FComponentEditorUtils::PropagateDefaultValueChange(SceneComponent, ValueProperty, ConvertToUnrealSpace_EulerDeg(OldComponentValue), NewRotation, UpdatedInstances);
							}

							break;
						}
					case ETransformField::Scale:
						{
							if (bPreserveScaleRatio)
							{
								// If we set a single axis, scale the others
								FVector::FReal Ratio = 0.0f;

								switch (Axis)
								{
								case EAxisList::X:
									if (bIsSliderTransaction)
									{
										Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.X);
										NewComponentValue.Y = NewComponentValue.X * Ratio;

										Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.X);
										NewComponentValue.Z = NewComponentValue.X * Ratio;
									}
									else
									{
										Ratio = OldComponentValue.X == 0.0f ? NewComponentValue.Z : NewComponentValue.X / OldComponentValue.X;
										NewComponentValue.Y *= Ratio;
										NewComponentValue.Z *= Ratio;
									}
									break;
								case EAxisList::Y:
									if (bIsSliderTransaction)
									{
										Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Y);
										NewComponentValue.X = NewComponentValue.Y * Ratio;

										Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.Y);
										NewComponentValue.Z = NewComponentValue.Y * Ratio;
									}
									else
									{
										Ratio = OldComponentValue.Y == 0.0f ? NewComponentValue.Z : NewComponentValue.Y / OldComponentValue.Y;
										NewComponentValue.X *= Ratio;
										NewComponentValue.Z *= Ratio;
									}
									break;
								case EAxisList::Z:
									if (bIsSliderTransaction)
									{
										Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Z);
										NewComponentValue.X = NewComponentValue.Z * Ratio;

										Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.Z);
										NewComponentValue.Y = NewComponentValue.Z * Ratio;
									}
									else
									{
										Ratio = OldComponentValue.Z == 0.0f ? NewComponentValue.Z : NewComponentValue.Z / OldComponentValue.Z;
										NewComponentValue.X *= Ratio;
										NewComponentValue.Y *= Ratio;
									}
									break;
								default:
									// Do nothing, this set multiple axis at once
									break;
								}
							}

							SceneComponent->SetRelativeScale3D(NewComponentValue);

							// If it's a template, propagate the change out to any current instances of the object
							if (bIsEditingTemplateObject)
							{
								TSet<USceneComponent*> UpdatedInstances;
								FComponentEditorUtils::PropagateDefaultValueChange(SceneComponent, ValueProperty, OldComponentValue, NewComponentValue, UpdatedInstances);
							}

							break;
						}
					}

					ModifiedObjects.Add(Object);
				}
			}
		}
	}

	if (ModifiedObjects.Num())
	{
		for (UObject* Object : ModifiedObjects)
		{
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			USceneComponent* OldSceneComponent = SceneComponent;

			if (SceneComponent)
			{
				AActor* EditedActor = SceneComponent->GetOwner();
				FString SceneComponentPath = SceneComponent->GetPathName(EditedActor);
				
				// This can invalidate OldSceneComponent
				OldSceneComponent->PostEditChangeChainProperty(PropertyChangedChainEvent);

				if (!bCommitted)
				{
					const FProperty* ConstValueProperty = ValueProperty;
					SnapshotTransactionBuffer(OldSceneComponent, MakeArrayView(&ConstValueProperty, 1));
				}

				SceneComponent = FindObject<USceneComponent>(EditedActor, *SceneComponentPath);

				if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
				{
					EditedActor->PostEditChangeChainProperty(PropertyChangedChainEvent);
					SceneComponent = FindObject<USceneComponent>(EditedActor, *SceneComponentPath);

					if (!bCommitted && OldSceneComponent != SceneComponent)
					{
						const FProperty* ConstValueProperty = ValueProperty;
						SnapshotTransactionBuffer(SceneComponent, MakeArrayView(&ConstValueProperty, 1));
					}
				}
				
				if (SceneComponent && !Object->IsTemplate())
				{
					if (TransformField == ETransformField::Rotation || TransformField == ETransformField::Location)
					{
						FRotator* FoundRotator = ObjectToRelativeRotationMap.Find(OldSceneComponent);

						if (FoundRotator)
						{
							FQuat OldQuat = FoundRotator->GetDenormalized().Quaternion();
							FQuat NewQuat = SceneComponent->GetRelativeRotation().GetDenormalized().Quaternion();

							if (OldQuat.Equals(NewQuat))
							{
								// Need to restore the manually set rotation as it was modified by quat conversion
								SceneComponent->SetRelativeRotation_Direct(*FoundRotator);
							}
						}
					}

					if (bCommitted)
					{
						// Broadcast when the actor is done moving
						GEditor->BroadcastEndObjectMovement(*SceneComponent);
						if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
						{
							GEditor->BroadcastEndObjectMovement(*EditedActor);
						}
					}
				}
			}
		}

		if (NotifyHook)
		{
			NotifyHook->NotifyPostChange(PropertyChangedEvent, ValueProperty);
		}
	}

	if (bCommitted && bBeganTransaction)
	{
		GEditor->EndTransaction();
		CacheDetails();
	}

	GUnrealEd->UpdatePivotLocationForSelection();
	GUnrealEd->SetPivotMovedIndependently(false);
	// Redraw
	GUnrealEd->RedrawLevelEditingViewports();
}

void FComponentTransformDetails::OnSetTransformAxis(FVector::FReal NewValue, ETextCommit::Type CommitInfo, ETransformField::Type TransformField, EAxisList::Type Axis, bool bCommitted)
{
	FVector NewVector = GetAxisFilteredVector(Axis, FVector(NewValue), FVector::ZeroVector);
	OnSetTransform(TransformField, Axis, NewVector, false, bCommitted);
}

void FComponentTransformDetails::BeginSliderTransaction(FText ActorTransaction, FText ComponentTransaction) const
{
	bool bBeganTransaction = false;
	for (TWeakObjectPtr<UObject> ObjectPtr : SelectedObjects)
	{
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			// Start a new transaction when a slider begins to change
			// We'll end it when the slider is released
			// NOTE: One transaction per change, not per actor
			if (!bBeganTransaction)
			{
				if (Object->IsA<AActor>())
				{
					GEditor->BeginTransaction(ActorTransaction);
				}
				else
				{
					GEditor->BeginTransaction(ComponentTransaction);
				}

				bBeganTransaction = true;
			}

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				FScopedSwitchWorldForObject WorldSwitcher(Object);

				if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
				{
					// Default subobjects must be included in any undo/redo operations
					SceneComponent->SetFlags(RF_Transactional);
				}

				// Call modify but not PreEdit, we don't do the proper "Edit" until it's committed
				SceneComponent->Modify();
			}
		}
	}

	// Just in case we couldn't start a new transaction for some reason
	if (!bBeganTransaction)
	{
		GEditor->BeginTransaction(ActorTransaction);
	}
}

void FComponentTransformDetails::OnBeginRotationSlider()
{
	FText ActorTransaction = LOCTEXT("OnSetRotation", "Set Rotation");
	FText ComponentTransaction = LOCTEXT("OnSetRotation_ComponentDirect", "Modify Component(s)");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);

	bEditingRotationInUI = true;
	bIsSliderTransaction = true;

	for (TWeakObjectPtr<UObject> ObjectPtr : SelectedObjects)
	{
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				FScopedSwitchWorldForObject WorldSwitcher(Object);

				// Add/update cached rotation value prior to slider interaction
				ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = SceneComponent->GetRelativeRotation();
			}
		}
	}
}

void FComponentTransformDetails::OnEndRotationSlider(FRotator::FReal NewValue)
{
	// Commit gets called right before this, only need to end the transaction
	bEditingRotationInUI = false;
	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FComponentTransformDetails::OnBeginLocationSlider()
{
	bIsSliderTransaction = true;
	FText ActorTransaction = LOCTEXT("OnSetLocation", "Set Location");
	FText ComponentTransaction = LOCTEXT("OnSetLocation_ComponentDirect", "Modify Component Location");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);
}

void FComponentTransformDetails::OnEndLocationSlider(FVector::FReal NewValue)
{
	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FComponentTransformDetails::OnBeginScaleSlider()
{
	// Assumption: slider isn't usable if multiple objects are selected
	SliderScaleRatio.X = CachedScale.X.GetValue();
	SliderScaleRatio.Y = CachedScale.Y.GetValue();
	SliderScaleRatio.Z = CachedScale.Z.GetValue();

	bIsSliderTransaction = true;
	FText ActorTransaction = LOCTEXT("OnSetScale", "Set Scale");
	FText ComponentTransaction = LOCTEXT("OnSetScale_ComponentDirect", "Modify Component Scale");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);
}

void FComponentTransformDetails::OnEndScaleSlider(FVector::FReal NewValue)
{
	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FComponentTransformDetails::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	TArray<UObject*> NewSceneComponents;
	for (const TWeakObjectPtr<UObject> Obj : CachedHandlesObjects)
	{
		if (UObject* Replacement = ReplacementMap.FindRef(Obj.GetEvenIfUnreachable()))
		{
			NewSceneComponents.Add(Replacement);
		}
	}

	if (NewSceneComponents.Num())
	{
		UpdatePropertyHandlesObjects(NewSceneComponents);
	}
}

namespace ComponentTransformDetailsPrivate
{
	TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> RadiansToDegrees(const TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal>& Rads)
	{
		constexpr FQuat::FReal RadsToDegrees = (180.f / UE_PI);
		TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> Output
		{
			Rads.Get<0>() * RadsToDegrees,
			Rads.Get<1>() * RadsToDegrees,
			Rads.Get<2>() * RadsToDegrees
		};
		return Output;
	}

	TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> DegreesToRadians(const TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal>& Rads)
	{
		constexpr FQuat::FReal DegreesToRadians = (UE_PI / 180.f);
		TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> Output
		{
			Rads.Get<0>() * DegreesToRadians,
			Rads.Get<1>() * DegreesToRadians,
			Rads.Get<2>() * DegreesToRadians
		};
		return Output;
	}
}

FVector FComponentTransformDetails::ConvertFromUnrealSpace_EulerDeg(const FRotator& Rotator) const
{
	using namespace ComponentTransformDetailsPrivate;
	if (!bIsAxisDisplayLeftUpForward)
	{
		return Rotator.Euler();
	}

	FQuat Q = Rotator.Quaternion().GetNormalized();
	TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> VerseEulerRads = Q.ToLUFEuler();

	// Since the value is converted from quaternion, will likely have denormals.  Clamp those values.
	auto SanitizeFloat = [](FQuat::FReal Val)->FQuat::FReal
	{
		if (FMath::IsNearlyZero(Val))
		{
			return 0.0;
		}
		return Val;
	};
	
	FVector VerseEulerRadsV
	{
		SanitizeFloat(VerseEulerRads.Get<0>()),
		SanitizeFloat(VerseEulerRads.Get<1>()),
		SanitizeFloat(VerseEulerRads.Get<2>())
	};
	FVector VerseEulerDegrees = FMath::RadiansToDegrees(VerseEulerRadsV);
	return VerseEulerDegrees;
}

FRotator FComponentTransformDetails::ConvertToUnrealSpace_EulerDeg(const FVector& Rotation) const
{
	using namespace ComponentTransformDetailsPrivate;
	if (!bIsAxisDisplayLeftUpForward)
	{
		return FRotator::MakeFromEuler(Rotation);
	}

	const FVector RotationRads = FMath::DegreesToRadians(Rotation);

	TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> RotationRadsT =
	{
		RotationRads.X,
		RotationRads.Y,
		RotationRads.Z
	};
	
	
	FQuat Quat = FQuat::MakeFromLUFEuler(RotationRadsT);
	Quat.Normalize();

	FRotator Result(Quat);
	return Result;
}

#undef LOCTEXT_NAMESPACE
