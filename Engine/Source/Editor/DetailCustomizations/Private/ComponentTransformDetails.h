// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetSelection.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomNodeBuilder.h"
#include "Internationalization/Text.h"
#include "Math/Axis.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/NumericTypeInterface.h"

class FDetailWidgetRow;
class FMenuBuilder;
class FNotifyHook;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UObject;
struct FSlateBrush;

namespace ETransformField
{
	enum Type
	{
		Location,
		Rotation,
		Scale
	};
}
/**
 * Manages the Transform section of a details view                    
 */
class FComponentTransformDetails : public TSharedFromThis<FComponentTransformDetails>, public IDetailCustomNodeBuilder, public TNumericUnitTypeInterface<FVector::FReal>
{
public:
	FComponentTransformDetails( const TArray< TWeakObjectPtr<UObject> >& InSelectedObjects, const FSelectedActorInfo& InSelectedActorInfo, IDetailLayoutBuilder& DetailBuilder );
	~FComponentTransformDetails();

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override {}
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual bool RequiresTick() const override { return true; }
	virtual FName GetName() const override { return "Transform"; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void Tick( float DeltaTime ) override;
	virtual void SetOnRebuildChildren( FSimpleDelegate OnRebuildChildren ) override{}

	void HideTransformField(const ETransformField::Type InTransformField)
	{
		HiddenFieldMask |= (1 << InTransformField);
	}

private:
	/** Caches some information of the actor (transform, locked location) for the user input boxes */
	void CacheDetails();

	/** @return Whether the transform details panel should be enabled (editable) or not (read-only / greyed out) */
	bool GetIsEnabled() const;
	bool GetIsLocationEnabled() const;
	bool GetIsRotationEnabled() const;
	bool GetIsScaleEnabled() const;
	bool GetIsTransformComponentEnabled(FName ComponentName) const;

	/** Sets a vector based on two source vectors and an axis list */
	FVector GetAxisFilteredVector(EAxisList::Type Axis, const FVector& NewValue, const FVector& OldValue);

	/**
	 * Sets the selected object(s) axis to passed in value
	 *
	 * @param TransformField	The field (location/rotation/scale) to modify
	 * @param Axis				Bitfield of which axis to set, can be multiple
	 * @param NewValue			The new vector values, it only uses the ones with specified axis
	 * @param bMirror			If true, set the value to it's inverse instead of using NewValue
	 * @param bCommittted		True if the value was committed, false is the value comes from the slider
	 */
	void OnSetTransform(ETransformField::Type TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted);

	/**
	 * Sets a single axis value, called from UI
	 *
	 * @param TransformField	The field (location/rotation/scale) to modify
	 * @param NewValue			The new translation value
	 * @param CommitInfo		Whether or not this was committed from pressing enter or losing focus
	 * @param Axis				Bitfield of which axis to set, can be multiple
	 * @param bCommittted		true if the value was committed, false is the value comes from the slider
	 */
	void OnSetTransformAxis(FVector::FReal NewValue, ETextCommit::Type CommitInfo, ETransformField::Type TransformField, EAxisList::Type Axis, bool bCommitted);

	/** 
	 * Helper to begin a new transaction for a slider interaction. 
	 * @param ActorTransaction		The name to give the transaction when changing an actor transform.
	 * @param ComponentTransaction	The name to give the transaction when directly editing a component transform.
	 */
	void BeginSliderTransaction(FText ActorTransaction, FText ComponentTransaction) const;

	/** Called when the one of the axis sliders for object rotation begins to change for the first time */
	void OnBeginRotationSlider();

	/** Called when the one of the axis sliders for object rotation is released */
	void OnEndRotationSlider(FRotator::FReal NewValue);

	/** Called when one of the axis sliders for object location begins to change */
	void OnBeginLocationSlider();

	/** Called when one of the axis sliders for object location is released */
	void OnEndLocationSlider(FVector::FReal NewValue);

	/** Called when one of the axis sliders for object scale begins to change */
	void OnBeginScaleSlider();

	/** Called when one of the axis sliders for object scale is released */
	void OnEndScaleSlider(FVector::FReal NewValue);

	/** @return Icon to use in the preserve scale ratio check box */
	const FSlateBrush* GetPreserveScaleRatioImage() const;

	/** @return The state of the preserve scale ratio checkbox */
	ECheckBoxState IsPreserveScaleRatioChecked() const;

	/**
	 * Called when the preserve scale ratio checkbox is toggled
	 */
	void OnPreserveScaleRatioToggled( ECheckBoxState NewState );

	/**
	 * Builds a transform field label
	 *
	 * @param TransformField The field to build the label for
	 * @return The label widget
	 */
	TSharedRef<SWidget> BuildTransformFieldLabel( ETransformField::Type TransformField );

	/**
	 * Gets display text for a transform field
	 *
	 * @param TransformField	The field to get the text for
	 * @return The text label for TransformField
	 */
	FText GetTransformFieldText( ETransformField::Type TransformField ) const;

	/**
	 * @return the text to display for translation                   
	 */
	FText GetLocationText() const;

	/**
	 * @return the text to display for rotation                   
	 */
	FText GetRotationText() const;

	/**
	 * @return the text to display for scale                   
	 */
	FText GetScaleText() const;

	/**
	 * Sets relative transform on the specified field
	 *
	 * @param The field that should be set to relative
	 */
	void OnSetAbsoluteTransform( ETransformField::Type TransformField, bool bAbsoluteEnabled);

	/** @return true if Absolute flag of transform type matches passed in bCheckAbsolute*/
	bool IsAbsoluteTransformChecked( ETransformField::Type TransformField, bool bAbsoluteEnabled=true) const;

	/** @return true if Absolute flag of transform can be changed */
	bool CanChangeAbsoluteFlag(ETransformField::Type TransformField) const;

	/** @return true of copy is enabled for the specified field */
	bool OnCanCopy( ETransformField::Type TransformField ) const;

	/**
	 * Copies the specified transform field to the clipboard
	 */
	void OnCopy( ETransformField::Type TransformField );
	
	void OnCopyToText(TSharedPtr<IPropertyHandle> PropertyHandle, FString& Text, const TOptional<FGuid>& InOperationId, ETransformField::Type InTransformField);

	void CopyToText(FString& Text, ETransformField::Type InTransformField);


	/**
	 * Pastes the specified transform field from the clipboard
	 */
	void OnPaste( ETransformField::Type TransformField );

	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, ETransformField::Type InTransformField);

	void PasteFromText(const FString& InTag, const FString& InText, ETransformField::Type InTransformField);
	
	/**
	 * Creates a UI action for copying a specified transform field
	 */
	FUIAction CreateCopyAction( ETransformField::Type TransformField ) const;

	/**
	 * Creates a UI action for pasting a specified transform field
	 */
	FUIAction CreatePasteAction( ETransformField::Type TransformField ) const;

	/** Called when the "Reset to Default" button for the location has been clicked */
	void OnLocationResetClicked();

	/** Called when the "Reset to Default" button for the rotation has been clicked */
	void OnRotationResetClicked();

	/** Called when the "Reset to Default" button for the scale has been clicked */
	void OnScaleResetClicked();

	/** Extend the context menu for the X component */
	void ExtendXScaleContextMenu( FMenuBuilder& MenuBuilder );
	/** Extend the context menu for the Y component */
	void ExtendYScaleContextMenu( FMenuBuilder& MenuBuilder );
	/** Extend the context menu for the Z component */
	void ExtendZScaleContextMenu( FMenuBuilder& MenuBuilder );

	/** Called when the X axis scale is mirrored */
	void OnXScaleMirrored();
	/** Called when the Y axis scale is mirrored */
	void OnYScaleMirrored();
	/** Called when the Z axis scale is mirrored */
	void OnZScaleMirrored();

	/** @return The X component of location */
	TOptional<FVector::FReal> GetLocationX() const;
	/** @return The Y component of location */
	TOptional<FVector::FReal> GetLocationY() const;
	/** @return The Z component of location */
	TOptional<FVector::FReal> GetLocationZ() const;
	/** @return The visibility of the "Reset to Default" button for the location component */
	bool GetLocationResetVisibility() const;

	/** @return The X component of rotation */
	TOptional<FRotator::FReal> GetRotationX() const;
	/** @return The Y component of rotation */
	TOptional<FRotator::FReal> GetRotationY() const;
	/** @return The Z component of rotation */
	TOptional<FRotator::FReal> GetRotationZ() const;
	/** @return The visibility of the "Reset to Default" button for the rotation component */
	bool GetRotationResetVisibility() const;

	/** @return The X component of scale */
	TOptional<FVector::FReal> GetScaleX() const;
	/** @return The Y component of scale */
	TOptional<FVector::FReal> GetScaleY() const;
	/** @return The Z component of scale */
	TOptional<FVector::FReal> GetScaleZ() const;
	/** @return The visibility of the "Reset to Default" button for the scale component */
	bool GetScaleResetVisibility() const;

	/** Cache a single unit to display all location components in */
	void CacheCommonLocationUnits();

	/** Get a property handle from a property name. */
	TSharedPtr<IPropertyHandle> FindOrCreatePropertyHandle(FName PropertyName, IDetailChildrenBuilder& ChildrenBuilder);
	/** Update the outer objects of the property handles generated from this transform. */
	void UpdatePropertyHandlesObjects(const TArray<UObject*> NewSceneComponents);

	/** Delegate called when the OnObjectsReplaced is broadcast by the engine. */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	// Gets Euler angles from Unreal's Rotator space and converts to the display/edit space
	// If display space is the Rotator space, then just Euler()
	// Otherwise performs handedness and rotation order calculations
	FVector ConvertFromUnrealSpace_EulerDeg(const FRotator& Rotator) const;
	// Converts back to Unreal's Rotator if display space is different than Rotator's space
	// Assumes that the Rotation provided is in degrees
	FRotator ConvertToUnrealSpace_EulerDeg(const FVector& Rotation) const;
	
private:
	/** A vector where it may optionally be unset */
	template <typename NumericType>
	struct FOptionalVector
	{
		/**
		 * Sets the value from an FVector
		 */
		void Set( const FVector& InVec )
		{
			X = InVec.X;
			Y = InVec.Y;
			Z = InVec.Z;
		}

		/**
		 * Sets the value from an FRotator
		 */
		void Set(const FRotator& InRot)
		{
			X = InRot.Roll;
			Y = InRot.Pitch;
			Z = InRot.Yaw;
		}
		
		/** @return Whether or not the value is set */
		bool IsSet() const
		{
			// The vector is set if all values are set
			return X.IsSet() && Y.IsSet() && Z.IsSet();
		}

		FVector ToVector() const
		{
			check(IsSet());
			return FVector(X.GetValue(), Y.GetValue(), Z.GetValue());
		}

		TOptional<NumericType> X;
		TOptional<NumericType> Y;
		TOptional<NumericType> Z;
	};
	
	FSelectedActorInfo SelectedActorInfo;
	/** Copy of selected actor references in the details view */
	TArray< TWeakObjectPtr<UObject> > SelectedObjects;
	/** Cache translation value of the selected set */
	FOptionalVector<FVector::FReal> CachedLocation;
	/** Cache rotation value of the selected set */
	FOptionalVector<FRotator::FReal> CachedRotation;
	/** Cache scale value of the selected set */
	FOptionalVector<FVector::FReal> CachedScale;
	/** Notify hook to use */
	FNotifyHook* NotifyHook;
	/** Mapping from object to relative rotation values which are not affected by Quat->Rotator conversions during transform calculations */
	TMap< UObject*, FRotator > ObjectToRelativeRotationMap;
	/** Whether or not we are in absolute translation mode */
	bool bAbsoluteLocation;
	/** Whether or not we are in absolute rotation mode */
	bool bAbsoluteRotation;
	/** Whether or not we are in absolute scale mode */
	bool bAbsoluteScale;
	/** Whether or not to preserve scale ratios */
	bool bPreserveScaleRatio;
	/** Scale ratio to use when we are using the sliders with bPreserveScaleRatio set. */
	FVector SliderScaleRatio;
	/** Flag to indicate we are currently editing the rotation in the UI, so we should rely on the cached value in objectToRelativeRotationMap, not the value from the object */
	bool bEditingRotationInUI;
	/** Flag to indicate we are currently performing a slider transaction */
	bool bIsSliderTransaction;
	/** Bitmask to indicate which fields should be hidden (if any) */
	uint8 HiddenFieldMask;
	/** Holds this transform's property handles. */
	TMap<FName, TSharedPtr< IPropertyHandle> > PropertyHandles;
	/** Holds the property handles' outer objects. Used to update the handles' objects when the actor construction script runs. */
	TArray< TWeakObjectPtr<UObject> > CachedHandlesObjects;
	/** Cached enabled value of the selected set */
	bool bIsEnabledCache;
	/** Whether or not the axis display coordinate system is LeftUpForward  */
	bool bIsAxisDisplayLeftUpForward;
};
