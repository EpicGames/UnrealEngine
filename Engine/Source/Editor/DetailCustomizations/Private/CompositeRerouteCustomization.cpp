// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeRerouteCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "Materials/MaterialExpressionPinBase.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "CompositeRerouteDetails"

TSharedRef<IPropertyTypeCustomization> FCompositeRerouteCustomization::MakeInstance()
{
	return MakeShareable(new FCompositeRerouteCustomization());
}

void FCompositeRerouteCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	/* do nothing */
}

void FCompositeRerouteCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCompositeReroute, Name))->CreatePropertyValueWidget()
		];
}

#undef LOCTEXT_NAMESPACE
