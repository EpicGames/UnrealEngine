// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FCompositeRerouteCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this property customization */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};

