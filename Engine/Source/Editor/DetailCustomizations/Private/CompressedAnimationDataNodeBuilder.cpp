// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompressedAnimationDataNodeBuilder.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "PlatformInfo.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Styling/AppStyle.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CompressedAnimationDataNodeBuilder"

FCompressedAnimationDataNodeBuilder::FCompressedAnimationDataNodeBuilder(UAnimSequence* InAnimSequence): WeakAnimSequence(InAnimSequence)
{
	CurrentTargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	SelectedPlatformName = CurrentTargetPlatform->CookingDeviceProfileName();
		
	for (const FName& PlatformName : PlatformInfo::GetAllVanillaPlatformNames())
	{
		PlatformsList.Add(MakeShared<FString>(PlatformName.ToString()));
	}

	for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfo::GetPlatformInfoArray())
	{
		if(PlatformInfo->PlatformType == EBuildTargetType::Game || PlatformInfo->PlatformType == EBuildTargetType::Program)
		{
			continue;
		}

		PlatformsList.Add(MakeShared<FString>(PlatformInfo->Name.ToString()));
	}

	PlatformsList.Sort([](TSharedPtr<FString> A, TSharedPtr<FString> B) { return *A < *B; });
	ensure(PlatformsList.ContainsByPredicate([this](TSharedPtr<FString> SharedPlatform) { return *SharedPlatform == CurrentTargetPlatform->IniPlatformName(); }));
}

void FCompressedAnimationDataNodeBuilder::Tick(float DeltaTime)
{
	if (const UAnimSequence* Sequence = WeakAnimSequence.Get())
	{
		const bool bHasCompressionData = Sequence->HasCompressedDataForPlatform(CurrentTargetPlatform);
		if (bCachedHasCompressionData != bHasCompressionData )
		{
			bCachedHasCompressionData = bHasCompressionData;
			OnRegenerateChildren.Execute();
		}
	}
}

const FSlateBrush* FCompressedAnimationDataNodeBuilder::GetSelectedPlatformBrush() const
{
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(SelectedPlatformName));
	if (PlatformInfo != nullptr)
	{
		return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal));
	}

	return FStyleDefaults::GetNoBrush();
}

TSharedRef<SWidget> FCompressedAnimationDataNodeBuilder::OnGeneratePlatformListWidget(TSharedPtr<FString> Platform)
{
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(*Platform));
	if (PlatformInfo != nullptr)
	{
		const float Indent = PlatformInfo->IsVanilla() ? 0.f : 16.f;

		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			 .AutoWidth()
			 .VAlign(VAlign_Center)
			 .Padding(4+Indent)
			[
				SNew(STextBlock)
				.Text(PlatformInfo->DisplayName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
	
	return SNullWidget::NullWidget;
}

void FCompressedAnimationDataNodeBuilder::OnPlatformSelectionChanged(TSharedPtr<FString> Platform, ESelectInfo::Type InSelectInfo)
{
	if (Platform.IsValid())
	{				
		SelectedPlatformName = *Platform;
	}

	const FName PlatformName(*Platform);
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);
	CurrentTargetPlatform = GetTargetPlatformManagerRef().FindTargetPlatform(PlatformName);

	check(CurrentTargetPlatform);
	// Request compressed data
	if (UAnimSequence* Sequence = WeakAnimSequence.Get())
	{
		Sequence->BeginCacheDerivedData(CurrentTargetPlatform);
		bCachedHasCompressionData = false;
		OnRegenerateChildren.Execute();
	}
}

FText FCompressedAnimationDataNodeBuilder::GetSelectedPlatformName() const
{
	return FText::FromString(SelectedPlatformName);
}

void FCompressedAnimationDataNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{			
	NodeRow.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ContentPadding(FMargin(0,2,0,2))
			.OnClicked_Lambda( [this]() { bExpanded = !bExpanded; OnToggleExpansion.ExecuteIfBound(bExpanded); return FReply::Handled(); } )
			.ForegroundColor( FSlateColor::UseForeground() )
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this]()
				{
					return LOCTEXT("CompressedAnimationDataLabel", "Compressed Animation Data");
				})
			]
		]
	];
		
	NodeRow.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&PlatformsList)
			.InitiallySelectedItem(*PlatformsList.FindByPredicate([this](TSharedPtr<FString> SharedPlatform) { return *SharedPlatform == CurrentTargetPlatform->CookingDeviceProfileName();}))
			.OnGenerateWidget_Static(&FCompressedAnimationDataNodeBuilder::OnGeneratePlatformListWidget)
			.OnSelectionChanged_Raw(this, &FCompressedAnimationDataNodeBuilder::OnPlatformSelectionChanged)
			[
				SNew(SHorizontalBox)
							
				+SHorizontalBox::Slot()
				 .AutoWidth()
				 .VAlign(VAlign_Center)
				 .Padding(4,0)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16,16))
					.Image_Raw(this, &FCompressedAnimationDataNodeBuilder::GetSelectedPlatformBrush)
				]

				+SHorizontalBox::Slot()
				 .AutoWidth()
				 .VAlign(VAlign_Center)
				 .Padding(4,0)
				[
					SNew(STextBlock)
					.Text_Raw(this, &FCompressedAnimationDataNodeBuilder::GetSelectedPlatformName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
			
		+SHorizontalBox::Slot()
		 .VAlign(VAlign_Center)
		 .Padding(2.0f, 0.0f)
		 .AutoWidth()
		[
			SNew(SThrobber)
			.Visibility_Raw(this, &FCompressedAnimationDataNodeBuilder::GetCompressionIndicatorVisibility)
			.ToolTipText(LOCTEXT("CompressionTooltip", "Waiting for compressed data..."))
		]
	];
}

EVisibility FCompressedAnimationDataNodeBuilder::GetCompressionIndicatorVisibility() const
{
	if (const UAnimSequence* Sequence = WeakAnimSequence.Get())
	{
		return Sequence->HasCompressedDataForPlatform(CurrentTargetPlatform) ? EVisibility::Collapsed : EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

void FCompressedAnimationDataNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	auto GetCompressedData = [this]() -> UAnimSequence::FScopedCompressedAnimSequence
	{
		const UAnimSequence* Sequence = WeakAnimSequence.Get();
		check(Sequence);
		return Sequence->GetCompressedData(CurrentTargetPlatform);
	};

	TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::Create([this]() -> EVisibility
	{
		const UAnimSequence* Sequence = WeakAnimSequence.Get();
		check(Sequence);
		return Sequence->GetCompressedData(CurrentTargetPlatform).Get().IsValid(Sequence) ? EVisibility::Visible : EVisibility::Collapsed;
	});

	FNumberFormattingOptions Options;
	Options.MaximumFractionalDigits = 3;
	const FText InvalidText = LOCTEXT("InvalidData", "Invalid");

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("CompressedDataHash", "Compressed Data Hash"))
		.NameWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("CompressedDataHash", "Compressed Data Hash"))
		]
		.ValueWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([this, GetCompressedData, InvalidText]() -> FText
			{
				if (const UAnimSequence* AnimSequence = WeakAnimSequence.Get())
				{
					const FIoHash Hash = AnimSequence->GetDerivedDataKeyHash(CurrentTargetPlatform);
					const FString HashString = LexToString(Hash);
					return FText::FromString(HashString);
				}

				return InvalidText;
			})
		]
		.Visibility(VisibilityAttribute);
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("UncompressedSize", "Uncompressed (source) Data Size"))
	    .NameWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("UncompressedSize", "Uncompressed (source) Data Size"))
		]
		.ValueWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
			{
				UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
				if (Data.Get().IsValid(WeakAnimSequence.Get()))
				{								
					return FText::AsMemory(Data.Get().CompressedRawDataSize, &Options);
				}

				return InvalidText;
			})	
		]
		.Visibility(VisibilityAttribute);
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("CompressedDataSize", "Compressed Data Size"))
	    .NameWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("CompressedSize", "Compressed Data Size"))
		]
		.ValueWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
			{
				UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
				if (Data.Get().IsValid(WeakAnimSequence.Get()))
				{								
					return FText::AsMemory((Data.Get().CompressedDataStructure ? Data.Get().CompressedDataStructure->GetApproxCompressedSize() : 0) + Data.Get().CompressedCurveByteStream.Num(), &Options);
				}

				return InvalidText;
			})
		]
		.Visibility(VisibilityAttribute);
	}

	const FName BoneGroupName("BoneGroup");
	IDetailGroup& BoneGroup = ChildrenBuilder.AddGroup(BoneGroupName, LOCTEXT("BoneGroupLabel", "Bone Data"), true);
	{
		{
			BoneGroup.AddWidgetRow()
			.NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("CompressedBoneSize", "Compressed Bone Data Size"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()))
					{								
						return FText::AsMemory(Data.Get().CompressedDataStructure ? Data.Get().CompressedDataStructure->GetApproxCompressedSize() : 0, &Options);
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);
		}
		
		{
			UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
			const TArray<FTrackToSkeletonMap>& Tracks = Data.Get().CompressedTrackToSkeletonMapTable;
			const int32 NumTracks = Tracks.Num();
			const UAnimSequence* Sequence = WeakAnimSequence.Get();
			const USkeleton* Skeleton = Sequence ? Sequence->GetSkeleton() : nullptr;
			if (NumTracks && Skeleton)
			{
				const FName CompressedTrackNamesGroup("CompressedTrackNamesGroup");
				IDetailGroup& CompressedTrackNameGroup = BoneGroup.AddGroup(CompressedTrackNamesGroup, LOCTEXT("CompressedTrackLabel", "Compressed Track Names"));
				
				CompressedTrackNameGroup.HeaderRow()
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("CompressedTrackLabel", "Compressed Track Names"))
				]
				.ValueWidget
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::Format(LOCTEXT("NumCompressedBonesFormat", "Number of Tracks: {0}"), FText::AsNumber(NumTracks)))
				];
				for (int32 Index = 0; Index < NumTracks; ++Index)
				{
					const int32 SkeletonIndex = Data.Get().GetSkeletonIndexFromTrackIndex(Index);
					
					CompressedTrackNameGroup.AddWidgetRow()
					.WholeRowWidget
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Fill)
						.Padding(5, 0, 0, 0)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(FText::FromName(Skeleton->GetReferenceSkeleton().GetBoneName(SkeletonIndex)))
						]
					]
					.Visibility(VisibilityAttribute);
				}
			}
		}	

		{
			const FName CompressedBoneErrorStats("CompressedBoneErrorStats");
			IDetailGroup& CompressedBoneErrorStatGroup = BoneGroup.AddGroup(CompressedBoneErrorStats, LOCTEXT("BoneErrorStatGroupLabel", "Bone Compression Statistics"));

			CompressedBoneErrorStatGroup.AddWidgetRow()
		    .NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("AverageErrorLabel", "Average Error"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()) && Data.Get().CompressedDataStructure.IsValid())
					{								
						return FText::AsNumber(Data.Get().CompressedDataStructure->BoneCompressionErrorStats.AverageError);
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);
				
			CompressedBoneErrorStatGroup.AddWidgetRow()
		    .NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaximumErrorLabel", "Maximum Error"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()) && Data.Get().CompressedDataStructure.IsValid())
					{								
						return FText::AsNumber(Data.Get().CompressedDataStructure->BoneCompressionErrorStats.MaxError, &Options);
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);

			CompressedBoneErrorStatGroup.AddWidgetRow()
		    .NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaxErrorTimeLabel", "Maximum Error Time-Interval"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()) && Data.Get().CompressedDataStructure.IsValid())
					{								
						return FText::AsNumber(Data.Get().CompressedDataStructure->BoneCompressionErrorStats.MaxErrorTime, &Options);
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);
				
			CompressedBoneErrorStatGroup.AddWidgetRow()
		    .NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaxErrorBoneLabel", "Maximum Error Bone Name"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()) && Data.Get().CompressedDataStructure.IsValid())
					{								
						FName BoneName = NAME_None;							
						const int32 ErrorBoneIndex = Data.Get().CompressedDataStructure->BoneCompressionErrorStats.MaxErrorBone;
						if (const UAnimSequence* AnimSequence = WeakAnimSequence.Get())
						{
							if (const USkeleton* Skeleton = AnimSequence->GetSkeleton())
							{
								BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(ErrorBoneIndex);
							}
						}
						return FText::Format(LOCTEXT("MaxErrorBoneNameFormat", "{0}"), FText::FromName(BoneName));
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);
		}
	}

	const FName CurveGroupName("CurveGroup");
	IDetailGroup& CurveGroup = ChildrenBuilder.AddGroup(BoneGroupName, LOCTEXT("CurveGroupLabel", "Curve Data"), true);
	{
		{
			CurveGroup.AddWidgetRow()
			.NameWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("CompressedCurveSize", "Compressed Curve Data Size"))
			]
			.ValueWidget
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([this, GetCompressedData, InvalidText, Options]() -> FText
				{
					UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
					if (Data.Get().IsValid(WeakAnimSequence.Get()))
					{								
						return FText::AsMemory(Data.Get().CompressedCurveByteStream.Num(), &Options);
					}

					return InvalidText;
				})
			]
			.Visibility(VisibilityAttribute);
		}

		{
			UAnimSequence::FScopedCompressedAnimSequence Data = GetCompressedData();
			const TArray<FAnimCompressedCurveIndexedName>& IndexedNames = Data.Get().IndexedCurveNames;
			if (IndexedNames.Num())
			{
				const FName CompressedCurveNamesGroup("CompressedCurveNamesGroup");
				IDetailGroup& CompressedCurveNameGroup = CurveGroup.AddGroup(CompressedCurveNamesGroup, LOCTEXT("CompressedCurvesLabel", "Compressed Curve Names"));
				CompressedCurveNameGroup.HeaderRow()
				.NameContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("CompressedCurvesLabel", "Compressed Curve Names"))
				]
				.ValueWidget
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::Format(LOCTEXT("NumCompressedCurvesFormat", "Number of Curves: {0}"), FText::AsNumber(IndexedNames.Num())))
				];	
				
				const int32 NumCurves = IndexedNames.Num();
				for (int32 Index = 0; Index < NumCurves; ++Index)
				{	
					CompressedCurveNameGroup.AddWidgetRow()
					.WholeRowWidget
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Fill)
						.Padding(5, 0, 0, 0)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(FText::FromName(IndexedNames[Index].CurveName))
						]
					]
					.Visibility(VisibilityAttribute);
				}
			}
		}		
	}	
}

#undef LOCTEXT_NAMESPACE // "CompressedAnimationDataNodeBuilder"