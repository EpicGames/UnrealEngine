// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/SWidget.h"

class UAnimSequence;

class FCompressedAnimationDataNodeBuilder : public IDetailCustomNodeBuilder
{
public:	
	FCompressedAnimationDataNodeBuilder(UAnimSequence* InAnimSequence);

	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override { OnRegenerateChildren = InOnRegenerateChildren; }
	virtual void SetOnToggleExpansion(FOnToggleNodeExpansion InOnToggleExpansion) override { OnToggleExpansion = InOnToggleExpansion; }
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual FName GetName() const override { return FName("FCompressedAnimationDataNodeBuilder"); }

	virtual void Tick(float DeltaTime) override;
	virtual bool RequiresTick() const override { return true; }
protected:
	EVisibility GetCompressionIndicatorVisibility() const;  
	const FSlateBrush* GetSelectedPlatformBrush() const;                                                        
	static TSharedRef<SWidget> OnGeneratePlatformListWidget(TSharedPtr<FString> Platform);                     
	void OnPlatformSelectionChanged( TSharedPtr<FString> Platform, ESelectInfo::Type InSelectInfo );            
	FText GetSelectedPlatformName() const;      

	TWeakObjectPtr<UAnimSequence> WeakAnimSequence;
	FOnToggleNodeExpansion OnToggleExpansion;
	FSimpleDelegate OnRegenerateChildren;
	const ITargetPlatform* CurrentTargetPlatform = nullptr;
	TArray<TSharedPtr<FString>> PlatformsList;
	bool bExpanded = false;

	bool bCachedHasCompressionData = false;

	FString SelectedPlatformName;
};