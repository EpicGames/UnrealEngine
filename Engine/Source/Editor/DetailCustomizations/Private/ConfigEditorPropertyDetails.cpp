// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConfigEditorPropertyDetails.h"

#include "ConfigPropertyHelper.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IConfigEditorModule.h"
#include "IDetailPropertyRow.h"
#include "IPropertyTable.h"
#include "IPropertyTableColumn.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyVisualization/ConfigPropertyColumn.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

class SWidget;


#define LOCTEXT_NAMESPACE "ConfigPropertyHelperDetails"

////////////////////////////////////////////////
// FConfigPropertyHelperDetails

void FConfigPropertyHelperDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty("EditProperty");
	DetailBuilder.HideProperty(PropertyHandle);

	FProperty* PropValue;
	PropertyHandle->GetValue(PropValue);
	OriginalProperty = CastFieldChecked<FProperty>(PropValue);

	// Create a unique runtime UClass with the provided property as the only member. We will use this in the details view for the config hierarchy.
	FName TempClassName = *(FString(TEXT("TempConfigEditorUClass_")) + OriginalProperty->GetOwnerVariant().GetName() + OriginalProperty->GetName());
	ConfigEditorPropertyViewClass = NewObject<UClass>(GetTransientPackage(), TempClassName, RF_Standalone);

	// Keep a record of the FProperty we are looking to update
	ConfigEditorCopyOfEditProperty = CastField<FProperty>(FField::Duplicate(OriginalProperty, ConfigEditorPropertyViewClass, PropValue->GetFName()));
	ConfigEditorPropertyViewClass->ClassConfigName = OriginalProperty->GetOwnerClass()->ClassConfigName;
	ConfigEditorPropertyViewClass->SetSuperStruct(UObject::StaticClass());
	ConfigEditorPropertyViewClass->ClassFlags |= (CLASS_DefaultConfig | CLASS_Config);
	ConfigEditorPropertyViewClass->AddCppProperty(ConfigEditorCopyOfEditProperty);
	ConfigEditorPropertyViewClass->Bind();
	ConfigEditorPropertyViewClass->StaticLink(true);
	ConfigEditorPropertyViewClass->AssembleReferenceTokenStream();
	ConfigEditorPropertyViewClass->AddToRoot();
	
	// Cache the CDO for the object
	ConfigEditorPropertyViewCDO = ConfigEditorPropertyViewClass->GetDefaultObject(true);
	ConfigEditorPropertyViewCDO->AddToRoot();

	// Get access to all of the config files where this property is configurable.
	ConfigFilesHandle = DetailBuilder.GetProperty("ConfigFilePropertyObjects");
	DetailBuilder.HideProperty(ConfigFilesHandle);

	// Add the properties to a property table so we can edit these.
	IDetailCategoryBuilder& ConfigHierarchyCategory = DetailBuilder.EditCategory("ConfigHierarchy");
	ConfigHierarchyCategory.AddCustomRow(LOCTEXT("ConfigHierarchy", "ConfigHierarchy"))
	[
		// Create a property table with the values.
		ConstructPropertyTable(DetailBuilder)
	];

	// Listen for changes to the properties, we handle these by updating the ini file associated.
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FConfigPropertyHelperDetails::OnPropertyValueChanged);
}


void FConfigPropertyHelperDetails::OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	UClass* OwnerClass = ConfigEditorCopyOfEditProperty->GetOwnerClass();
	if (Object->IsA(OwnerClass))
	{
		const FString* FileName = ConfigFileAndPropertySourcePairings.FindKey(Object);
		if (FileName != nullptr)
		{
			const FString& ConfigIniName = *FileName;

			// @todo everywhere in this code - check platform and swap Config system!

			// save the object properties to this file
			UObject::FSaveConfigContext Context;
			Context.Filename = ConfigIniName;
			Context.bSaveToLayer = true;
			Context.PropertyNames = { OriginalProperty->GetFName() };
			Context.OverrideConfigSectionName = OriginalProperty->GetOwnerClass()->GetPathName();
			Object->SaveConfig(Context);

			// Update the CDO, as this change might have had an impact on it's value.
			OriginalProperty->GetOwnerClass()->GetDefaultObject()->ReloadConfig();
		}
	}
}


void FConfigPropertyHelperDetails::AddEditablePropertyForConfig(IDetailLayoutBuilder& DetailBuilder, UPropertyConfigFileDisplayRow* ConfigFilePropertyRowObj)
{
	AssociatedConfigFileAndObjectPairings.Add(ConfigFilePropertyRowObj->NormalizedConfigFileName, (UObject*)ConfigFilePropertyRowObj);

	// Add the properties to a property table so we can edit these.
	IDetailCategoryBuilder& TempCategory = DetailBuilder.EditCategory("TempCategory");

	UObject* ConfigEntryObject = StaticDuplicateObject(ConfigEditorPropertyViewCDO, GetTransientPackage(), *(ConfigFilePropertyRowObj->ConfigFileName + TEXT("_cdoDupe_") + ConfigEditorPropertyViewCDO->GetName()));
	ConfigEntryObject->AddToRoot();

	ConfigFilePropertyRowObj->TempConfigObject = ConfigEntryObject;


	FString ExistingConfigEntryValue;
	FString SectionName = OriginalProperty->GetOwnerClass()->GetPathName();
	FString ConfigName = OriginalProperty->GetOwnerClass()->GetConfigName();
	FName PropertyName = ConfigEditorCopyOfEditProperty->GetFName();


	if (const FConfigBranch* Branch = GConfig->FindBranch(*ConfigName, ConfigName))
	{
		if (const FConfigCommandStream* Layer = Branch->GetStaticLayer(ConfigFilePropertyRowObj->NormalizedConfigFileName))
		{
			if (const FConfigCommandStreamSection* Sec = Layer->Find(SectionName))
			{
				if (const FConfigValue* Value = Sec->Find(PropertyName))
				{
					ConfigEditorCopyOfEditProperty->ImportText_InContainer(*Value->GetValueForWriting(), ConfigEntryObject, nullptr, 0);
				}
			}
		}
	}

	// Cache a reference for future usage.
	ConfigFileAndPropertySourcePairings.Add(ConfigFilePropertyRowObj->NormalizedConfigFileName, (UObject*)ConfigEntryObject);

	// We need to add a property row for each config file entry. 
	// This allows us to have an editable widget for each config file.
	TArray<UObject*> ConfigPropertyDisplayObjects;
	ConfigPropertyDisplayObjects.Add(ConfigEntryObject);
	if (IDetailPropertyRow* ExternalRow = TempCategory.AddExternalObjectProperty(ConfigPropertyDisplayObjects, ConfigEditorCopyOfEditProperty->GetFName()))
	{
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		ExternalRow->GetDefaultWidgets(NameWidget, ValueWidget);

		// Register the Value widget and config file pairing with the config editor.
		// The config editor needs this to determine what a cell presenter shows.
		IConfigEditorModule& ConfigEditor = FModuleManager::Get().LoadModuleChecked<IConfigEditorModule>("ConfigEditor");
		ConfigEditor.AddExternalPropertyValueWidgetAndConfigPairing(ConfigFilePropertyRowObj->ConfigFileName, ValueWidget);
		
		// now hide the property so it is not added to the property display view
		ExternalRow->Visibility(EVisibility::Hidden);
	}
}


TSharedRef<SWidget> FConfigPropertyHelperDetails::ConstructPropertyTable(IDetailLayoutBuilder& DetailBuilder)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyTable = PropertyEditorModule.CreatePropertyTable();
	PropertyTable->SetSelectionMode(ESelectionMode::None);
	PropertyTable->SetSelectionUnit(EPropertyTableSelectionUnit::None);
	PropertyTable->SetIsUserAllowedToChangeRoot(false);
	PropertyTable->SetShowObjectName(false);

	RepopulatePropertyTable(DetailBuilder);

	TArray< TSharedRef<class IPropertyTableCustomColumn>> CustomColumns;
	TSharedRef<FConfigPropertyCustomColumn> EditPropertyColumn = MakeShareable(new FConfigPropertyCustomColumn());
	EditPropertyColumn->EditProperty = ConfigEditorCopyOfEditProperty; // FindFieldChecked<FProperty>(FPropertyConfigFileDisplay::StaticClass(), TEXT("EditProperty"));
	CustomColumns.Add(EditPropertyColumn);

	//TSharedRef<FConfigPropertyConfigFileStateCustomColumn> ConfigFileStateColumn = MakeShareable(new FConfigPropertyConfigFileStateCustomColumn());
	//ConfigFileStateColumn->SupportedProperty = FindFieldChecked<FProperty>(FPropertyConfigFileDisplay::StaticClass(), TEXT("FileState"));
	//CustomColumns.Add(ConfigFileStateColumn);

	return PropertyEditorModule.CreatePropertyTableWidget(PropertyTable.ToSharedRef(), CustomColumns);
}


void FConfigPropertyHelperDetails::RepopulatePropertyTable(IDetailLayoutBuilder& DetailBuilder)
{
	// Clear out any previous entries from the table.
	AssociatedConfigFileAndObjectPairings.Empty();

	// Add an entry for each config so the value can be set in each of the config files independently.
	uint32 ConfigCount = 0;
	TSharedPtr<IPropertyHandleArray> ConfigFilesArrayHandle = ConfigFilesHandle->AsArray();
	ConfigFilesArrayHandle->GetNumElements(ConfigCount);

	// For each config file, add the capacity to edit this property.
	for (uint32 Index = 0; Index < ConfigCount; Index++)
	{
		FString ConfigFile;
		TSharedRef<IPropertyHandle> ConfigFileElementHandle = ConfigFilesArrayHandle->GetElement(Index);

		UObject* ConfigFileSinglePropertyHelperObj = nullptr;
		ensure(ConfigFileElementHandle->GetValue(ConfigFileSinglePropertyHelperObj) == FPropertyAccess::Success);

		UPropertyConfigFileDisplayRow* ConfigFileSinglePropertyHelper = CastChecked<UPropertyConfigFileDisplayRow>(ConfigFileSinglePropertyHelperObj);

		AddEditablePropertyForConfig(DetailBuilder, ConfigFileSinglePropertyHelper);
	}

	// We need a row for each config file
	TArray<UObject*> ConfigPropertyDisplayObjects;
	AssociatedConfigFileAndObjectPairings.GenerateValueArray(ConfigPropertyDisplayObjects);
	PropertyTable->SetObjects(ConfigPropertyDisplayObjects);

	// We need a column for each property in our Helper class.
	for (FProperty* NextProperty = UPropertyConfigFileDisplayRow::StaticClass()->PropertyLink; NextProperty; NextProperty = NextProperty->PropertyLinkNext)
	{
		PropertyTable->AddColumn((TWeakFieldPtr<FProperty>)NextProperty);
	}

	// Ensure the columns cannot be removed.
	TArray<TSharedRef<IPropertyTableColumn>> Columns = PropertyTable->GetColumns();
	for (TSharedRef<IPropertyTableColumn> Column : Columns)
	{
		Column->SetFrozen(true);
	}

	// Create the 'Config File' vs 'Property' table
	PropertyTable->RequestRefresh();
}


#undef LOCTEXT_NAMESPACE
