// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class FProperty;
class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyTable;
class SWidget;
class UClass;
class UObject;
struct FPropertyChangedEvent;

/**
 * Implements details panel customizations for UConfigPropertyHelper fields.
 */
class FConfigPropertyHelperDetails
	: public IDetailCustomization
{
	//////////////////////////////////////////////////////////////////////////
	/// Detail Customization
public:

	/**
	 * Makes a new instance of this config editor detail layout class.
	 *
	 * @return The created instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FConfigPropertyHelperDetails);
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails( IDetailLayoutBuilder& InDetailBuilder ) override;


	//////////////////////////////////////////////////////////////////////////
	/// Property Table functionality
private:

	/**
	 * Create a property table of the Config Files vs Property we are editing
	 *
	 * @return The created instance of the property table.
	 */
	TSharedRef<SWidget> ConstructPropertyTable(IDetailLayoutBuilder& DetailBuilder);

	/**
	 * Populate the Property table with entries for the provided config files.
	 */
	void RepopulatePropertyTable(IDetailLayoutBuilder& DetailBuilder);

	/**
	 * Event which is triggered through changes in our editor.
	 *
	 * @param Object				- The object whose property has changed.
	 * @param PropertyChangedEvent	- The event information of the change.
	 */
	void OnPropertyValueChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

private:
	
	// The table which holds our editable properties.
	TSharedPtr<IPropertyTable> PropertyTable;


	//////////////////////////////////////////////////////////////////////////
	/// Misc operations...
private:
	void AddEditablePropertyForConfig(IDetailLayoutBuilder& DetailBuilder, class UPropertyConfigFileDisplayRow* ConfigFileProperty);


	//////////////////////////////////////////////////////////////////////////
	/// Misc properties...
private:

	// Keep track of the property handle for the config files.
	TSharedPtr<IPropertyHandle> ConfigFilesHandle;

	// A copy of the edit property we use with our helper class to update values on a per-config basis.
	FProperty* ConfigEditorCopyOfEditProperty;

	// The original property from the Project settings, that we have chosen to edit.
	FProperty* OriginalProperty;

	// A runtime class generated with the Original property as a member.
	// This allows us to edit a property on a per config basis.
	UClass* ConfigEditorPropertyViewClass;

	// The 'Class Default Object' of the runtime class we generate.
	// We duplicate this for each config file instance.
	UObject* ConfigEditorPropertyViewCDO;

	// Coupling of Config files and their editable objects.
	TMap<FString, UObject*> AssociatedConfigFileAndObjectPairings;

	// Mapping of Config Files and ConfigEditorPropertyViewClass objects.
	TMap<FString, UObject*> ConfigFileAndPropertySourcePairings;
};
