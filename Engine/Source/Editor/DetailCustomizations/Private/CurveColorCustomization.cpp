// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveColorCustomization.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/UnrealString.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/KeyHandle.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "MiniCurveEditor.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "PackageTools.h"
#include "PropertyHandle.h"
#include "SCurveEditor.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "CurveColorCustomization"

const FVector2D FCurveColorCustomization::DEFAULT_WINDOW_SIZE = FVector2D(800, 500);

TSharedRef<IPropertyTypeCustomization> FCurveColorCustomization::MakeInstance()
{
	return MakeShareable( new FCurveColorCustomization );
}

FCurveColorCustomization::~FCurveColorCustomization()
{
	if (CurveWidget.IsValid() && CurveWidget->GetCurveOwner() == this)
	{
		CurveWidget->SetCurveOwner(nullptr, false);
	}

	DestroyPopOutWindow();
}

FCurveColorCustomization::FCurveColorCustomization()
	: RuntimeCurve(NULL)
	, Owner(NULL)
	, ViewMinInput(0.0f)
	, ViewMaxInput(5.0f)
{
}

void FCurveColorCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;

	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData( StructPtrs );
	check(StructPtrs.Num()!=0);

	if (StructPtrs.Num() == 1)
	{
		static const FName AlwaysDisplayColorCurves(TEXT("AlwaysDisplayColorCurves"));
		static const FName AlwaysHideGradientEditor(TEXT("AlwaysHideGradientEditor"));

		TOptional<bool> bAlwaysDisplayColorCurves;
		if (InStructPropertyHandle->HasMetaData(AlwaysDisplayColorCurves))
		{
			bAlwaysDisplayColorCurves = InStructPropertyHandle->GetBoolMetaData(AlwaysDisplayColorCurves);
		}

		TOptional<bool> bAlwaysHideGradientEditor;
		if (InStructPropertyHandle->HasMetaData(AlwaysHideGradientEditor))
		{
			bAlwaysHideGradientEditor = InStructPropertyHandle->GetBoolMetaData(AlwaysHideGradientEditor);
		}

		RuntimeCurve = reinterpret_cast<FRuntimeCurveLinearColor*>(StructPtrs[0]);

		if (OuterObjects.Num() == 1)
		{
			Owner = OuterObjects[0];
		}

		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget( FText::GetEmpty(), FText::GetEmpty() )
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			.MinDesiredWidth(200)
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				.OnMouseDoubleClick(this, &FCurveColorCustomization::OnCurvePreviewDoubleClick)
				[
					SAssignNew(CurveWidget, SCurveEditor)
					.ViewMinInput(this, &FCurveColorCustomization::GetViewMinInput)
					.ViewMaxInput(this, &FCurveColorCustomization::GetViewMaxInput)
					.TimelineLength(this, &FCurveColorCustomization::GetTimelineLength)
					.OnSetInputViewRange(this, &FCurveColorCustomization::SetInputViewRange)
					.HideUI(false)
					.DesiredSize(FVector2D(300, 150))
					.AlwaysDisplayColorCurves(bAlwaysDisplayColorCurves.Get(false))
					.AlwaysHideGradientEditor(bAlwaysHideGradientEditor.Get(false))
				]
			];

		check(CurveWidget.IsValid());
		FCurveOwnerInterface* CurveOwner = nullptr;
		bool bCanEdit = true;
		if (RuntimeCurve && RuntimeCurve->ExternalCurve)
		{
			CurveOwner = RuntimeCurve->ExternalCurve;
			bCanEdit = false;
		}
		else
		{
			CurveOwner = this;
		}

		CurveWidget->SetCurveOwner(CurveOwner, bCanEdit);
		CurveWidget->SetPropertyUtils(StructCustomizationUtils.GetPropertyUtilities());

		// If any property on the chain gets reset while the Curve Owner is set  
		TSharedPtr<IPropertyHandle> RootHandle = StructPropertyHandle;
		while (RootHandle->GetParentHandle().IsValid())
		{
			RootHandle = RootHandle->GetParentHandle();
		}

		CurveWidget->RegisterToPropertyChangedEvent(RootHandle);
	}
	else
	{
		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MultipleCurves", "Multiple Curves - unable to modify"))
				]
			];
	}
}

void FCurveColorCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedPtr<IPropertyHandle> Child = StructPropertyHandle->GetChildHandle( ChildIndex );

		if( Child->GetProperty()->GetName() == TEXT("ExternalCurve") )
		{
			ExternalCurveHandle = Child;

			FSimpleDelegate OnCurveChangedDelegate = FSimpleDelegate::CreateSP( this, &FCurveColorCustomization::OnExternalCurveChanged, InStructPropertyHandle );
			Child->SetOnPropertyValueChanged(OnCurveChangedDelegate);

			StructBuilder.AddCustomRow(LOCTEXT("ExternalCurveLabel", "ExternalCurve"))
				.NameContent()
				[
					Child->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							Child->CreatePropertyValueWidget()
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(1,0)
						[
							SNew(SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ContentPadding(1.f)
							.ToolTipText(LOCTEXT("ConvertInternalCurveTooltip", "Convert to Internal Color Curve"))
							.OnClicked(this, &FCurveColorCustomization::OnConvertButtonClicked)
							.IsEnabled(this, &FCurveColorCustomization::IsConvertButtonEnabled)
							[
								SNew(SImage)
								.Image( FAppStyle::GetBrush(TEXT("PropertyWindow.Button_Clear")) )
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text( LOCTEXT( "CreateAssetButton", "Create External Curve" ) )
						.ToolTipText(LOCTEXT( "CreateAssetTooltip", "Create a new Color Curve asset from this curve") )
						.OnClicked(this, &FCurveColorCustomization::OnCreateButtonClicked)
						.IsEnabled(this, &FCurveColorCustomization::IsCreateButtonEnabled)
					]
				];
		}
		else
		{
			StructBuilder.AddProperty(Child.ToSharedRef());
		}
	}
}

static const FName RedCurveName(TEXT("R"));
static const FName GreenCurveName(TEXT("G"));
static const FName BlueCurveName(TEXT("B"));
static const FName AlphaCurveName(TEXT("A"));

TArray<FRichCurveEditInfoConst> FCurveColorCustomization::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[3], AlphaCurveName));
	return Curves;
}

void FCurveColorCustomization::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->ColorCurves[3], AlphaCurveName));
}

TArray<FRichCurveEditInfo> FCurveColorCustomization::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->ColorCurves[0], RedCurveName));
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->ColorCurves[1], GreenCurveName));
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->ColorCurves[2], BlueCurveName));
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->ColorCurves[3], AlphaCurveName));
	return Curves;
}

void FCurveColorCustomization::ModifyOwner()
{
	if (Owner)
	{
		Owner->Modify(true);
	}
}

TArray<const UObject*> FCurveColorCustomization::GetOwners() const
{
	TArray<const UObject*> Owners;
	if (Owner)
	{
		Owners.Add(Owner);
	}

	return Owners;
}

void FCurveColorCustomization::MakeTransactional()
{
	if (Owner)
	{
		Owner->SetFlags(Owner->GetFlags() | RF_Transactional);
	}
}

void FCurveColorCustomization::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
}

FLinearColor FCurveColorCustomization::GetLinearColorValue(float InTime) const
{
	if (RuntimeCurve)
	{
		return RuntimeCurve->GetLinearColorValue(InTime);
	}
	return FLinearColor::Black; 
}

bool FCurveColorCustomization::HasAnyAlphaKeys() const
{
	if (RuntimeCurve)
	{
		return RuntimeCurve->ColorCurves[3].GetNumKeys() > 0;
	}
	return false;
}

bool FCurveColorCustomization::IsValidCurve( FRichCurveEditInfo CurveInfo )
{
	return 
		CurveInfo.CurveToEdit == &RuntimeCurve->ColorCurves[0] ||
		CurveInfo.CurveToEdit == &RuntimeCurve->ColorCurves[1] ||
		CurveInfo.CurveToEdit == &RuntimeCurve->ColorCurves[2] ||
		CurveInfo.CurveToEdit == &RuntimeCurve->ColorCurves[3];
}

float FCurveColorCustomization::GetTimelineLength() const
{
	return 0.f;
}

void FCurveColorCustomization::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = InViewMaxInput;
	ViewMinInput = InViewMinInput;
}

void FCurveColorCustomization::OnExternalCurveChanged(TSharedRef<IPropertyHandle> CurvePropertyHandle)
{
	if (RuntimeCurve)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			CurveWidget->SetCurveOwner(RuntimeCurve->ExternalCurve, false);
		}
		else
		{
			CurveWidget->SetCurveOwner(this);
		}

		CurvePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

FReply FCurveColorCustomization::OnCreateButtonClicked()
{
	if (CurveWidget.IsValid())
	{
		FString DefaultAsset = FPackageName::GetLongPackagePath(Owner->GetOutermost()->GetName()) + TEXT("/") + Owner->GetName() + TEXT("_ExternalCurve");

		TSharedRef<SDlgPickAssetPath> NewCurveDlg =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("NewCurveDialogTitle", "Choose Location for External Curve Asset"))
			.DefaultAssetPath(FText::FromString(DefaultAsset));

		if (NewCurveDlg->ShowModal() != EAppReturnType::Cancel)
		{
			FString Package(NewCurveDlg->GetFullAssetPath().ToString());
			FString Name(NewCurveDlg->GetAssetName().ToString());

			// Find (or create!) the desired package for this object
			UPackage* Pkg = CreatePackage( *Package);
			UPackage* OutermostPkg = Pkg->GetOutermost();

			TArray<UPackage*> TopLevelPackages;
			TopLevelPackages.Add( OutermostPkg );
			if (!UPackageTools::HandleFullyLoadingPackages(TopLevelPackages, LOCTEXT("CreateANewObject", "Create a new object")))
			{
				// User aborted.
				return FReply::Handled();
			}

			if (!PromptUserIfExistingObject(Name, Package, Pkg))
			{
				return FReply::Handled();
			}

			// PromptUserIfExistingObject may have GCed and recreated our outermost package - re-acquire it here.
			OutermostPkg = Pkg->GetOutermost();

			// Create a new asset and set it as the external curve
			FName AssetName = *Name;
			UCurveLinearColor* NewCurve = Cast<UCurveLinearColor>(CurveWidget->CreateCurveObject(UCurveLinearColor::StaticClass(), Pkg, AssetName));
			if( NewCurve )
			{
				// run through points of editor data and add to external curve
				for (int32 Index = 0; Index < 4; Index++)
				{
					CopyCurveData(&RuntimeCurve->ColorCurves[Index], &NewCurve->FloatCurves[Index]);
				}

				// Set the new object as the sole selection.
				USelection* SelectionSet = GEditor->GetSelectedObjects();
				SelectionSet->DeselectAll();
				SelectionSet->Select( NewCurve );

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewCurve);

				// Mark the package dirty...
				OutermostPkg->MarkPackageDirty();

				// Make sure expected type of pointer passed to SetValue, so that it's not interpreted as a bool
				ExternalCurveHandle->SetValue(NewCurve);
			}
		}
	}
	return FReply::Handled();
}

bool FCurveColorCustomization::IsCreateButtonEnabled() const
{
	return CurveWidget.IsValid() && RuntimeCurve != NULL && RuntimeCurve->ExternalCurve == NULL;
}

FReply FCurveColorCustomization::OnConvertButtonClicked()
{
	if (RuntimeCurve && RuntimeCurve->ExternalCurve)
	{
		// clear points of editor data
		for (int32 Index = 0; Index < 4; Index++)
		{
			RuntimeCurve->ColorCurves[Index].Reset();
		}

		// run through points of external curve and add to editor data
		for (int32 Index = 0; Index < 4; Index++)
		{
			CopyCurveData(&RuntimeCurve->ExternalCurve->FloatCurves[Index], &RuntimeCurve->ColorCurves[Index]);
		}

		// null out external curve
		const UObject* NullObject = NULL;
		ExternalCurveHandle->SetValue(NullObject);
	}
	return FReply::Handled();
}

bool FCurveColorCustomization::IsConvertButtonEnabled() const
{
	return RuntimeCurve != NULL && RuntimeCurve->ExternalCurve != NULL;
}

FReply FCurveColorCustomization::OnCurvePreviewDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RuntimeCurve->ExternalCurve);
		}
		else
		{
			DestroyPopOutWindow();

			// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
			const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
			FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

			FVector2f AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition( Anchor, FCurveColorCustomization::DEFAULT_WINDOW_SIZE, true, FVector2f::ZeroVector, Orient_Horizontal );

			TSharedPtr<SWindow> Window = SNew(SWindow)
				.Title( FText::Format( LOCTEXT("WindowHeader", "{0} - Internal Color Curve Editor"), StructPropertyHandle->GetPropertyDisplayName()) )
				.ClientSize( FCurveColorCustomization::DEFAULT_WINDOW_SIZE )
				.ScreenPosition(AdjustedSummonLocation)
				.AutoCenter(EAutoCenter::None)
				.SupportsMaximize(true)
				.SupportsMinimize(false)
				.SizingRule( ESizingRule::UserSized );

			// init the mini curve editor widget
			TSharedRef<SMiniCurveEditor> MiniCurveEditor =
				SNew(SMiniCurveEditor)
				.CurveOwner(this)
				.OwnerObject(Owner)
				.ParentWindow(Window);

			Window->SetContent( MiniCurveEditor );

			// Find the window of the parent widget
			FWidgetPath WidgetPath;
			FSlateApplication::Get().GeneratePathToWidgetChecked( CurveWidget.ToSharedRef(), WidgetPath );
			Window = FSlateApplication::Get().AddWindowAsNativeChild( Window.ToSharedRef(), WidgetPath.GetWindow() );

			//hold on to the window created for external use...
			CurveEditorWindow = Window;
		}
	}
	return FReply::Handled();
}

void FCurveColorCustomization::CopyCurveData( const FRichCurve* SrcCurve, FRichCurve* DestCurve )
{
	if( SrcCurve && DestCurve )
	{
		for (auto It(SrcCurve->GetKeyIterator()); It; ++It)
		{
			const FRichCurveKey& Key = *It;
			FKeyHandle KeyHandle = DestCurve->AddKey(Key.Time, Key.Value);
			DestCurve->GetKey(KeyHandle) = Key;
		}
	}
}

void FCurveColorCustomization::DestroyPopOutWindow()
{
	if (CurveEditorWindow.IsValid())
	{
		CurveEditorWindow.Pin()->RequestDestroyWindow();
		CurveEditorWindow.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
