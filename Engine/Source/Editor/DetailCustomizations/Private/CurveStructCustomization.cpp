// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveStructCustomization.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/UnrealString.h"
#include "Curves/CurveFloat.h"
#include "Curves/KeyHandle.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "MiniCurveEditor.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "PackageTools.h"
#include "PropertyHandle.h"
#include "SCurveEditor.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "CurveStructCustomization"

const FVector2D FCurveStructCustomization::DEFAULT_WINDOW_SIZE = FVector2D(800, 500);

TSharedRef<IPropertyTypeCustomization> FCurveStructCustomization::MakeInstance() 
{
	return MakeShareable( new FCurveStructCustomization );
}

FCurveStructCustomization::~FCurveStructCustomization()
{
	if (CurveWidget.IsValid() && CurveWidget->GetCurveOwner() == this)
	{
		CurveWidget->SetCurveOwner(nullptr, false);
	}

	DestroyPopOutWindow();

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

FCurveStructCustomization::FCurveStructCustomization()
	: RuntimeCurve(NULL)
	, Owner(NULL)
	, ViewMinInput(0.0f)
	, ViewMaxInput(5.0f)
{
	GEditor->RegisterForUndo(this);
}

void FCurveStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	this->StructPropertyHandle = InStructPropertyHandle;
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData( StructPtrs );

	if (StructPtrs.Num() == 1)
	{
		static const FName XAxisName(TEXT("XAxisName"));
		static const FName YAxisName(TEXT("YAxisName"));

		TOptional<FString> XAxisString;
		if ( InStructPropertyHandle->HasMetaData(XAxisName) )
		{
			XAxisString = InStructPropertyHandle->GetMetaData(XAxisName);
		}

		TOptional<FString> YAxisString;
		if ( InStructPropertyHandle->HasMetaData(YAxisName) )
		{
			YAxisString = InStructPropertyHandle->GetMetaData(YAxisName);
		}

		RuntimeCurve = reinterpret_cast<FRuntimeFloatCurve*>(StructPtrs[0]);

		if (OuterObjects.Num() == 1)
		{
			Owner = OuterObjects[0];
		}

		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			.MinDesiredWidth(200)
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				.OnMouseDoubleClick(this, &FCurveStructCustomization::OnCurvePreviewDoubleClick)
				[
					SAssignNew(CurveWidget, SCurveEditor)
					.ViewMinInput(this, &FCurveStructCustomization::GetViewMinInput)
					.ViewMaxInput(this, &FCurveStructCustomization::GetViewMaxInput)
					.TimelineLength(this, &FCurveStructCustomization::GetTimelineLength)
					.OnSetInputViewRange(this, &FCurveStructCustomization::SetInputViewRange)
					.XAxisName(XAxisString)
					.YAxisName(YAxisString)
					.HideUI(false)
					.DesiredSize(FVector2D(300, 150))
				]
			];

		check(CurveWidget.IsValid());

		FCurveOwnerInterface* CurveOwner = nullptr;
		bool bCanEdit = true;
		if (RuntimeCurve && RuntimeCurve->ExternalCurve)
		{
			CurveOwner = RuntimeCurve->ExternalCurve;
			bCanEdit = false;
		}
		else
		{
			CurveOwner = this;
		}

		CurveWidget->SetCurveOwner(CurveOwner, bCanEdit);
		CurveWidget->SetPropertyUtils(StructCustomizationUtils.GetPropertyUtilities());

		// If any property on the chain gets reset while the Curve Owner is set  
		TSharedPtr<IPropertyHandle> RootHandle = StructPropertyHandle;
		while (RootHandle->GetParentHandle().IsValid())
		{
			RootHandle = RootHandle->GetParentHandle();
		}

		CurveWidget->RegisterToPropertyChangedEvent(RootHandle);
	}
	else
	{
		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SBorder)
			.VAlign(VAlign_Fill)
			[
				SNew(STextBlock)
				.Text(StructPtrs.Num() == 0 ? LOCTEXT("NoCurves", "No Curves - unable to modify") : LOCTEXT("MultipleCurves", "Multiple Curves - unable to modify"))
			]
		];
	}
}

void FCurveStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	// this customization can be called without an instance now in the details
	// panel, so we need to make sure to initialize members here as well.
	if(!StructPropertyHandle.IsValid())
	{
		StructPropertyHandle = InStructPropertyHandle;
	}
	
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedPtr<IPropertyHandle> Child = StructPropertyHandle->GetChildHandle( ChildIndex );

		if( Child->GetProperty()->GetName() == TEXT("ExternalCurve") )
		{
			ExternalCurveHandle = Child;

			FSimpleDelegate OnCurveChangedDelegate = FSimpleDelegate::CreateSP( this, &FCurveStructCustomization::OnExternalCurveChanged, InStructPropertyHandle );
			Child->SetOnPropertyValueChanged(OnCurveChangedDelegate);

			StructBuilder.AddCustomRow(LOCTEXT("ExternalCurveLabel", "ExternalCurve"))
				.NameContent()
				[
					Child->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							Child->CreatePropertyValueWidget()
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(1,0)
						[
							SNew(SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ContentPadding(1.f)
							.ToolTipText(LOCTEXT("ConvertInternalCurveTooltip", "Convert to Internal Curve"))
							.OnClicked(this, &FCurveStructCustomization::OnConvertButtonClicked)
							.IsEnabled(this, &FCurveStructCustomization::IsConvertButtonEnabled)
							[
								SNew(SImage)
								.Image( FAppStyle::GetBrush(TEXT("PropertyWindow.Button_Clear")) )
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text( LOCTEXT( "CreateAssetButton", "Create External Curve" ) )
							.ToolTipText(LOCTEXT( "CreateAssetTooltip", "Create a new CurveFloat asset from this curve") )
							.OnClicked(this, &FCurveStructCustomization::OnCreateButtonClicked)
							.IsEnabled(this, &FCurveStructCustomization::IsCreateButtonEnabled)
							.Visibility(Owner != nullptr ? EVisibility::Visible : EVisibility::Collapsed)
						]
					]
				];
		}
		else
		{
			StructBuilder.AddProperty(Child.ToSharedRef());
		}
	}
}

TArray<FRichCurveEditInfoConst> FCurveStructCustomization::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	FRichCurveEditInfoConst CurveEditInfo = FRichCurveEditInfoConst(&RuntimeCurve->EditorCurveData);
	if (StructPropertyHandle.IsValid())
	{
		// Use the DisplayName of the property in place of the CurveName (which is None after creation above)
		CurveEditInfo.CurveName = FName(StructPropertyHandle->GetPropertyDisplayName().ToString());
	}
	Curves.Add(CurveEditInfo);
	return Curves;
}

void FCurveStructCustomization::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	FRichCurveEditInfoConst CurveEditInfo = FRichCurveEditInfoConst(&RuntimeCurve->EditorCurveData);
	if (StructPropertyHandle.IsValid())
	{
		// Use the DisplayName of the property in place of the CurveName (which is None after creation above)
		CurveEditInfo.CurveName = FName(StructPropertyHandle->GetPropertyDisplayName().ToString());
	}
	Curves.Add(CurveEditInfo);
}

TArray<FRichCurveEditInfo> FCurveStructCustomization::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	FRichCurveEditInfo CurveEditInfo = FRichCurveEditInfo(&RuntimeCurve->EditorCurveData);
	if (StructPropertyHandle.IsValid())
	{
		// Use the DisplayName of the property in place of the CurveName (which is None after creation above)
		CurveEditInfo.CurveName = FName(StructPropertyHandle->GetPropertyDisplayName().ToString());
	}
	Curves.Add(CurveEditInfo);
	return Curves;
}

void FCurveStructCustomization::ModifyOwner()
{
	if (Owner)
	{
		Owner->Modify(true);
	}
}

TArray<const UObject*> FCurveStructCustomization::GetOwners() const
{
	TArray<const UObject*> Owners;
	if (Owner)
	{
		Owners.Add(Owner);
	}

	return Owners;
}

void FCurveStructCustomization::MakeTransactional()
{
	if (Owner)
	{
		Owner->SetFlags(Owner->GetFlags() | RF_Transactional);
	}
}

void FCurveStructCustomization::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
}

bool FCurveStructCustomization::IsValidCurve( FRichCurveEditInfo CurveInfo )
{
	return CurveInfo.CurveToEdit == &RuntimeCurve->EditorCurveData;
}

void FCurveStructCustomization::PostUndo(bool bSuccess)
{
	// reset the cached curves  
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData( StructPtrs );
	if (StructPtrs.Num() == 1)
	{
		RuntimeCurve = reinterpret_cast<FRuntimeFloatCurve*>(StructPtrs[0]);
		if (RuntimeCurve)
		{
			if (RuntimeCurve->ExternalCurve)
			{
				CurveWidget->SetCurveOwner(RuntimeCurve->ExternalCurve, false);
			}
			else
			{
				CurveWidget->SetCurveOwner(this, StructPropertyHandle->IsEditable());
			}
		}
	}
}

float FCurveStructCustomization::GetTimelineLength() const
{
	return 0.f;
}

void FCurveStructCustomization::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = InViewMaxInput;
	ViewMinInput = InViewMinInput;
}

void FCurveStructCustomization::OnExternalCurveChanged(TSharedRef<IPropertyHandle> CurvePropertyHandle)
{
	if (RuntimeCurve)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			CurveWidget->SetCurveOwner(RuntimeCurve->ExternalCurve, false);
		}
		else
		{
			CurveWidget->SetCurveOwner(this, CurvePropertyHandle->IsEditable());
		}
	}
}

FReply FCurveStructCustomization::OnCreateButtonClicked()
{
	if (CurveWidget.IsValid())
	{
		FString DefaultAsset = FPackageName::GetLongPackagePath(Owner->GetOutermost()->GetName()) + TEXT("/") + Owner->GetName() + TEXT("_ExternalCurve");

		TSharedRef<SDlgPickAssetPath> NewCurveDlg =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("NewCurveDialogTitle", "Choose Location for External Curve Asset"))
			.DefaultAssetPath(FText::FromString(DefaultAsset));

		if (NewCurveDlg->ShowModal() != EAppReturnType::Cancel)
		{
			FString Package(NewCurveDlg->GetFullAssetPath().ToString());
			FString Name(NewCurveDlg->GetAssetName().ToString());

			// Find (or create!) the desired package for this object
			UPackage* Pkg = CreatePackage( *Package);
			UPackage* OutermostPkg = Pkg->GetOutermost();

			TArray<UPackage*> TopLevelPackages;
			TopLevelPackages.Add( OutermostPkg );
			if (!UPackageTools::HandleFullyLoadingPackages(TopLevelPackages, LOCTEXT("CreateANewObject", "Create a new object")))
			{
				// User aborted.
				return FReply::Handled();
			}

			if (!PromptUserIfExistingObject(Name, Package, Pkg))
			{
				return FReply::Handled();
			}

			// PromptUserIfExistingObject may have GCed and recreated our outermost package - re-acquire it here.
			OutermostPkg = Pkg->GetOutermost();

			// Create a new asset and set it as the external curve
			FName AssetName = *Name;
			UCurveFloat* NewCurve = Cast<UCurveFloat>(CurveWidget->CreateCurveObject(UCurveFloat::StaticClass(), Pkg, AssetName));
			if( NewCurve )
			{
				// run through points of editor data and add to external curve
				CopyCurveData(&RuntimeCurve->EditorCurveData, &NewCurve->FloatCurve);

				// Set the new object as the sole selection.
				USelection* SelectionSet = GEditor->GetSelectedObjects();
				SelectionSet->DeselectAll();
				SelectionSet->Select( NewCurve );

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewCurve);

				// Mark the package dirty...
				OutermostPkg->MarkPackageDirty();

				// Make sure expected type of pointer passed to SetValue, so that it's not interpreted as a bool
				ExternalCurveHandle->SetValue(NewCurve);
			}
		}
	}
	return FReply::Handled();
}

bool FCurveStructCustomization::IsCreateButtonEnabled() const
{
	return CurveWidget.IsValid() && RuntimeCurve != NULL && RuntimeCurve->ExternalCurve == NULL;
}

FReply FCurveStructCustomization::OnConvertButtonClicked()
{
	if (RuntimeCurve && RuntimeCurve->ExternalCurve)
	{
		// clear points of editor data
		RuntimeCurve->EditorCurveData.Reset();

		// run through points of external curve and add to editor data
		CopyCurveData(&RuntimeCurve->ExternalCurve->FloatCurve, &RuntimeCurve->EditorCurveData);

		// null out external curve
		const UObject* NullObject = NULL;
		ExternalCurveHandle->SetValue(NullObject);
	}
	return FReply::Handled();
}

bool FCurveStructCustomization::IsConvertButtonEnabled() const
{
	return RuntimeCurve != NULL && RuntimeCurve->ExternalCurve != NULL;
}

FReply FCurveStructCustomization::OnCurvePreviewDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RuntimeCurve->ExternalCurve);
		}
		else
		{
			DestroyPopOutWindow();

			// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
			const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
			FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

			FVector2f AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition( Anchor, FCurveStructCustomization::DEFAULT_WINDOW_SIZE, true, FVector2f::ZeroVector, Orient_Horizontal );

			TSharedPtr<SWindow> Window = SNew(SWindow)
				.Title( FText::Format( LOCTEXT("WindowHeader", "{0} - Internal Curve Editor"), StructPropertyHandle->GetPropertyDisplayName()) )
				.ClientSize( FCurveStructCustomization::DEFAULT_WINDOW_SIZE )
				.ScreenPosition(AdjustedSummonLocation)
				.AutoCenter(EAutoCenter::None)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.SizingRule( ESizingRule::FixedSize );

			// init the mini curve editor widget
			TSharedRef<SMiniCurveEditor> MiniCurveEditor =
				SNew(SMiniCurveEditor)
				.CurveOwner(this)
				.OwnerObject(Owner)
				.ParentWindow(Window);

			Window->SetContent( MiniCurveEditor );

			// Find the window of the parent widget
			FWidgetPath WidgetPath;
			FSlateApplication::Get().GeneratePathToWidgetChecked( CurveWidget.ToSharedRef(), WidgetPath );
			Window = FSlateApplication::Get().AddWindowAsNativeChild( Window.ToSharedRef(), WidgetPath.GetWindow() );

			//hold on to the window created for external use...
			CurveEditorWindow = Window;
		}
	}
	return FReply::Handled();
}

void FCurveStructCustomization::CopyCurveData( const FRichCurve* SrcCurve, FRichCurve* DestCurve )
{
	if( SrcCurve && DestCurve )
	{
		for (auto It(SrcCurve->GetKeyIterator()); It; ++It)
		{
			const FRichCurveKey& Key = *It;
			FKeyHandle KeyHandle = DestCurve->AddKey(Key.Time, Key.Value);
			DestCurve->GetKey(KeyHandle) = Key;
		}
	}
}

void FCurveStructCustomization::DestroyPopOutWindow()
{
	if (CurveEditorWindow.IsValid())
	{
		CurveEditorWindow.Pin()->RequestDestroyWindow();
		CurveEditorWindow.Reset();
	}
}
#undef LOCTEXT_NAMESPACE
