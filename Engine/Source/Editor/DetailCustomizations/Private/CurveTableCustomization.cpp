// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/CurveTableCustomization.h"

#include "Framework/Views/ITypedTableView.h"
#include "SlotBase.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

TSharedRef<SWidget> FCurveTableCustomizationLayout::GetListContent()
{
	SAssignNew(RowNameComboListView, SListView<TSharedPtr<FString> >)
		.ListItemsSource(&RowNames)
		.OnSelectionChanged(this, &FCurveTableCustomizationLayout::OnSelectionChanged)
		.OnGenerateRow(this, &FCurveTableCustomizationLayout::HandleRowNameComboBoxGenarateWidget)
		.SelectionMode(ESelectionMode::Single);


	if (CurrentSelectedItem.IsValid())
	{
		RowNameComboListView->SetSelection(CurrentSelectedItem);
	}

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &FCurveTableCustomizationLayout::OnFilterTextChanged)
		]
	+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			RowNameComboListView.ToSharedRef()
		];
}
