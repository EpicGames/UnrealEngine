// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveVectorCustomization.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/UnrealString.h"
#include "Curves/CurveVector.h"
#include "Curves/KeyHandle.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "MiniCurveEditor.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "PackageTools.h"
#include "PropertyHandle.h"
#include "SCurveEditor.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "CurveVectorCustomization"

const FVector2D FCurveVectorCustomization::DEFAULT_WINDOW_SIZE = FVector2D(800, 500);

TSharedRef<IPropertyTypeCustomization> FCurveVectorCustomization::MakeInstance()
{
	return MakeShareable(new FCurveVectorCustomization);
}

FCurveVectorCustomization::~FCurveVectorCustomization()
{
	if (CurveWidget.IsValid() && CurveWidget->GetCurveOwner() == this)
	{
		CurveWidget->SetCurveOwner(nullptr, false);
	}

	DestroyPopOutWindow();
}

FCurveVectorCustomization::FCurveVectorCustomization()
	: RuntimeCurve(NULL)
	, Owner(NULL)
	, ViewMinInput(0.0f)
	, ViewMaxInput(5.0f)
{
}

void FCurveVectorCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;

	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData( StructPtrs );

	if (StructPtrs.Num() == 1)
	{
		static const FName XAxisName(TEXT("XAxisName"));
		static const FName YAxisName(TEXT("YAxisName"));

		TOptional<FString> XAxisString;
		if ( InStructPropertyHandle->HasMetaData(XAxisName) )
		{
			XAxisString = InStructPropertyHandle->GetMetaData(XAxisName);
		}

		TOptional<FString> YAxisString;
		if ( InStructPropertyHandle->HasMetaData(YAxisName) )
		{
			YAxisString = InStructPropertyHandle->GetMetaData(YAxisName);
		}

		RuntimeCurve = reinterpret_cast<FRuntimeVectorCurve*>(StructPtrs[0]);

		if (OuterObjects.Num() == 1)
		{
			Owner = OuterObjects[0];
		}

		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(HAlign_Fill)
			.MinDesiredWidth(200)
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				.OnMouseDoubleClick(this, &FCurveVectorCustomization::OnCurvePreviewDoubleClick)
				[
					SAssignNew(CurveWidget, SCurveEditor)
					.ViewMinInput(this, &FCurveVectorCustomization::GetViewMinInput)
					.ViewMaxInput(this, &FCurveVectorCustomization::GetViewMaxInput)
					.TimelineLength(this, &FCurveVectorCustomization::GetTimelineLength)
					.OnSetInputViewRange(this, &FCurveVectorCustomization::SetInputViewRange)
					.XAxisName(XAxisString)
					.YAxisName(YAxisString)
					.HideUI(false)
					.DesiredSize(FVector2D(300, 150))
				]
			];

		check(CurveWidget.IsValid());
		FCurveOwnerInterface* CurveOwner = nullptr;
		bool bCanEdit = true;
		if (RuntimeCurve && RuntimeCurve->ExternalCurve)
		{
			CurveOwner = RuntimeCurve->ExternalCurve;
			bCanEdit = false;
		}
		else
		{
			CurveOwner = this;
		}

		CurveWidget->SetCurveOwner(CurveOwner, bCanEdit);
		CurveWidget->SetPropertyUtils(StructCustomizationUtils.GetPropertyUtilities());

		// If any property on the chain gets reset while the Curve Owner is set  
		TSharedPtr<IPropertyHandle> RootHandle = StructPropertyHandle;
		while (RootHandle->GetParentHandle().IsValid())
		{
			RootHandle = RootHandle->GetParentHandle();
		}

		CurveWidget->RegisterToPropertyChangedEvent(RootHandle);
	}
	else
	{
		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SBorder)
				.VAlign(VAlign_Fill)
				[
					SNew(STextBlock)
					.Text(StructPtrs.Num() == 0 ? LOCTEXT("NoCurves", "No Curves - unable to modify") : LOCTEXT("MultipleCurves", "Multiple Curves - unable to modify"))
				]
			];
	}
}

void FCurveVectorCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedPtr<IPropertyHandle> Child = StructPropertyHandle->GetChildHandle( ChildIndex );

		if( Child->GetProperty()->GetName() == TEXT("ExternalCurve") )
		{
			ExternalCurveHandle = Child;

			FSimpleDelegate OnCurveChangedDelegate = FSimpleDelegate::CreateSP(this, &FCurveVectorCustomization::OnExternalCurveChanged, InStructPropertyHandle);
			Child->SetOnPropertyValueChanged(OnCurveChangedDelegate);

			StructBuilder.AddCustomRow(LOCTEXT("ExternalCurveLabel", "ExternalCurve"))
				.NameContent()
				[
					Child->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							Child->CreatePropertyValueWidget()
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(1,0)
						[
							SNew(SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ContentPadding(1.f)
							.ToolTipText(LOCTEXT("ConvertInternalCurveTooltip", "Convert to Internal Curve"))
							.OnClicked(this, &FCurveVectorCustomization::OnConvertButtonClicked)
							.IsEnabled(this, &FCurveVectorCustomization::IsConvertButtonEnabled)
							[
								SNew(SImage)
								.Image( FAppStyle::GetBrush(TEXT("PropertyWindow.Button_Clear")) )
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.Text( LOCTEXT( "CreateAssetButton", "Create External Curve" ) )
							.ToolTipText(LOCTEXT( "CreateAssetTooltip", "Create a new CurveVector asset from this curve") )
							.OnClicked(this, &FCurveVectorCustomization::OnCreateButtonClicked)
							.IsEnabled(this, &FCurveVectorCustomization::IsCreateButtonEnabled)
							.Visibility(Owner != nullptr ? EVisibility::Visible : EVisibility::Collapsed)
						]
					]
				];
		}
		else
		{
			StructBuilder.AddProperty(Child.ToSharedRef());
		}
	}
}

static const FName XCurveName(TEXT("X"));
static const FName YCurveName(TEXT("Y"));
static const FName ZCurveName(TEXT("Z"));

TArray<FRichCurveEditInfoConst> FCurveVectorCustomization::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[2], ZCurveName));
	return Curves;
}

void FCurveVectorCustomization::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfoConst(&RuntimeCurve->VectorCurves[2], ZCurveName));
}

TArray<FRichCurveEditInfo> FCurveVectorCustomization::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->VectorCurves[0], XCurveName));
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->VectorCurves[1], YCurveName));
	Curves.Add(FRichCurveEditInfo(&RuntimeCurve->VectorCurves[2], ZCurveName));
	return Curves;
}

void FCurveVectorCustomization::ModifyOwner()
{
	if (Owner)
	{
		Owner->Modify(true);
	}
}

TArray<const UObject*> FCurveVectorCustomization::GetOwners() const
{
	TArray<const UObject*> Owners;
	if (Owner)
	{
		Owners.Add(Owner);
	}

	return Owners;
}

void FCurveVectorCustomization::MakeTransactional()
{
	if (Owner)
	{
		Owner->SetFlags(Owner->GetFlags() | RF_Transactional);
	}
}

void FCurveVectorCustomization::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
}

bool FCurveVectorCustomization::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	return CurveInfo.CurveToEdit == &RuntimeCurve->VectorCurves[0] ||
		   CurveInfo.CurveToEdit == &RuntimeCurve->VectorCurves[1] ||
		   CurveInfo.CurveToEdit == &RuntimeCurve->VectorCurves[2];
}

float FCurveVectorCustomization::GetTimelineLength() const
{
	return 0.f;
}

void FCurveVectorCustomization::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = InViewMaxInput;
	ViewMinInput = InViewMinInput;
}

void FCurveVectorCustomization::OnExternalCurveChanged(TSharedRef<IPropertyHandle> CurvePropertyHandle)
{
	if (RuntimeCurve)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			CurveWidget->SetCurveOwner(RuntimeCurve->ExternalCurve, false);
		}
		else
		{
			CurveWidget->SetCurveOwner(this, CurvePropertyHandle->IsEditable());
		}
	}

	CurvePropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

FReply FCurveVectorCustomization::OnCreateButtonClicked()
{
	if (CurveWidget.IsValid())
	{
		FString DefaultAsset = FPackageName::GetLongPackagePath(Owner->GetOutermost()->GetName()) + TEXT("/") + Owner->GetName() + TEXT("_ExternalCurve");

		TSharedRef<SDlgPickAssetPath> NewCurveDlg =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("NewCurveDialogTitle", "Choose Location for External Curve Asset"))
			.DefaultAssetPath(FText::FromString(DefaultAsset));

		if (NewCurveDlg->ShowModal() != EAppReturnType::Cancel)
		{
			FString Package(NewCurveDlg->GetFullAssetPath().ToString());
			FString Name(NewCurveDlg->GetAssetName().ToString());

			// Find (or create!) the desired package for this object
			UPackage* Pkg = CreatePackage( *Package);
			UPackage* OutermostPkg = Pkg->GetOutermost();

			TArray<UPackage*> TopLevelPackages;
			TopLevelPackages.Add( OutermostPkg );
			if (!UPackageTools::HandleFullyLoadingPackages(TopLevelPackages, LOCTEXT("CreateANewObject", "Create a new object")))
			{
				// User aborted.
				return FReply::Handled();
			}

			if (!PromptUserIfExistingObject(Name, Package, Pkg))
			{
				return FReply::Handled();
			}

			// PromptUserIfExistingObject may have GCed and recreated our outermost package - re-acquire it here.
			OutermostPkg = Pkg->GetOutermost();

			// Create a new asset and set it as the external curve
			FName AssetName = *Name;
			UCurveVector* NewCurve = Cast<UCurveVector>(CurveWidget->CreateCurveObject(UCurveVector::StaticClass(), Pkg, AssetName));
			if (NewCurve)
			{
				// run through points of editor data and add to external curve
				for (int32 Index = 0; Index < 3; Index++)
				{
					CopyCurveData(&RuntimeCurve->VectorCurves[Index], &NewCurve->FloatCurves[Index]);
				}

				// Set the new object as the sole selection.
				USelection* SelectionSet = GEditor->GetSelectedObjects();
				SelectionSet->DeselectAll();
				SelectionSet->Select( NewCurve );

				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewCurve);

				// Mark the package dirty...
				OutermostPkg->MarkPackageDirty();

				// Make sure expected type of pointer passed to SetValue, so that it's not interpreted as a bool
				ExternalCurveHandle->SetValue(NewCurve);
			}
		}
	}
	return FReply::Handled();
}

bool FCurveVectorCustomization::IsCreateButtonEnabled() const
{
	return CurveWidget.IsValid() && RuntimeCurve != NULL && RuntimeCurve->ExternalCurve == NULL;
}

FReply FCurveVectorCustomization::OnConvertButtonClicked()
{
	if (RuntimeCurve && RuntimeCurve->ExternalCurve)
	{
		// clear points of editor data
		for (int32 Index = 0; Index < 3; Index++)
		{
			RuntimeCurve->VectorCurves[Index].Reset();
		}

		// run through points of external curve and add to editor data
		for (int32 Index = 0; Index < 3; Index++)
		{
			CopyCurveData(&RuntimeCurve->ExternalCurve->FloatCurves[Index], &RuntimeCurve->VectorCurves[Index]);
		}

		// null out external curve
		const UObject* NullObject = NULL;
		ExternalCurveHandle->SetValue(NullObject);
	}
	return FReply::Handled();
}

bool FCurveVectorCustomization::IsConvertButtonEnabled() const
{
	return RuntimeCurve != NULL && RuntimeCurve->ExternalCurve != NULL;
}

FReply FCurveVectorCustomization::OnCurvePreviewDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (RuntimeCurve->ExternalCurve)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RuntimeCurve->ExternalCurve);
		}
		else
		{
			DestroyPopOutWindow();

			// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
			const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
			FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

			FVector2f AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor, FCurveVectorCustomization::DEFAULT_WINDOW_SIZE, true, FVector2f::ZeroVector, Orient_Horizontal);

			TSharedPtr<SWindow> Window = SNew(SWindow)
				.Title(FText::Format(LOCTEXT("WindowHeader", "{0} - Internal Vector Curve Editor"), StructPropertyHandle->GetPropertyDisplayName()))
				.ClientSize(FCurveVectorCustomization::DEFAULT_WINDOW_SIZE)
				.ScreenPosition(AdjustedSummonLocation)
				.AutoCenter(EAutoCenter::None)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.SizingRule( ESizingRule::FixedSize );

			// init the mini curve editor widget
			TSharedRef<SMiniCurveEditor> MiniCurveEditor =
				SNew(SMiniCurveEditor)
				.CurveOwner(this)
				.OwnerObject(Owner)
				.ParentWindow(Window);

			Window->SetContent( MiniCurveEditor );

			// Find the window of the parent widget
			FWidgetPath WidgetPath;
			FSlateApplication::Get().GeneratePathToWidgetChecked( CurveWidget.ToSharedRef(), WidgetPath );
			Window = FSlateApplication::Get().AddWindowAsNativeChild( Window.ToSharedRef(), WidgetPath.GetWindow() );

			//hold on to the window created for external use...
			CurveEditorWindow = Window;
		}
	}
	return FReply::Handled();
}

void FCurveVectorCustomization::CopyCurveData(const FRichCurve* SrcCurve, FRichCurve* DestCurve)
{
	if( SrcCurve && DestCurve )
	{
		for (auto It(SrcCurve->GetKeyIterator()); It; ++It)
		{
			const FRichCurveKey& Key = *It;
			FKeyHandle KeyHandle = DestCurve->AddKey(Key.Time, Key.Value);
			DestCurve->GetKey(KeyHandle) = Key;
		}
	}
}

void FCurveVectorCustomization::DestroyPopOutWindow()
{
	if (CurveEditorWindow.IsValid())
	{
		CurveEditorWindow.Pin()->RequestDestroyWindow();
		CurveEditorWindow.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
