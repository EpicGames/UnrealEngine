// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Curves/CurveOwnerInterface.h"
#include "Curves/RichCurve.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SCurveEditor;
class SWindow;
class UObject;
struct FGeometry;
struct FPointerEvent;
struct FRuntimeVectorCurve;

/**
 * Customizes a RuntimeVectorCurve struct to display a Curve Editor
 */
class FCurveVectorCustomization : public IPropertyTypeCustomization, public FCurveOwnerInterface
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual ~FCurveVectorCustomization();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	/** FCurveOwnerInterface interface */
	UE_DEPRECATED(5.6, "Use version taking a TAdderReserverRef")
	virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	virtual TArray<FRichCurveEditInfo> GetCurves() override;
	virtual void ModifyOwner() override;
	virtual TArray<const UObject*> GetOwners() const override;
	virtual void MakeTransactional() override;
	virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	virtual bool IsValidCurve( FRichCurveEditInfo CurveInfo ) override;

private:
	/**
	 * Constructor
	 */
	FCurveVectorCustomization();

	/**
	 * Get View Min Input for the Curve Editor
	 */
	float GetViewMinInput() const { return ViewMinInput; }

	/**
	 * Get View Min Input for the Curve Editor
	 */
	float GetViewMaxInput() const { return ViewMaxInput; }

	/**
	 * Get Timeline Length for the Curve Editor
	 */
	float GetTimelineLength() const;

	/**
	 * Set View Min and Max Inputs for the Curve Editor
	 */
	void SetInputViewRange(float InViewMinInput, float InViewMaxInput);

	/**
	 * Called when RuntimeVectorCurve's External Curve is changed
	 */
	void OnExternalCurveChanged(TSharedRef<IPropertyHandle> CurvePropertyHandle);

	/**
	 * Called when button clicked to create an External Vector Curve
	 */
	FReply OnCreateButtonClicked();

	/**
	 * Whether Create button is enabled
	 */
	bool IsCreateButtonEnabled() const;

	/**
	 * Called when button clicked to convert External Curve to internal curve
	 */
	FReply OnConvertButtonClicked();

	/**
	 * Whether Convert button is enabled
	 */
	bool IsConvertButtonEnabled() const;

	/**
	 * Called when user double clicks on the curve preview to open a full size editor
	 */
	FReply OnCurvePreviewDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/**
	 * Copies data from one Rich Curve to another
	 */
	static void CopyCurveData(const FRichCurve* SrcCurve, FRichCurve* DestCurve);

	/**
	 * Destroys the pop out window used for editing internal curves.
	 */
	void DestroyPopOutWindow();

private:
	/** Cached RuntimeCurveLinearColor struct handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Cached External Curve handle */
	TSharedPtr<IPropertyHandle> ExternalCurveHandle;

	/** Small preview Curve Editor */
	TSharedPtr<SCurveEditor> CurveWidget;

	/** Window for pop out Curve Editor */
	TWeakPtr<SWindow> CurveEditorWindow;

	/** Cached pointer to the actual RuntimeVectorCurve struct */
	FRuntimeVectorCurve* RuntimeCurve;

	/** Object that owns the RuntimeFloatCurve */
	UObject* Owner;

	/** View Min Input for the Curve Editor */
	float ViewMinInput;

	/** View Max Input for the Curve Editor */
	float ViewMaxInput;

	/** Size of the pop out Curve Editor window */
	static const FVector2D DEFAULT_WINDOW_SIZE;
};
