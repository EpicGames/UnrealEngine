// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomPrimitiveDataCustomization.h"

#include "Components/ActorComponent.h"
#include "Components/MeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/TextRenderComponent.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IMaterialEditor.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialInterface.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "CustomPrimitiveDataCustomization"

TSharedRef<IPropertyTypeCustomization> FCustomPrimitiveDataCustomization::MakeInstance()
{
	return MakeShareable(new FCustomPrimitiveDataCustomization);
}

FCustomPrimitiveDataCustomization::FCustomPrimitiveDataCustomization()
{
	// NOTE: Optimally would be bound to a "OnMaterialChanged" for each component
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FCustomPrimitiveDataCustomization::OnObjectPropertyChanged);
	UMaterial::OnMaterialCompilationFinished().AddRaw(this, &FCustomPrimitiveDataCustomization::OnMaterialCompiled);
}

FCustomPrimitiveDataCustomization::~FCustomPrimitiveDataCustomization()
{
	Cleanup();

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
}

void FCustomPrimitiveDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> DataProperty = PropertyHandle->GetChildHandle("Data");

	// Move the data array to be the outer, so we don't have to expand the struct
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		DataProperty->CreatePropertyValueWidget()
	];
}

void FCustomPrimitiveDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	Cleanup();

	PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	DataHandle = PropertyHandle->GetChildHandle("Data");
	DataArrayHandle = DataHandle->AsArray();

	int32 MaxPrimitiveDataIndex = INDEX_NONE;

	ForEachSelectedComponent([&](UPrimitiveComponent* Component)
	{
		PopulateParameterData(Component, MaxPrimitiveDataIndex);
	});

	uint32 NumElements;
	FPropertyAccess::Result AccessResult = GetNumElements(NumElements);

	FSimpleDelegate OnElemsChanged = FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::RequestRefresh);
	DataArrayHandle->SetOnNumElementsChanged(OnElemsChanged);
	DataHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::OnElementsModified, AccessResult, NumElements));

	const int32 NumPrimitiveIndices = FMath::Max(MaxPrimitiveDataIndex + 1, (int32)NumElements);

	if (NumPrimitiveIndices == 0)
	{
		return;
	}

	// We're only editable if the property is editable and if we're not a multi-selection situation
	const bool bDataEditable = DataHandle.IsValid() && DataHandle->IsEditable() && AccessResult == FPropertyAccess::Success;

	uint8 VectorGroupPrimIdx = 0;
	IDetailGroup* VectorGroup = NULL;

	for (uint8 PrimIdx = 0; PrimIdx < NumPrimitiveIndices; ++PrimIdx)
	{
		TSharedPtr<IPropertyHandle> ElementHandle = PrimIdx < NumElements ? TSharedPtr<IPropertyHandle>(DataArrayHandle->GetElement(PrimIdx)) : NULL;

		if (VectorGroup && (PrimIdx - VectorGroupPrimIdx) > 3)
		{
			// We're no longer in a vector group
			VectorGroup = NULL;
		}

		// Always prioritize the first vector found, and only if it's the first element of the vector
		if (VectorGroup == NULL && VectorParameterData.Contains(PrimIdx))
		{
			bool bContainsFirstElementOfVector = false;
			for (const FCustomPrimitiveDataParameter& ParameterData : VectorParameterData[PrimIdx])
			{
				if (ParameterData.IndexOffset == 0)
				{
					bContainsFirstElementOfVector = true;
					break;
				}
			}

			if (bContainsFirstElementOfVector)
			{
				// Create a collapsing group that contains our color picker, so we can quickly assign colors to our vector
				VectorGroupPrimIdx = PrimIdx;
				VectorGroup = CreateVectorGroup(ChildBuilder, PrimIdx, bDataEditable, NumElements, CustomizationUtils);
			}
		}

		if (ScalarParameterData.Contains(PrimIdx) || VectorParameterData.Contains(PrimIdx))
		{
			CreateParameterRow(ChildBuilder, PrimIdx, ElementHandle, bDataEditable, VectorGroup, CustomizationUtils);
		}
		else
		{
			// We've encountered a gap in declared custom primitive data, mark it undeclared
			TSharedRef<SWidget> UndeclaredWidget = GetUndeclaredParameterWidget(PrimIdx, CustomizationUtils);
			TSharedRef<SWidget> NameWidget = CreateNameWidget(PrimIdx, UndeclaredWidget, CustomizationUtils);

			if (ElementHandle.IsValid())
			{
				ChildBuilder.AddProperty(ElementHandle.ToSharedRef())
				.CustomWidget()
				.NameContent()
				.HAlign(HAlign_Fill)
				[
					NameWidget
				]
				.ValueContent()
				[
					ElementHandle->CreatePropertyValueWidget(false)
				]
				.IsEnabled(bDataEditable);;
			}
			else
			{
				ChildBuilder.AddCustomRow(FText::AsNumber(PrimIdx))
				.NameContent()
				.HAlign(HAlign_Fill)
				[
					NameWidget
				]
				.IsEnabled(bDataEditable);
			}
		}
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
IDetailGroup* FCustomPrimitiveDataCustomization::CreateVectorGroup(IDetailChildrenBuilder& ChildBuilder, uint8 PrimIdx, bool bDataEditable, int32 NumElements, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	IDetailGroup* VectorGroup = &ChildBuilder.AddGroup(VectorParameterData[PrimIdx][0].Info.Name, FText::FromName(VectorParameterData[PrimIdx][0].Info.Name));

	TSharedPtr<SColorBlock> ColorBlock;
	TSharedPtr<SVerticalBox> VectorGroupNameBox = SNew(SVerticalBox);

	// Use this to make sure we don't make duplicate parameters for the group header
	TSet<FGuid> AddedParametersForThisGroup;
	TSet<FName> ParameterNames;
	TSet<TSoftObjectPtr<UMaterialInterface>> Materials;

	for (const FCustomPrimitiveDataParameter& ParameterData : VectorParameterData[PrimIdx])
	{
		if (AddedParametersForThisGroup.Contains(ParameterData.ExpressionID))
		{
			continue;
		}

		Materials.Add(ParameterData.Material->GetMaterial());
		AddedParametersForThisGroup.Add(ParameterData.ExpressionID);
		ParameterNames.Add(ParameterData.Info.Name);

		TSharedRef<SHyperlink> Hyperlink = CreateHyperlink(FText::FromName(ParameterData.Info.Name), ParameterData.Material, ParameterData.ExpressionID);
		Hyperlink->SetToolTipText(LOCTEXT("VectorHyperlinkTooltip", "Jump to Vector Parameter"));

		VectorGroupNameBox->AddSlot()
		.Padding(2.f)
		[
			Hyperlink
		];
	}

	if (MaterialsToWatch.Num() != Materials.Num())
	{
		// Some materials aren't defining parameters at this index, add the undeclared parameter widget in case this was user error
		VectorGroupNameBox->AddSlot()
		.Padding(2.f)
		[
			GetUndeclaredParameterWidget(PrimIdx, CustomizationUtils)
		];
	}

	TSharedPtr<SWidget> NameContent;
	if (ParameterNames.Num() > 1)
	{
		NameContent = CreateWarningWidget(VectorGroupNameBox.ToSharedRef(), LOCTEXT("OverlappingVectorParameters", "Primitive index has overlapping parameter names declared, make sure vector names match to remove warning"));
	}
	else
	{
		NameContent = VectorGroupNameBox;
	}

	VectorGroup->HeaderRow()
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		NameContent.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled(bDataEditable)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0.f, 2.f)
		[
			SAssignNew(ColorBlock, SColorBlock)
			.Color(this, &FCustomPrimitiveDataCustomization::GetVectorColor, PrimIdx)
			.ShowBackgroundForAlpha(true)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Separate)
			.OnMouseButtonDown(this, &FCustomPrimitiveDataCustomization::OnMouseButtonDownColorBlock, PrimIdx)
			.Size(FVector2D(35.0f, 12.0f))
			.Visibility(PrimIdx < NumElements ? EVisibility::Visible : EVisibility::Collapsed)
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::OnAddedDesiredPrimitiveData, (uint8)(PrimIdx + 3)), FText(), (int32)NumElements < PrimIdx + 4)
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeEmptyButton(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::OnRemovedPrimitiveData, PrimIdx), LOCTEXT("RemoveVector", "Removes this vector (and anything after)"), (int32)PrimIdx < NumElements)
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeResetButton(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::SetDefaultVectorValue, PrimIdx), FText(), (int32)PrimIdx < NumElements)
		]
	];

	ColorBlocks.Add(PrimIdx, ColorBlock);
	
	return VectorGroup;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FCustomPrimitiveDataCustomization::CreateParameterRow(IDetailChildrenBuilder& ChildBuilder, uint8 PrimIdx, TSharedPtr<IPropertyHandle> ElementHandle, bool bDataEditable, IDetailGroup* VectorGroup, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Use this to make sure we don't make duplicate parameters in each row
	TSet<FGuid> AddedParametersForThisRow;

	TArray<FText> SearchText;
	TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	TSet<UMaterial*> Materials;
	TSet<FName> ParameterNames;

	if (VectorParameterData.Contains(PrimIdx))
	{
		for (const FCustomPrimitiveDataParameter& ParameterData : VectorParameterData[PrimIdx])
		{
			if (AddedParametersForThisRow.Contains(ParameterData.ExpressionID))
			{
				continue;
			}

			Materials.Add(ParameterData.Material->GetMaterial());
			AddedParametersForThisRow.Add(ParameterData.ExpressionID);

			FMaterialParameterMetadata ParameterMetadata;
			ParameterData.Material->GetParameterDefaultValue(EMaterialParameterType::Vector, ParameterData.Info, ParameterMetadata);

			FText ChannelName;
			switch (ParameterData.IndexOffset)
			{
			case 0:
				ChannelName = ParameterMetadata.ChannelNames.R.IsEmpty() ? LOCTEXT("DefaultVectorChannelRed", "R") : ParameterMetadata.ChannelNames.R;
				break;
			case 1:
				ChannelName = ParameterMetadata.ChannelNames.G.IsEmpty() ? LOCTEXT("DefaultVectorChannelGreen", "G") : ParameterMetadata.ChannelNames.G;
				break;
			case 2:
				ChannelName = ParameterMetadata.ChannelNames.B.IsEmpty() ? LOCTEXT("DefaultVectorChannelBlue", "B") : ParameterMetadata.ChannelNames.B;
				break;
			case 3:
				ChannelName = ParameterMetadata.ChannelNames.A.IsEmpty() ? LOCTEXT("DefaultVectorChannelAlpha", "A") : ParameterMetadata.ChannelNames.A;
				break;
			default:
				checkNoEntry();
				break;
			}

			ParameterNames.Add(*ChannelName.ToString());

			const FText ParameterNameText = FText::FromName(ParameterData.Info.Name);
			const FText ParameterName = FText::Format(
				LOCTEXT("VectorParameterName", "{0}.{1}"),
				ParameterNameText,
				ChannelName
			);

			TSharedRef<SHyperlink> Hyperlink = CreateHyperlink(ParameterName, ParameterData.Material, ParameterData.ExpressionID);

			const FText HyperlinkToolTipText = FText::Format(
				LOCTEXT("VectorChannelHyperlinkTooltip", "Jump to Vector Parameter\nNOTE: Vector channel name not used as parameter name for setters,\nuse vector parameter name \"{0}\" or primitive index {1} instead."),
				ParameterNameText,
				FText::AsNumber(PrimIdx));
			Hyperlink->SetToolTipText(HyperlinkToolTipText);

			VerticalBox->AddSlot()
			.Padding(2.f)
			[
				Hyperlink
			];

			SearchText.Add(ParameterName);
		}
	}

	if (ScalarParameterData.Contains(PrimIdx))
	{
		for (const FCustomPrimitiveDataParameter& ParameterData : ScalarParameterData[PrimIdx])
		{
			if (AddedParametersForThisRow.Contains(ParameterData.ExpressionID))
			{
				continue;
			}

			Materials.Add(ParameterData.Material->GetMaterial());
			AddedParametersForThisRow.Add(ParameterData.ExpressionID);
			ParameterNames.Add(ParameterData.Info.Name);

			const FText ParameterName = FText::FromName(ParameterData.Info.Name);

			TSharedRef<SHyperlink> Hyperlink = CreateHyperlink(ParameterName, ParameterData.Material, ParameterData.ExpressionID);
			Hyperlink->SetToolTipText(LOCTEXT("ScalarHyperlinkTooltip", "Jump to Scalar Parameter"));

			VerticalBox->AddSlot()
			.Padding(2.f)
			[
				Hyperlink
			];

			SearchText.Add(ParameterName);
		}
	}

	if (MaterialsToWatch.Num() != Materials.Num())
	{
		// Some components aren't defining parameters at this index, add the undeclared parameter widget in case this was user error
		VerticalBox->AddSlot()
		.Padding(2.f)
		[
			GetUndeclaredParameterWidget(PrimIdx, CustomizationUtils)
		];
	}

	TSharedPtr<SWidget> NameContent;
	if (ParameterNames.Num() > 1)
	{
		NameContent = CreateWarningWidget(VerticalBox.ToSharedRef(), LOCTEXT("OverlappingParameters", "Primitive index has overlapping parameter names declared, make sure scalar and/or vector channel names match to remove warning"));
	}
	else
	{
		NameContent = VerticalBox;
	}

	NameContent = CreateNameWidget(PrimIdx, NameContent.ToSharedRef(), CustomizationUtils);

	if (ElementHandle.IsValid())
	{
		// We already have data for this row, be sure to use it
		TSharedRef<IPropertyHandle> ElementHandleRef = ElementHandle.ToSharedRef();
		IDetailPropertyRow& Row = VectorGroup ? VectorGroup->AddPropertyRow(ElementHandleRef) : ChildBuilder.AddProperty(ElementHandleRef);

		TSharedRef<SWidget> ValueWidget = ElementHandle->CreatePropertyValueWidget(false);
		ValueWidget->SetEnabled(bDataEditable);
		ElementHandleRef->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::SetDefaultValue, PrimIdx));

		Row.CustomWidget()
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			NameContent.ToSharedRef()
		]
		.ValueContent()
		[
			ValueWidget
		];
	}
	else
	{
		// We don't have data for this row, add an empty row that contains the parameter names and the ability to add data up until this point
		FDetailWidgetRow& Row = VectorGroup ? VectorGroup->AddWidgetRow() : ChildBuilder.AddCustomRow(FText::Join(LOCTEXT("SearchTextDelimiter", " "), SearchText));

		Row.NameContent()
		.HAlign(HAlign_Fill)
		[
			NameContent.ToSharedRef()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(bDataEditable)
			+ SHorizontalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FCustomPrimitiveDataCustomization::OnAddedDesiredPrimitiveData, PrimIdx))
			]
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

template<typename Predicate>
void FCustomPrimitiveDataCustomization::ForEachSelectedComponent(Predicate Pred)
{
	if (PropertyUtils.IsValid())
	{
		for (TWeakObjectPtr<UObject> Object : PropertyUtils->GetSelectedObjects())
		{
			if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object.Get()))
			{
				Pred(Component);
			}
			else if (AActor* Actor = Cast<AActor>(Object.Get()))
			{
				for (UActorComponent* ActorComponent : Actor->GetComponents())
				{
					if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ActorComponent))
					{
						Pred(PrimitiveComponent);
					}
				}
			}
		}
	}
}

bool FCustomPrimitiveDataCustomization::IsSelected(UPrimitiveComponent* Component) const
{
	return Component && PropertyUtils.IsValid() && PropertyUtils->GetSelectedObjects().ContainsByPredicate(
		[WeakComp = TWeakObjectPtr<UObject>(Component), WeakActor = TWeakObjectPtr<UObject>(Component->GetOwner())](const TWeakObjectPtr<UObject>& SelectedObject)
		{
			// Selected objects could be components or actors
			return SelectedObject.IsValid() && (SelectedObject == WeakComp || SelectedObject == WeakActor);
		});
}

void FCustomPrimitiveDataCustomization::Cleanup()
{
	PropertyUtils = NULL;
	DataHandle = NULL;
	DataArrayHandle = NULL;

	ComponentsToWatch.Empty();
	ComponentMaterialCounts.Empty();
	MaterialsToWatch.Empty();
	VectorParameterData.Empty();
	ScalarParameterData.Empty();
	ColorBlocks.Empty();
}

void FCustomPrimitiveDataCustomization::PopulateParameterData(UPrimitiveComponent* PrimitiveComponent, int32& MaxPrimitiveDataIndex)
{
	check(PrimitiveComponent);
	TArray<TPair<uint8, FCustomPrimitiveDataParameter>> Parameters;
	PrimitiveComponent->GatherDefaultCustomPrimitiveDataParametersFromMaterials(Parameters, MaxPrimitiveDataIndex);

	const int32 NumMaterials = PrimitiveComponent->GetNumMaterials();

	TSet<TSoftObjectPtr<UMaterial>>& CachedComponentMaterials = ComponentsToWatch.FindOrAdd(PrimitiveComponent);
	ComponentMaterialCounts.FindOrAdd(PrimitiveComponent) = NumMaterials;

	for (auto& [PrimIdx, Parameter] : Parameters)
	{
		UMaterial* Material = Parameter.Material.IsValid() ? Parameter.Material->GetMaterial() : nullptr;

		if (!Material)
		{
			continue;
		}

		MaterialsToWatch.Add(Material);
		CachedComponentMaterials.Add(Material);

		if (Parameter.Type == EMaterialParameterType::Scalar)
		{
			ScalarParameterData.FindOrAdd(PrimIdx).Add(MoveTemp(Parameter));
		}
		else if (Parameter.Type == EMaterialParameterType::Vector)
		{
			// Add each element individually, so that we can overlap vector parameter names. Can move the last one.
			VectorParameterData.FindOrAdd(PrimIdx).Emplace_GetRef(Parameter).IndexOffset = 0;
			VectorParameterData.FindOrAdd(PrimIdx + 1).Emplace_GetRef(Parameter).IndexOffset = 1;
			VectorParameterData.FindOrAdd(PrimIdx + 2).Emplace_GetRef(Parameter).IndexOffset = 2;
			VectorParameterData.FindOrAdd(PrimIdx + 3).Emplace_GetRef(MoveTemp(Parameter)).IndexOffset = 3;
		}
	}
}

void FCustomPrimitiveDataCustomization::RequestRefresh()
{
	if (PropertyUtils.IsValid())
	{
		PropertyUtils->RequestForceRefresh();
	}
}

void FCustomPrimitiveDataCustomization::OnElementsModified(const FPropertyAccess::Result OldAccessResult, const uint32 OldNumElements)
{
	uint32 NumElements;
	FPropertyAccess::Result AccessResult = GetNumElements(NumElements);

	// There's been a change in our array structure, whether that be from change in access or size
	if (AccessResult != OldAccessResult || NumElements != OldNumElements)
	{
		RequestRefresh();
	}
}

void FCustomPrimitiveDataCustomization::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Object);
	const EPropertyChangeType::Type IgnoreFlags = EPropertyChangeType::Interactive | EPropertyChangeType::Redirected;

	if (!(PropertyChangedEvent.ChangeType & IgnoreFlags) && ComponentsToWatch.Contains(PrimComponent) && ComponentMaterialCounts.Contains(PrimComponent)
		&& IsSelected(PrimComponent)) // Need to test this in case we're hitting a stale hash in ComponentsToWatch (#jira UE-136687)
	{
		bool bMaterialChange = false;

		if (PrimComponent->IsA<UMeshComponent>())
		{
			bMaterialChange = PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials);
		}
		else if (PrimComponent->IsA<UTextRenderComponent>())
		{
			bMaterialChange = PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTextRenderComponent, TextMaterial);
		}
		else
		{
			// Fall back if not handled
			// NOTE: Optimally would be done via an "OnMaterialChanged" for each component, however,
			// the property name checks above should handle most cases
			TSet<TSoftObjectPtr<UMaterial>>& CachedComponentMaterials = ComponentsToWatch[PrimComponent];
			int32 CachedComponentMaterialCount = ComponentMaterialCounts[PrimComponent];
			const int32 NumMaterials = PrimComponent->GetNumMaterials();

			if (NumMaterials != CachedComponentMaterialCount)
			{
				bMaterialChange = true;
			}
			else
			{
				TSet<TSoftObjectPtr<UMaterial>> CurrentMaterials;
				CurrentMaterials.Reserve(NumMaterials);

				for (int32 i = 0; i < NumMaterials; ++i)
				{
					UMaterialInterface* MaterialInterface = PrimComponent->GetMaterial(i);
					UMaterial* Material = MaterialInterface ? MaterialInterface->GetMaterial() : NULL;

					if (Material)
					{
						CurrentMaterials.Add(Material);
					}
				}

				bMaterialChange = CurrentMaterials.Difference(CachedComponentMaterials).Num() > 0;
			}
		}

		if (bMaterialChange)
		{
			RequestRefresh();
		}
	}
}

void FCustomPrimitiveDataCustomization::OnMaterialCompiled(UMaterialInterface* Material)
{
	// NOTE: We use a soft object ptr here as the old material object will be stale on compile
	if (MaterialsToWatch.Contains(Material))
	{
		RequestRefresh();
	}
}

void FCustomPrimitiveDataCustomization::OnNavigate(TWeakObjectPtr<UMaterialInterface> MaterialInterface, FGuid ExpressionID)
{
	UMaterial* Material = MaterialInterface.IsValid() ? MaterialInterface->GetMaterial() : NULL;
	
	if (UMaterialExpression* Expression = Material ? Material->FindExpressionByGUID<UMaterialExpression>(ExpressionID) : NULL)
	{
		// FindExpression is recursive, so we need to ensure we open the correct asset
		UObject* Asset = Expression->GetOutermostObject();
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

		IAssetEditorInstance* AssetEditorInstance = AssetEditorSubsystem->OpenEditorForAsset(Asset) ? AssetEditorSubsystem->FindEditorForAsset(Asset, true) : NULL;
		if (AssetEditorInstance != NULL)
		{
			if (AssetEditorInstance->GetEditorName() == "MaterialEditor")
			{
				((IMaterialEditor*)AssetEditorInstance)->JumpToExpression(Expression);
			}
			else
			{
				ensureMsgf(false, TEXT("Missing navigate to expression for editor '%s'"), *AssetEditorInstance->GetEditorName().ToString());
			}
		}
	}
}

void FCustomPrimitiveDataCustomization::OnAddedDesiredPrimitiveData(uint8 PrimIdx)
{
	uint32 NumElements;
	if (GetNumElements(NumElements) == FPropertyAccess::Success && PrimIdx >= NumElements)
	{
		GEditor->BeginTransaction(LOCTEXT("OnAddedDesiredPrimitiveData", "Added Items"));

		DataHandle->NotifyPreChange();

		TArray<void*> RawArray;
		DataHandle->AccessRawData(RawArray);
		
		if (RawArray.Num() > 0)
		{
			for (uint32 i = NumElements; i <= static_cast<uint32>(PrimIdx); ++i)
			{
				SetDefaultValue(static_cast<uint8>(i));
			}
		}
		
		DataHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);

		GEditor->EndTransaction();
	}
}

void FCustomPrimitiveDataCustomization::OnRemovedPrimitiveData(uint8 PrimIdx)
{
	uint32 NumElements;
	if (GetNumElements(NumElements) == FPropertyAccess::Success && PrimIdx < NumElements)
	{
		GEditor->BeginTransaction(LOCTEXT("OnRemovedPrimitiveData", "Removed Items"));

		for (uint32 i = NumElements; i-- > static_cast<uint32>(PrimIdx); )
		{
			DataArrayHandle->DeleteItem(i);
		}

		GEditor->EndTransaction();
	}
}

FLinearColor FCustomPrimitiveDataCustomization::GetVectorColor(uint8 PrimIdx) const
{
	FVector4f Color(ForceInitToZero);

	uint32 NumElems;
	if (GetNumElements(NumElems) == FPropertyAccess::Success)
	{
		const uint32 MaxElems = FMath::Min(NumElems, static_cast<uint32>(PrimIdx + 4));

		for (uint32 i = PrimIdx; i < MaxElems; ++i)
		{
			DataArrayHandle->GetElement(i)->GetValue(Color[i - PrimIdx]);
		}
	}

	return FLinearColor(Color);
}

void FCustomPrimitiveDataCustomization::SetVectorColor(FLinearColor NewColor, uint8 PrimIdx)
{
	FVector4f Color(NewColor);

	uint32 NumElems;
	if (GetNumElements(NumElems) == FPropertyAccess::Success)
	{
		DataHandle->NotifyPreChange();
		DataHandle->EnumerateRawData([&](void* Ptr, const int32 ObjectIndex, const int32 NumObjects)
		{
			const uint32 MaxElems = FMath::Min(NumElems, PrimIdx + 4u);

			if (TArray<float>* const DataArray = static_cast<TArray<float>*>(Ptr))
			{
				for (uint32 i = PrimIdx; i < MaxElems; ++i)
				{
					(*DataArray)[i] = Color[i - PrimIdx];
				}
			}
 
			return true;
		});
		DataHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DataHandle->NotifyFinishedChangingProperties();

	}
}

void FCustomPrimitiveDataCustomization::SetDefaultValue(uint8 PrimIdx)
{
	TSet<TWeakObjectPtr<UPrimitiveComponent>> ChangedComponents;

	// Prioritize vector data since we have a color picker
	if (VectorParameterData.Contains(PrimIdx))
	{
		for (const FCustomPrimitiveDataParameter& ParameterData : VectorParameterData[PrimIdx])
		{
			if (ParameterData.Component.IsValid() && !ChangedComponents.Contains(ParameterData.Component))
			{
				FLinearColor Color(ForceInitToZero);
				if (!ParameterData.Material.IsValid() || ParameterData.Material->GetVectorParameterValue(ParameterData.Info, Color))
				{
					float* ColorPtr = reinterpret_cast<float*>(&Color);
					ParameterData.Component->SetDefaultCustomPrimitiveDataFloat(PrimIdx, ColorPtr[ParameterData.IndexOffset]);

					ChangedComponents.Add(ParameterData.Component);
				}
			}
		}
	}

	if (ScalarParameterData.Contains(PrimIdx))
	{
		for (const FCustomPrimitiveDataParameter& ParameterData : ScalarParameterData[PrimIdx])
		{
			if (ParameterData.Component.IsValid() && !ChangedComponents.Contains(ParameterData.Component))
			{
				float Value = 0.f;
				if (!ParameterData.Material.IsValid() || ParameterData.Material->GetScalarParameterValue(ParameterData.Info, Value))
				{
					ParameterData.Component->SetDefaultCustomPrimitiveDataFloat(PrimIdx, Value);

					ChangedComponents.Add(ParameterData.Component);
				}
			}
		}
	}
}

void FCustomPrimitiveDataCustomization::SetDefaultVectorValue(uint8 PrimIdx)
{
	uint32 NumElems;
	if (GetNumElements(NumElems) == FPropertyAccess::Success)
	{
		GEditor->BeginTransaction(LOCTEXT("SetDefaultVectorValue", "Reset Vector To Default"));

		const uint32 MaxElems = FMath::Min(NumElems, static_cast<uint32>(PrimIdx + 4));
		for (uint32 i = PrimIdx; i < MaxElems; ++i)
		{
			DataArrayHandle->GetElement(i)->ResetToDefault();
		}

		GEditor->EndTransaction();
	}
}

FReply FCustomPrimitiveDataCustomization::OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, uint8 PrimIdx)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	GEditor->BeginTransaction(FText::Format(LOCTEXT("SetVectorColor", "Edit Primitive Data Vector: {0}"), FText::AsNumber(PrimIdx)));

	FColorPickerArgs PickerArgs;
	PickerArgs.bOnlyRefreshOnOk = true;
	PickerArgs.bUseAlpha = true;
	PickerArgs.InitialColor = GetVectorColor(PrimIdx);
	PickerArgs.ParentWidget = ColorBlocks[PrimIdx];
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FCustomPrimitiveDataCustomization::SetVectorColor, PrimIdx);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FCustomPrimitiveDataCustomization::OnColorPickerCancelled, PrimIdx);
	PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &FCustomPrimitiveDataCustomization::OnColorPickerWindowClosed);

	OpenColorPicker(PickerArgs);

	return FReply::Handled();
}

void FCustomPrimitiveDataCustomization::OnColorPickerCancelled(FLinearColor OriginalColor, uint8 PrimIdx)
{
	SetVectorColor(OriginalColor, PrimIdx);
	GEditor->CancelTransaction(0);
}

void FCustomPrimitiveDataCustomization::OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window)
{
	GEditor->EndTransaction();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FCustomPrimitiveDataCustomization::CreateNameWidget(int32 PrimIdx, TSharedRef<SWidget> ParameterName, IPropertyTypeCustomizationUtils& CustomizationUtils) const
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(2.f, 2.f, 16.0f, 2.f)
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(PrimIdx))
			.Font(CustomizationUtils.GetRegularFont())
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(0.f, 2.0f)
		[
			ParameterName
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SHyperlink> FCustomPrimitiveDataCustomization::CreateHyperlink(FText Text, TWeakObjectPtr<UMaterialInterface> Material, const FGuid& ExpressionID)
{
	return SNew(SHyperlink)
		.Text(Text)
		.OnNavigate(this, &FCustomPrimitiveDataCustomization::OnNavigate, Material, ExpressionID)
		.Style(FAppStyle::Get(), "HoverOnlyHyperlink")
		.TextStyle(FAppStyle::Get(), "DetailsView.HyperlinkStyle");
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FCustomPrimitiveDataCustomization::GetUndeclaredParameterWidget(int32 PrimIdx, IPropertyTypeCustomizationUtils& CustomizationUtils) const
{
	const FText ToolTipText = FText::Format(
		LOCTEXT("UndeclaredParameterTooltip", "An assigned material doesn't declare a parameter for primitive index {0}"),
		FText::AsNumber(PrimIdx));
	const float FontSize = CustomizationUtils.GetRegularFont().GetClampSize();

	return SNew(SHorizontalBox)
		.ToolTipText(ToolTipText)
		+ SHorizontalBox::Slot()
		.Padding(4.f, 0.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(FontSize))
			.Image(FAppStyle::GetBrush("Icons.Warning"))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("UndeclaredParameter", "Undeclared"))
			.Font(CustomizationUtils.GetRegularFont())
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FCustomPrimitiveDataCustomization::CreateWarningWidget(TSharedRef<SWidget> Content, FText WarningText) const
{
	// Similar to SWarningOrErrorBox widget
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("RoundedWarning"))
		[
			SNew(SHorizontalBox)
			.ToolTipText(WarningText)
			+ SHorizontalBox::Slot()
			.Padding(16.f, 2.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.WarningWithColor"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0.f, 2.f, 16.f, 2.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				Content
			]
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FPropertyAccess::Result FCustomPrimitiveDataCustomization::GetNumElements(uint32& NumElements) const
{
	if (DataHandle.IsValid() && DataArrayHandle.IsValid())
	{
		DataArrayHandle->GetNumElements(NumElements);

		// This is a low touch way to work out whether we have multiple selections or not,
		// since FPropertyHandleArray::GetNumElements above always returns FPropertyAccess::Success
		void* Address;
		return DataHandle->GetValueData(Address);
	}

	NumElements = 0;
	return FPropertyAccess::Fail;
}

#undef LOCTEXT_NAMESPACE
