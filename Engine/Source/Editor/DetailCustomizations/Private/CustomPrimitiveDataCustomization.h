// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "PropertyEditorModule.h"
#include "Components/PrimitiveComponent.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDetailWidgetRow;
class FText;
class IDetailChildrenBuilder;
class IDetailGroup;
class IPropertyHandle;
class IPropertyHandleArray;
class IPropertyUtilities;
class SColorBlock;
class SWidget;
class SWindow;
class UMaterial;
class UMaterialInterface;
class UObject;
class UPrimitiveComponent;
struct FGeometry;
struct FPointerEvent;
struct FPropertyChangedEvent;

class FCustomPrimitiveDataCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FCustomPrimitiveDataCustomization();
	~FCustomPrimitiveDataCustomization();

	/** IPropertyTypeCustomization interface */
	void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	// TODO [jonathan.bard] Remove on non-release branch
	bool bDeferringRefresh = false;

	TSharedPtr<IPropertyUtilities> PropertyUtils;
	TSharedPtr<IPropertyHandle> DataHandle;
	TSharedPtr<IPropertyHandleArray> DataArrayHandle;

	TMap<TWeakObjectPtr<UPrimitiveComponent>, TSet<TSoftObjectPtr<UMaterial>>> ComponentsToWatch;
	TMap<TWeakObjectPtr<UPrimitiveComponent>, uint32> ComponentMaterialCounts;
	TSet<TSoftObjectPtr<UMaterialInterface>> MaterialsToWatch;

	TMap<uint8, TArray<FCustomPrimitiveDataParameter>> VectorParameterData;
	TMap<uint8, TArray<FCustomPrimitiveDataParameter>> ScalarParameterData;
	TMap<uint8, TSharedPtr<SColorBlock>> ColorBlocks;

	/**
	 * Creates a group for the specified vector parameter, with a header that contains the vector parameter name and
	 * a color block that opens a color picker when clicked. Also contains add, delete and reset to default buttons to
	 * quickly populate a vector parameter
	 */
	IDetailGroup* CreateVectorGroup(IDetailChildrenBuilder& ChildBuilder, uint8 PrimIdx, bool bDataEditable, int32 NumElements, IPropertyTypeCustomizationUtils& CustomizationUtils);

	/**
	 * Creates a row for the specified primitive index. Will create hyperlinks for each parameter, to take the user to the parameter's node.
	 * If there are multiple unique parameters for the one row, it will fill the name content with each individual parameter name, or 
	 * mention that the row is undefined for a component.
	 * 
	 * If the parameter doesn't have an element yet, provides an add button for the user to quickly add elements up until that parameter.
	 */
	void CreateParameterRow(IDetailChildrenBuilder& ChildBuilder, uint8 PrimIdx, TSharedPtr<IPropertyHandle> ElementHandle, bool bDataEditable, IDetailGroup* VectorGroup, IPropertyTypeCustomizationUtils& CustomizationUtils);

	template<typename Predicate>
	void ForEachSelectedComponent(Predicate Pred);
	bool IsSelected(UPrimitiveComponent* Component) const;

	void Cleanup();

	/** Populates our parameter data for scalar and vector parameters. Updates the max primitive index if necessary. */
	void PopulateParameterData(UPrimitiveComponent* PrimitiveComponent, int32& MaxPrimitiveDataIndex);

	void RequestRefresh();
	void OnElementsModified(const enum FPropertyAccess::Result OldAccessResult, const uint32 OldNumElements);
	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	void OnMaterialCompiled(UMaterialInterface* Material);

	/** Will open the material editor to the specified parameter expression when a user clicks a hyperlink */
	void OnNavigate(TWeakObjectPtr<UMaterialInterface> Material, FGuid ExpressionID);

	/** Adds elements to the primitive data array up until and including the specified PrimIdx. Also initializes elements with defaults from material interface */
	void OnAddedDesiredPrimitiveData(uint8 PrimIdx);

	/** Remove all elements up until and including this primitive index */
	void OnRemovedPrimitiveData(uint8 PrimIdx);

	FLinearColor GetVectorColor( uint8 PrimIdx) const;
	void SetVectorColor(FLinearColor NewColor, uint8 PrimIdx);

	/** Iterates through each parameter, and sets their values to the material interface's defaults on a per-component basis */
	void SetDefaultValue(uint8 PrimIdx);

	/** Helper method to SetDefaultValue for vectors, but makes one transaction for all edits */
	void SetDefaultVectorValue(uint8 PrimIdx);

	FReply OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, uint8 PrimIdx);
	void OnColorPickerCancelled(FLinearColor OriginalColor, uint8 PrimIdx);
	void OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window);

	TSharedRef<SWidget> CreateNameWidget(int32 PrimIdx, TSharedRef<SWidget> ParameterName, IPropertyTypeCustomizationUtils& CustomizationUtils) const;

	/** Hyperlink that's used for navigating to parameters */
	TSharedRef<class SHyperlink> CreateHyperlink(FText Text, TWeakObjectPtr<UMaterialInterface> Material, const FGuid& ExpressionId);

	/** Default widget that's used when a component is missing declared parameters for the specified primitive index */
	TSharedRef<SWidget> GetUndeclaredParameterWidget(int32 PrimIdx, IPropertyTypeCustomizationUtils& CustomizationUtils) const;

	/** Creates a widget wraps content with a warning icon, border and tooltip text */
	TSharedRef<SWidget> CreateWarningWidget(TSharedRef<SWidget> Content, FText WarningText) const;

	/** Get the number of elements and the access result of our primitive data array */
	enum FPropertyAccess::Result GetNumElements(uint32& NumElements) const;
};

