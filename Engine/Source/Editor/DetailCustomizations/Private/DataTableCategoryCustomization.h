// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "IDetailChildrenBuilder.h"
#include "Engine/DataTable.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "FDataTableCategoryCustomizationLayout"

/**
 * Customizes a DataTable asset to use a dropdown
 */
class FDataTableCategoryCustomizationLayout : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable(new FDataTableCategoryCustomizationLayout);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		this->StructPropertyHandle = InStructPropertyHandle;

		if (StructPropertyHandle->HasMetaData(TEXT("RowType")))
		{
			const FString& RowType = StructPropertyHandle->GetMetaData(TEXT("RowType"));
			RowTypeFilter = FName(*RowType);
			RowFilterStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowType);
		}
		
		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			];
	}

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		/** Get all the existing property handles */
		DataTablePropertyHandle		= InStructPropertyHandle->GetChildHandle("DataTable");
		ColumnNamePropertyHandle	= InStructPropertyHandle->GetChildHandle("ColumnName");
		RowContentsPropertyHandle	= InStructPropertyHandle->GetChildHandle("RowContents");

		if (DataTablePropertyHandle->IsValidHandle()
			&& ColumnNamePropertyHandle->IsValidHandle()
			&& RowContentsPropertyHandle->IsValidHandle())
		{
			/** Construct a asset picker widget with a custom filter */
			StructBuilder.AddCustomRow(LOCTEXT("DataTable_TableName", "Data Table"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataTable_TableName", "Data Table"))
					.Font(StructCustomizationUtils.GetRegularFont())
				]
				.ValueContent()
				.MaxDesiredWidth(0.0f) // don't constrain the combo button width
				[
					SNew(SObjectPropertyEntryBox)
					.PropertyHandle(DataTablePropertyHandle)
					.AllowedClass(UDataTable::StaticClass())
					.OnShouldFilterAsset(this, &FDataTableCategoryCustomizationLayout::ShouldFilterAsset)
				];
			
			FSimpleDelegate OnDataTableChangedDelegate = FSimpleDelegate::CreateSP(this, &FDataTableCategoryCustomizationLayout::OnDataTableChanged);
			DataTablePropertyHandle->SetOnPropertyValueChanged(OnDataTableChangedDelegate);

			/** Init the array of strings from the fname map */
			TSharedPtr<FString> InitialColumnValue = InitColumnWidgetContent();

			/** Construct a combo box widget to select from a list of valid options */
			StructBuilder.AddCustomRow(LOCTEXT("DataTable_ColumnName", "Column Name"))
			.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataTable_ColumnName", "Column Name"))
					.Font(StructCustomizationUtils.GetRegularFont())
				]
			.ValueContent()
				[
					SAssignNew(ColumnNameComboBox, SComboBox<TSharedPtr<FString> >)
					.OptionsSource(&ColumnNames)
					.OnGenerateWidget(this, &FDataTableCategoryCustomizationLayout::HandleColumnNameComboBoxGenarateWidget)
					.OnSelectionChanged(this, &FDataTableCategoryCustomizationLayout::OnColumnSelectionChanged)
					.InitiallySelectedItem(InitialColumnValue)
					[
						SNew(STextBlock)
						.Text(this, &FDataTableCategoryCustomizationLayout::GetColumnNameComboBoxContentText)
					]
				];

			/** Init the array of strings from the fname map */
			TSharedPtr<FString> InitialRowValue = InitRowWidgetContent();

			/** Construct a combo box widget to select from a list of valid options */
			StructBuilder.AddCustomRow(LOCTEXT("DataTable_RowContains", "Row Contains"))
			.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataTable_RowContains", "Row Contains"))
					.Font(StructCustomizationUtils.GetRegularFont())
				]
			.ValueContent()
				[
					SAssignNew(RowContentsComboBox, SComboBox<TSharedPtr<FString> >)
					.OptionsSource(&RowContents)
					.OnGenerateWidget(this, &FDataTableCategoryCustomizationLayout::HandleRowContentsComboBoxGenarateWidget)
					.OnSelectionChanged(this, &FDataTableCategoryCustomizationLayout::OnRowSelectionChanged)
					.InitiallySelectedItem(InitialRowValue)
					[
						SNew(STextBlock)
						.Text(this, &FDataTableCategoryCustomizationLayout::GetRowContentsComboBoxContentText)
					]
				];
		}
	}

private:
	/** Init the contents the combobox sources its data off */
	TSharedPtr<FString> InitRowWidgetContent()
	{
		TSharedPtr<FString> InitialValue = MakeShareable(new FString(LOCTEXT("DataTable_None", "None").ToString()));

		FName RowContains;
		const FPropertyAccess::Result RowResult = RowContentsPropertyHandle->GetValue(RowContains);
		RowContents.Empty();
		
		/** Get the properties we wish to work with */
		UDataTable* DataTable = NULL;
		DataTablePropertyHandle->GetValue((UObject*&)DataTable);

		FName ColumnName = NAME_None;
		ColumnNamePropertyHandle->GetValue(ColumnName);

		if (DataTable != NULL && ColumnName != NAME_None)
		{
			TArray<TArray<FString> > RowData = DataTable->GetTableData();
			if (RowData.Num() > 0)
			{
				TArray<FString>& ColumnTitles = RowData[0];

				// find the column we are interested in
				int Column;
				for (Column = 0; Column < ColumnTitles.Num(); ++Column)
				{
					if (ColumnName.ToString() == ColumnTitles[Column])
					{
						break;
					}
				}

				TSet<FString> Strings;

				// go through the rest of the rows and add the contents of the column we want to our list
				for (int Row = 1; Row < RowData.Num(); ++Row)
				{
					TSharedRef<FString> RowContentsItem = MakeShareable(new FString(RowData[Row][Column]));
					if (!Strings.Contains(*RowContentsItem))
					{
						RowContents.Add(RowContentsItem);
						Strings.Add(*RowContentsItem);
					}
					/** Set the initial value to the currently selected item */
					if (RowData[Row][Column] == RowContains.ToString())
					{
						InitialValue = RowContentsItem;
					}
				}
			}
		}

		/** Reset the initial value to ensure a valid entry is set */
		if (RowResult != FPropertyAccess::MultipleValues)
		{
			FName NewValue = FName(**InitialValue);
			RowContentsPropertyHandle->SetValue(NewValue);
		}

		return InitialValue;
	}

	/** Init the contents the combobox sources its data off */
	TSharedPtr<FString> InitColumnWidgetContent()
	{
		TSharedPtr<FString> InitialValue = MakeShareable(new FString(LOCTEXT("DataTable_None", "None").ToString()));

		FName ColumnName;
		const FPropertyAccess::Result ColumnResult = ColumnNamePropertyHandle->GetValue(ColumnName);
		ColumnNames.Empty();
		
		/** Get the properties we wish to work with */
		UDataTable* DataTable = NULL;
		DataTablePropertyHandle->GetValue((UObject*&)DataTable);

		if (DataTable != NULL)
		{
			TArray<TArray<FString> > RowData = DataTable->GetTableData();
			if (RowData.Num() > 0)
			{
				TArray<FString>& ColumnTitles = RowData[0];

				for (int Column = 0; Column < ColumnTitles.Num(); ++Column)
				{
					/** Create a simple array of the column names */
					TSharedRef<FString> ColumnNameItem = MakeShareable(new FString(ColumnTitles[Column]));
					ColumnNames.Add(ColumnNameItem);

					/** Set the initial value to the currently selected item */
					if (ColumnTitles[Column] == ColumnName.ToString())
					{
						InitialValue = ColumnNameItem;
					}
				}
			}
		}

		/** Reset the initial value to ensure a valid entry is set */
		if (ColumnResult != FPropertyAccess::MultipleValues)
		{
			FName NewValue = FName(**InitialValue);
			ColumnNamePropertyHandle->SetValue(NewValue);
		}

		return InitialValue;
	}

	/** Return the representation of the the column names to display */
	TSharedRef<SWidget> HandleColumnNameComboBoxGenarateWidget(TSharedPtr<FString> Item)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(*Item));
	}

	/** Return the representation of the the row names to display */
	TSharedRef<SWidget> HandleRowContentsComboBoxGenarateWidget(TSharedPtr<FString> Item)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(*Item));
	}
	
	/** Display the current column selection */
	FText GetColumnNameComboBoxContentText() const
	{
		FString ColumnNameValue;
		const FPropertyAccess::Result ColumnResult = ColumnNamePropertyHandle->GetValue(ColumnNameValue);
		if (ColumnResult != FPropertyAccess::MultipleValues)
		{
			TSharedPtr<FString> SelectedColumnName = ColumnNameComboBox->GetSelectedItem();
			if (SelectedColumnName.IsValid())
			{
				return FText::FromString(*SelectedColumnName);
			}
			else
			{
				return LOCTEXT("DataTable_None", "None");
			}
		}
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	/** Display the current row selection */
	FText GetRowContentsComboBoxContentText() const
	{
		FString RowContainsValue;
		const FPropertyAccess::Result RowResult = RowContentsPropertyHandle->GetValue(RowContainsValue);
		if (RowResult != FPropertyAccess::MultipleValues)
		{
			TSharedPtr<FString> SelectedRowContents = RowContentsComboBox->GetSelectedItem();
			if (SelectedRowContents.IsValid())
			{
				return FText::FromString(*SelectedRowContents);
			}
			else
			{
				return LOCTEXT("DataTable_None", "None");
			}
		}
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	/** Delegate to refresh the drop down when the datatable changes */
	void OnDataTableChanged()
	{
		TSharedPtr<FString> InitialColumnValue = InitColumnWidgetContent();
		ColumnNameComboBox->SetSelectedItem(InitialColumnValue);
		ColumnNameComboBox->RefreshOptions();

		TSharedPtr<FString> InitialRowValue = InitRowWidgetContent();
		RowContentsComboBox->SetSelectedItem(InitialRowValue);
		RowContentsComboBox->RefreshOptions();
	}

	/** Update the root data on a change of selection */
	void OnColumnSelectionChanged(TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		if (SelectedItem.IsValid())
		{
			FName NewValue = FName(**SelectedItem);
			ColumnNamePropertyHandle->SetValue(NewValue);
		}

		TSharedPtr<FString> InitialRowValue = InitRowWidgetContent();
		RowContentsComboBox->SetSelectedItem(InitialRowValue);
		RowContentsComboBox->RefreshOptions();
	}

	/** Update the root data on a change of selection */
	void OnRowSelectionChanged(TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo)
	{
		if (SelectedItem.IsValid())
		{
			FName NewValue = FName(**SelectedItem);
			RowContentsPropertyHandle->SetValue(NewValue);
		}
	}
	
	/** Returns true if we should hide asset due to wrong type */
	bool ShouldFilterAsset(const FAssetData& AssetData)
	{
		if (!RowTypeFilter.IsNone())
		{
			static const FName RowStructureTagName("RowStructure");
			FString RowStructure;
			if (AssetData.GetTagValue<FString>(RowStructureTagName, RowStructure))
			{
				if (RowStructure == RowTypeFilter.ToString())
				{
					return false;
				}

				// This is slow, but at the moment we don't have an alternative to the short struct name search
				UScriptStruct* RowStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowStructure);
				if (RowStruct && RowFilterStruct && RowStruct->IsChildOf(RowFilterStruct))
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

	/** The column combobox object */
	TSharedPtr<SComboBox<TSharedPtr<FString> > > ColumnNameComboBox;
	/** The row combobox object */
	TSharedPtr<SComboBox<TSharedPtr<FString> > > RowContentsComboBox;
	/** Handle to the struct properties being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> DataTablePropertyHandle;
	TSharedPtr<IPropertyHandle> RowContentsPropertyHandle;
	TSharedPtr<IPropertyHandle> ColumnNamePropertyHandle;
	/** A cached copy of strings to populate the combo box */
	TArray<TSharedPtr<FString> > RowContents;
	/** A cached copy of strings to populate the column combo box */
	TArray<TSharedPtr<FString> > ColumnNames;
	/** The MetaData derived filter for the row type */
	FName RowTypeFilter;
	UScriptStruct* RowFilterStruct = nullptr;
};

#undef LOCTEXT_NAMESPACE
