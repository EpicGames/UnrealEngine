// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "Components/WrapBox.h"
#include "Containers/Map.h"
#include "DataTableEditorUtils.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/DataTable.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/PropertyAccessUtil.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

class SToolTip;

#define LOCTEXT_NAMESPACE "FDataTableCustomizationLayout"

void FDataTableCustomizationLayout::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	this->StructPropertyHandle = InStructPropertyHandle;

	if (StructPropertyHandle->HasMetaData(TEXT("RowType")))
	{
		const FString& RowType = StructPropertyHandle->GetMetaData(TEXT("RowType"));
		RowTypeFilter = FName(*RowType);
		RowFilterStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowType);
	}

	if (StructPropertyHandle->HasMetaData(TEXT("RowPreviewProperty")))
	{
		RowPreviewPropertyName = FName(*StructPropertyHandle->GetMetaData(TEXT("RowPreviewProperty")));
	}
	else
	{
		RowPreviewPropertyName = NAME_None;
	}

	FSimpleDelegate OnDataTableChangedDelegate = FSimpleDelegate::CreateSP(this, &FDataTableCustomizationLayout::OnDataTableChanged);
	StructPropertyHandle->SetOnPropertyValueChanged(OnDataTableChangedDelegate);
}
	
FText FDataTableCustomizationLayout::GetPreviewText() const
{
	UDataTable *CurrentDataTable = nullptr;
	FName CurrentRowName;
	if (!RowPreviewPropertyName.IsNone() && GetCurrentValue(CurrentDataTable, CurrentRowName) && CurrentDataTable)
	{
		FProperty* DataTableRowProperty = PropertyAccessUtil::FindPropertyByName(RowPreviewPropertyName, CurrentDataTable->GetRowStruct());
		uint8* DataTableRow = CurrentDataTable->FindRowUnchecked(CurrentRowName);

		if (DataTableRowProperty && DataTableRow)
		{
			uint8* RowData = DataTableRowProperty->ContainerPtrToValuePtr<uint8>(DataTableRow, 0);

			FString PreviewValue;
			DataTableRowProperty->ExportText_Direct(
				PreviewValue,
				RowData,
				RowData,
				nullptr,
				PPF_PropertyWindow | PPF_Delimited | PPF_SimpleObjectText
			);

			return FText::FromString(FString::Printf(TEXT("%s = %s"), *RowPreviewPropertyName.ToString(), *PreviewValue));
		}
	}

	return FText::GetEmpty();
}

void FDataTableCustomizationLayout::CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	/** Get all the existing property handles */
	DataTablePropertyHandle = InStructPropertyHandle->GetChildHandle("DataTable");
	RowNamePropertyHandle = InStructPropertyHandle->GetChildHandle("RowName");
	const FName PropertyName = InStructPropertyHandle->GetProperty()->GetFName();

	if (DataTablePropertyHandle->IsValidHandle() && RowNamePropertyHandle->IsValidHandle())
	{
		/** Setup Change callback */
		FSimpleDelegate OnDataTableChangedDelegate = FSimpleDelegate::CreateSP(this, &FDataTableCustomizationLayout::OnDataTableChanged);
		DataTablePropertyHandle->SetOnPropertyValueChanged(OnDataTableChangedDelegate);

		FPropertyComboBoxArgs ComboArgs(RowNamePropertyHandle, 
			FOnGetPropertyComboBoxStrings::CreateSP(this, &FDataTableCustomizationLayout::OnGetRowStrings), 
			FOnGetPropertyComboBoxValue::CreateSP(this, &FDataTableCustomizationLayout::OnGetRowValueString));
		ComboArgs.ShowSearchForItemCount = 1;

		FUIAction CopyAction;
		FUIAction PasteAction;
		InStructPropertyHandle->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

		FDetailWidgetRow& InlineRow = StructBuilder.AddCustomRow(LOCTEXT("DataTableRowHandle", "Data Table Row Handle"))
			.CopyAction(CopyAction)
			.PasteAction(PasteAction)
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(800.f)
			[
				SNew(SWrapBox)
				.UseAllottedSize(true)
				.Orientation(Orient_Horizontal)
				.HAlign(HAlign_Fill)
				+ SWrapBox::Slot()
					[
						SNew(SObjectPropertyEntryBox)
						.PropertyHandle(DataTablePropertyHandle)
						.AllowedClass(UDataTable::StaticClass())
						.OnShouldFilterAsset(this, &FDataTableCustomizationLayout::ShouldFilterAsset)
					]
				+ SWrapBox::Slot()
					[
						PropertyCustomizationHelpers::MakePropertyComboBox(ComboArgs)
					]			
				+ SWrapBox::Slot()
					.Padding(8, 0)
					[
						SNew(STextBlock)
							.Visibility_Lambda([this]() { return RowPreviewPropertyName.IsNone() ? EVisibility::Hidden : EVisibility::Visible; })
							.Text(this, &FDataTableCustomizationLayout::GetPreviewText)
					]
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::CreateLambda([InStructPropertyHandle]() -> bool
				{
					return InStructPropertyHandle->IsValidHandle() && InStructPropertyHandle->DiffersFromDefault();
				}),
				FSimpleDelegate::CreateLambda([InStructPropertyHandle]()
				{
					if (InStructPropertyHandle->IsValidHandle())
					{
						InStructPropertyHandle->ResetToDefault();
					}
				})
			)
		);

		FDataTableEditorUtils::AddSearchForReferencesContextMenu(InlineRow, FExecuteAction::CreateSP(this, &FDataTableCustomizationLayout::OnSearchForReferences));				
	}
}

bool FDataTableCustomizationLayout::GetCurrentValue(UDataTable*& OutDataTable, FName& OutName) const
{
	if (RowNamePropertyHandle.IsValid() && RowNamePropertyHandle->IsValidHandle() && DataTablePropertyHandle.IsValid() && DataTablePropertyHandle->IsValidHandle())
	{
		// If either handle is multiple value or failure, fail
		UObject* SourceDataTable = nullptr;
		if (DataTablePropertyHandle->GetValue(SourceDataTable) == FPropertyAccess::Success)
		{
			OutDataTable = Cast<UDataTable>(SourceDataTable);

			if (RowNamePropertyHandle->GetValue(OutName) == FPropertyAccess::Success)
			{
				return true;
			}
		}
	}
	return false;
}

void FDataTableCustomizationLayout::OnSearchForReferences()
{
	UDataTable* DataTable;
	FName RowName;

	if (GetCurrentValue(DataTable, RowName) && DataTable)
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		AssetIdentifiers.Add(FAssetIdentifier(DataTable, RowName));

		FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
	}
}

FString FDataTableCustomizationLayout::OnGetRowValueString() const
{
	if (!RowNamePropertyHandle.IsValid() || !RowNamePropertyHandle->IsValidHandle())
	{
		return FString();
	}

	FName RowNameValue;
	const FPropertyAccess::Result RowResult = RowNamePropertyHandle->GetValue(RowNameValue);
	if (RowResult == FPropertyAccess::Success)
	{
		if (RowNameValue.IsNone())
		{
			return LOCTEXT("DataTable_None", "None").ToString();
		}
		return RowNameValue.ToString();
	}
	else if (RowResult == FPropertyAccess::Fail)
	{
		return LOCTEXT("DataTable_None", "None").ToString();
	}
	else
	{
		return LOCTEXT("MultipleValues", "Multiple Values").ToString();
	}
}

void FDataTableCustomizationLayout::OnGetRowStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const
{
	UDataTable* DataTable = nullptr;
	FName IgnoredRowName;

	// Ignore return value as we will show rows if table is the same but row names are multiple values
	GetCurrentValue(DataTable, IgnoredRowName);

	TArray<FName> AllRowNames;
	if (DataTable != nullptr)
	{
		for (TMap<FName, uint8*>::TConstIterator Iterator(DataTable->GetRowMap()); Iterator; ++Iterator)
		{
			AllRowNames.Add(Iterator.Key());
		}

		// Sort the names alphabetically.
		AllRowNames.Sort(FNameLexicalLess());
	}

	for (const FName& RowName : AllRowNames)
	{
		OutStrings.Add(MakeShared<FString>(RowName.ToString()));
		OutRestrictedItems.Add(false);
	}
}

void FDataTableCustomizationLayout::OnDataTableChanged()
{
	UDataTable* CurrentTable;
	FName OldName;

	// Clear name on table change if no longer valid
	if (GetCurrentValue(CurrentTable, OldName))
	{
		if (!CurrentTable || !CurrentTable->FindRowUnchecked(OldName))
		{
			RowNamePropertyHandle->SetValue(FName());
		}
	}
}

bool FDataTableCustomizationLayout::ShouldFilterAsset(const struct FAssetData& AssetData)
{
	if (!RowTypeFilter.IsNone())
	{
		static const FName RowStructureTagName("RowStructure");
		FString RowStructure;
		if (AssetData.GetTagValue<FString>(RowStructureTagName, RowStructure))
		{
			if (RowStructure == RowTypeFilter.ToString())
			{
				return false;
			}

			// This is slow, but at the moment we don't have an alternative to the short struct name search
			UScriptStruct* RowStruct = UClass::TryFindTypeSlow<UScriptStruct>(RowStructure);
			if (RowStruct && RowFilterStruct && RowStruct->IsChildOf(RowFilterStruct))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

