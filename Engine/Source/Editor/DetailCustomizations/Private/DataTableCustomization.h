// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IPropertyHandle;
class SToolTip;
class UDataTable;
class UScriptStruct;
struct FAssetData;

/**
 * Customizes a DataTable asset to use a dropdown
 */
class FDataTableCustomizationLayout : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable( new FDataTableCustomizationLayout );
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	/** Reads current value of handles */
	bool GetCurrentValue(UDataTable*& OutDataTable, FName& OutName) const;

	/** Returns true if we should hide asset due to wrong type */
	bool ShouldFilterAsset(const FAssetData& AssetData);

	/** Delegate to refresh the drop down when the datatable changes */
	void OnDataTableChanged();

	/** Open reference viewer */
	void OnSearchForReferences();

	/** Get list of all rows */
	void OnGetRowStrings(TArray< TSharedPtr<FString> >& OutStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) const;

	/** Gets currently selected row display string */
	FString OnGetRowValueString() const;

	/** Enables previewing of a selected row value **/
	FText GetPreviewText() const;

	/** Handle to the struct properties being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> DataTablePropertyHandle;
	TSharedPtr<IPropertyHandle> RowNamePropertyHandle;
	/** The MetaData derived filter for the row type */
	FName RowTypeFilter;
	FName RowPreviewPropertyName;
	UScriptStruct* RowFilterStruct;
	bool bShowOnlyInnerProperties = false;
};
