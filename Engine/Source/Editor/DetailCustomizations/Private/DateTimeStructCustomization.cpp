// Copyright Epic Games, Inc. All Rights Reserved.

#include "DateTimeStructCustomization.h"

#include "Containers/Array.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "InternationalizationSettingsModel.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/Timespan.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "DateTimeStructCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FDateTimeStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	/* do nothing */
}


void FDateTimeStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;

	const FText DateTimeTooltipText = LOCTEXT("Tooltip", "A date-time string in the following format: YYYY.MM.DD-hh.mm.ss +hhmm \n\nThe time can be entered as UTC (timezone-agnostic) or with a timezone specific UTC offset (+hhmm). The value is displayed here with the local timezone's UTC offset.");

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SEditableTextBox)
				.ClearKeyboardFocusOnCommit(false)
				.IsEnabled(StructPropertyHandle, &IPropertyHandle::IsEditable)
				.ForegroundColor(this, &FDateTimeStructCustomization::HandleTextBoxForegroundColor)
				.OnTextChanged(this, &FDateTimeStructCustomization::HandleTextBoxTextChanged)
				.OnTextCommitted(this, &FDateTimeStructCustomization::HandleTextBoxTextCommited)
				.ToolTipText(DateTimeTooltipText)
				.SelectAllTextOnCommit(true)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text(this, &FDateTimeStructCustomization::HandleTextBoxText)
		];
}

FDateTimeStructCustomization::FDateTimeStructCustomization()
	: InputValid(true)
{
}



/* FDateTimeStructCustomization callbacks
 *****************************************************************************/

FSlateColor FDateTimeStructCustomization::HandleTextBoxForegroundColor() const
{
	if (InputValid)
	{
		static const FName DefaultForeground("Colors.Foreground");
		return FAppStyle::Get().GetSlateColor(DefaultForeground);
	}

	static const FName Red("Colors.AccentRed");

	return FAppStyle::Get().GetSlateColor(Red);
}


FText FDateTimeStructCustomization::HandleTextBoxText() const
{
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	if (RawData.Num() != 1)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	auto DateTimePtr = static_cast<FDateTime*>(RawData[0]);
	if (!DateTimePtr)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(ToDateTimeZoneString(*DateTimePtr));
}


void FDateTimeStructCustomization::HandleTextBoxTextChanged(const FText& NewText)
{
	FDateTime DateTime;
	InputValid = ParseDateTimeZone(NewText.ToString(), DateTime);
}


void FDateTimeStructCustomization::HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo)
{
	FDateTime ParsedDateTime;

	InputValid = ParseDateTimeZone(NewText.ToString(), ParsedDateTime);
	if (InputValid && PropertyHandle.IsValid())
	{
		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);

		PropertyHandle->NotifyPreChange();
		for (auto RawDataInstance : RawData)
		{
			*(FDateTime*)RawDataInstance = ParsedDateTime;
		}
		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}


int32 FDateTimeStructCustomization::GetLocalTimezone()
{
	const UInternationalizationSettingsModel* const InternationalizationSettingsModel = GetDefault<UInternationalizationSettingsModel>();
	if (!InternationalizationSettingsModel)
	{
		return TIMEZONE_UTC;
	}

	return InternationalizationSettingsModel->GetTimezoneValue();
}


bool FDateTimeStructCustomization::ParseDateTimeZone(const FString& DateTimeZoneString, FDateTime& OutDateTime)
{
	static FString Delimiter = FString(TEXT(" "));

	// Split our DatetimeZone string into a date and a timezone marker
	FString DateString;
	FString TimezoneString;
	if (!DateTimeZoneString.Split(Delimiter, &DateString, &TimezoneString, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
	{
		DateString = DateTimeZoneString;
	}

	// Trim surrounding whitespace
	DateString.TrimStartAndEndInline();
	TimezoneString.TrimStartAndEndInline();

	// Validate date
	FDateTime LocalizedDate;
	if (DateString.IsEmpty() || !FDateTime::Parse(DateString, LocalizedDate))
	{
		return false;
	}

	// Validate timezone marker
	if (TimezoneString.IsEmpty())
	{
		// If no timezone is present, we assume the user's preferred timezone
		OutDateTime = ConvertTime(LocalizedDate, GetLocalTimezone(), TIMEZONE_UTC);
		return true;
	}

	// Fail if timezone string isn't numeric
	if (!TimezoneString.IsNumeric())
	{
		return false;
	}

	// Convert timezone into int
	int32 Timezone = FCString::Atoi(*TimezoneString);
	Timezone = ConvertShortTimezone(Timezone);

	// Check for timezones in the full-format HHMM, ex: -0500, +1345, etc
	const int32 TimezoneHour = Timezone / 100;
	const bool bHasValidMinuteOffset = ((FMath::Abs(Timezone) % 100) % 15 == 0);
	const bool bIsTimezoneHourValid = (TimezoneHour >= -12 && TimezoneHour <= 14);
	if (bHasValidMinuteOffset && bIsTimezoneHourValid)
	{
		OutDateTime = ConvertTime(LocalizedDate, Timezone, TIMEZONE_UTC);
		return true;
	}

	// Not a valid time
	return false;
}


FDateTime FDateTimeStructCustomization::ConvertTime(const FDateTime& InDate, int32 InTimezone, int32 OutTimezone)
{
	if (InTimezone == OutTimezone)
	{
		return InDate;
	}

	// Timezone Hour ranges go from -12 to +14 from UTC
	// Convert from whole-hour to the full-format HHMM (-5 -> -0500, 0 -> +0000, etc)
	InTimezone = ConvertShortTimezone(InTimezone);
	OutTimezone = ConvertShortTimezone(OutTimezone);

	// Extract timezone minutes
	const int32 InTimezoneMinutes = (FMath::Abs(InTimezone) % 100);
	const int32 OutTimezoneMinutes = (FMath::Abs(OutTimezone) % 100);

	// Calculate our Minutes difference
	const int32 MinutesDifference =	OutTimezoneMinutes - InTimezoneMinutes;

	// Calculate our Hours difference
	const int32 HoursDifference = (OutTimezone / 100) - (InTimezone / 100);

	return FDateTime(InDate + FTimespan(HoursDifference, MinutesDifference, 0));
}


FString FDateTimeStructCustomization::ToDateTimeZoneString(const FDateTime& UTCDate)
{
	const int32 DisplayTimezone = GetLocalTimezone();
	const FDateTime LocalTime = ConvertTime(UTCDate, TIMEZONE_UTC, DisplayTimezone);
	return FString::Printf(TEXT("%s %s%0.4d"), *LocalTime.ToString(), (DisplayTimezone >= 0 ? TEXT("+") : TEXT("")), DisplayTimezone);

}


int32 FDateTimeStructCustomization::ConvertShortTimezone(int32 ShortTimezone)
{
	// Convert timezones from short-format into long format, -5 -> -0500
	// Timezone Hour ranges go from -12 to +14 from UTC
	if (ShortTimezone >= -12 && ShortTimezone <= 14)
	{
		return ShortTimezone * 100;
	}

	// Not a short-form timezone
	return ShortTimezone;
}

#undef LOCTEXT_NAMESPACE
