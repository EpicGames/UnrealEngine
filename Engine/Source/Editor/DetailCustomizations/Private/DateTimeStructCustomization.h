// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class SEditableTextBox;

/**
 * Implements a details view customization for the FDateTime structure.
 */
class FDateTimeStructCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FDateTimeStructCustomization());
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	FDateTimeStructCustomization();

private:

	/** Handles getting the text color of the editable text box. */
	FSlateColor HandleTextBoxForegroundColor() const;

	/** Handles getting the text to be displayed in the editable text box. */
	FText HandleTextBoxText() const;

	/** Handles changing the value in the editable text box. */
	void HandleTextBoxTextChanged(const FText& NewText);

	/** Handles committing the text in the editable text box. */
	void HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo);

private: // Timezone handling methods
	/** UTC constant */
	static const int32 TIMEZONE_UTC = 0;

	/** Get our local timezone based on user settings*/
	static int32 GetLocalTimezone();

	/** Parse a DateTime string for timezone information and then convert that time into UTC */
	static bool ParseDateTimeZone(const FString& DateTimeZoneString, FDateTime& OutDateTime);

	/** Convert a time to another timezone */
	static FDateTime ConvertTime(const FDateTime& InDate, int32 InTimezone, int32 OutTimezone);

	/** Convert a Date to a string with the users preferred local timezone */
	static FString ToDateTimeZoneString(const FDateTime& UTCDate);

	/** Convert a short timezone to a longer timezone */
	static FORCEINLINE int32 ConvertShortTimezone(int32 ShortTimezone);

private:

	/** Holds a flag indicating whether the current input is a valid GUID. */
	bool InputValid;

	/** Holds a handle to the property being edited. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Holds the text box for editing the Guid. */
	TSharedPtr<SEditableTextBox> TextBox;
};
