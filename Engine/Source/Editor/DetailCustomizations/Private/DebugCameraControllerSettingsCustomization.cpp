// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugCameraControllerSettingsCustomization.h"
#include "Engine/DebugCameraControllerSettings.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyRestriction.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "ShowFlags.h"
#include "RHI.h"


TSharedRef<IPropertyTypeCustomization> FDebugCameraControllerSettingsViewModeIndexCustomization::MakeInstance()
{
	return MakeShareable( new FDebugCameraControllerSettingsViewModeIndexCustomization);
}

FDebugCameraControllerSettingsViewModeIndexCustomization::FDebugCameraControllerSettingsViewModeIndexCustomization()
{
}

void FDebugCameraControllerSettingsViewModeIndexCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TSharedPtr<IPropertyHandle> ViewModeIndexHandle;

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		const TSharedRef< IPropertyHandle > ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

		if (ChildHandle->GetProperty()->GetName() == TEXT("ViewModeIndex"))
		{
			ViewModeIndexHandle = ChildHandle;
		}
	}

	check(ViewModeIndexHandle.IsValid());

	TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(NSLOCTEXT("DebugCycleViewModes", "DebugCycleViewModes", "Cycle view modes for debug camera controller")));
	const UEnum* const ViewModeIndexEnum = StaticEnum<EViewModeIndex>();
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeBuffer));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeNanite));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeLumen));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeMegaLights));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeSubstrate));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeGroom));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeVirtualShadowMap));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_VisualizeGPUSkinCache));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_LWCComplexity));
	EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_StationaryLightOverlap));

	if (!GRHISupportsRayTracing || !GRHISupportsRayTracingShaders)
	{
		EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_PathTracing));
		EnumRestriction->AddHiddenValue(ViewModeIndexEnum->GetNameStringByValue((uint8)EViewModeIndex::VMI_RayTracingDebug));
	}

	ViewModeIndexHandle->AddRestriction(EnumRestriction.ToSharedRef());

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(500)
		[
			ViewModeIndexHandle->CreatePropertyValueWidget()
		];
}

void FDebugCameraControllerSettingsViewModeIndexCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

