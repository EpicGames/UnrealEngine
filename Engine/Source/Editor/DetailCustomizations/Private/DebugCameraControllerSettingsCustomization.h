// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;


class FDebugCameraControllerSettingsViewModeIndexCustomization : public IPropertyTypeCustomization
{
public:
	FDebugCameraControllerSettingsViewModeIndexCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	//~ End IPropertyTypeCustomization
};


