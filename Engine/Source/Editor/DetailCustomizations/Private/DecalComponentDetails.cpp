// Copyright Epic Games, Inc. All Rights Reserved.

#include "DecalComponentDetails.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Materials/MaterialInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "DecalComponentDetails"

static TAutoConsoleVariable<bool> CVarDecalFilterMaterialList(
	TEXT("r.Decal.FilterMaterialList"),
	true,
	TEXT("Enable filtering of material list in Decal Component details panel.")
);

TSharedRef<IDetailCustomization> FDecalComponentDetails::MakeInstance()
{
	return MakeShareable( new FDecalComponentDetails);
}

void FDecalComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}

	// Filter only show decal materials.
	TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(TEXT("DecalMaterial"));
	DetailBuilder.EditDefaultProperty(PropertyHandle)->CustomWidget()
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(PropertyHandle)
		.AllowedClass(UMaterialInterface::StaticClass())
		.ThumbnailPool(DetailBuilder.GetThumbnailPool())
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FDecalComponentDetails::ShouldFilterDecalMaterialAsset))
	];
}

bool FDecalComponentDetails::ShouldFilterDecalMaterialAsset(FAssetData const& AssetData)
{
	if (!CVarDecalFilterMaterialList.GetValueOnAnyThread())
	{
		return false;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	return !IsDecalMaterialAssetRecursive(AssetData, AssetRegistryModule.Get());
}

bool FDecalComponentDetails::IsDecalMaterialAssetRecursive(FAssetData const& AssetData, IAssetRegistry const& AssetRegistry)
{
	FString DataStr;
	if (AssetData.GetTagValue("Parent", DataStr))
	{
		FAssetData ParentAsset = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(*DataStr));
		return IsDecalMaterialAssetRecursive(ParentAsset, AssetRegistry);
	}
	else if (AssetData.GetTagValue("MaterialDomain", DataStr))
	{
		return DataStr.Equals(TEXT("MD_DeferredDecal"));
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
