// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

struct FAssetData;
class IAssetRegistry;

class FDecalComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	/** Returns false if asset should appear in filtered list. */
	static bool ShouldFilterDecalMaterialAsset(FAssetData const& AssetData);
	/** Returns true if this asset is a decal material, or child of a decal material. */
	static bool IsDecalMaterialAssetRecursive(FAssetData const& AssetData, IAssetRegistry const& AssetRegistry);
};
