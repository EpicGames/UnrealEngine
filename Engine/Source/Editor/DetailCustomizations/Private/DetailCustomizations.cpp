// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations.h"

#include "AIDataProviderValueDetails.h"
#include "ActorComponentDetails.h"
#include "ActorDetails.h"
#include "AdvancedWidgetsModule.h"
#include "AmbientSoundDetails.h"
#include "AnimMontageSegmentDetails.h"
#include "AnimSequenceDetails.h"
#include "AnimStateAliasNodeDetails.h"
#include "AnimStateNodeDetails.h"
#include "AnimTrailNodeDetails.h"
#include "AnimTransitionNodeDetails.h"
#include "AnimationAssetDetails.h"
#include "AssetImportDataCustomization.h"
#include "AssetViewerSettingsCustomization.h"
#include "AttenuationSettingsCustomizations.h"
#include "AudioSettingsDetails.h"
#include "AutoReimportDirectoryCustomization.h"
#include "BlackboardEntryDetails.h"
#include "BodyInstanceCustomization.h"
#include "BodySetupDetails.h"
#include "BoundsCopyComponentDetails.h"
#include "BrushDetails.h"
#include "CameraCropSettingsCustomization.h"
#include "CameraDetails.h"
#include "CameraFilmbackSettingsCustomization.h"
#include "CameraFocusSettingsCustomization.h"
#include "CameraLensSettingsCustomization.h"
#include "CaptureResolutionCustomization.h"
#include "CollectionReferenceStructCustomization.h"
#include "CollisionProfileDetails.h"
#include "CollisionProfileNameCustomization.h"
#include "ColorGradingSettingsCustomization.h"
#include "ComponentReferenceCustomization.h"
#include "CompositeRerouteCustomization.h"
#include "ConfigEditorPropertyDetails.h"
#include "CurveColorCustomization.h"
#include "CurveStructCustomization.h"
#include "CurveVectorCustomization.h"
#include "CustomPrimitiveDataCustomization.h"
#include "Customizations/ColorStructCustomization.h"
#include "Customizations/CurveTableCustomization.h"
#include "Customizations/LinearColorRampCustomization.h"
#include "Customizations/MathStructCustomizations.h"
#include "Customizations/MathStructProxyCustomizations.h"
#include "Customizations/SlateBrushCustomization.h"
#include "Customizations/SlateFontInfoCustomization.h"
#include "Customizations/VectorStructCustomization.h"
#include "DataTableCategoryCustomization.h"
#include "DataTableCustomization.h"
#include "DateTimeStructCustomization.h"
#include "DebugCameraControllerSettingsCustomization.h"
#include "DecalComponentDetails.h"
#include "Delegates/Delegate.h"
#include "DeviceProfileDetails.h"
#include "DialogueStructsCustomizations.h"
#include "DialogueWaveDetails.h"
#include "DirectionalLightComponentDetails.h"
#include "DirectoryPathStructCustomization.h"
#include "DistanceDatumStructCustomization.h"
#include "DocumentationActorDetails.h"
#include "EdGraphUtilities.h"
#include "EnvQueryParamInstanceCustomization.h"
#include "FbxImportUIDetails.h"
#include "FbxSceneImportDataDetails.h"
#include "FilePathStructCustomization.h"
#include "FrameRateCustomization.h"
#include "GeneralProjectSettingsDetails.h"
#include "GuidStructCustomization.h"
#include "HAL/Platform.h"
#include "HardwareTargetingSettingsDetails.h"
#include "HierarchicalSimplificationCustomizations.h"
#include "ImportantToggleSettingCustomization.h"
#include "InputSettingsDetails.h"
#include "InputStructCustomization.h"
#include "InstancedStaticMeshComponentDetails.h"
#include "Internationalization/Internationalization.h"
#include "IntervalStructCustomization.h"
#include "KeyStructCustomization.h"
#include "LevelSequenceActorDetails.h"
#include "LevelSequenceBurnInOptionsCustomization.h"
#include "LightComponentDetails.h"
#include "LightingChannelsCustomization.h"
#include "LinuxTargetSettingsDetails.h"
#include "LocalLightComponentDetails.h"
#include "MarginCustomization.h"
#include "MaterialAttributePropertyDetails.h"
#include "MaterialExpressionTextureBaseDetails.h"
#include "MaterialInstanceDynamicDetails.h"
#include "MaterialParameterCollectionDetails.h"
#include "MaterialProxySettingsCustomizations.h"
#include "MaterialShadingModelCustomization.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "MeshComponentDetails.h"
#include "MeshDeformerCustomizations.h"
#include "MeshMergingSettingsCustomization.h"
#include "MeshProxySettingsCustomizations.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "MotionControllerDetails.h"
#include "MotionControllerPinFactory.h"
#include "MoviePlayerSettingsDetails.h"
#include "MovieSceneBindingOverrideDataCustomization.h"
#include "MovieSceneCaptureCustomization.h"
#include "MovieSceneEvalOptionsCustomization.h"
#include "MovieSceneEventParametersCustomization.h"
#include "MovieSceneSequenceLoopCountCustomization.h"
#include "NavAgentSelectorCustomization.h"
#include "NavLinkStructCustomization.h"
#include "ObjectDetails.h"
#include "ParticleModuleDetails.h"
#include "ParticleSysParamStructCustomization.h"
#include "ParticleSystemComponentDetails.h"
#include "PerPlatformPropertyCustomization.h"
#include "PerQualityLevelPropertyCustomization.h"
#include "PhysicsConstraintComponentDetails.h"
#include "PhysicsSettingsDetails.h"
#include "PoseAssetDetails.h"
#include "PostProcessSettingsCustomization.h"
#include "PrimitiveComponentDetails.h"
#include "PropertyEditorModule.h"
#include "RangeStructCustomization.h"
#include "RawDistributionVectorStructCustomization.h"
#include "ReflectionCaptureDetails.h"
#include "RenderPassesCustomization.h"
#include "RotatorStructCustomization.h"
#include "SceneCaptureDetails.h"
#include "SceneComponentDetails.h"
#include "SkeletalControlNodeDetails.h"
#include "SkeletalMeshDetails.h"
#include "SkeletalMeshComponentDetails.h"
#include "SkeletalMeshLODSettingsDetails.h"
#include "SkeletalMeshReductionSettingsDetails.h"
#include "SkeletonDetails.h"
#include "SkinnedMeshComponentDetails.h"
#include "SkyLightComponentDetails.h"
#include "SlateColorCustomization.h"
#include "SlateSoundCustomization.h"
#include "SoundSubmixSendFilterCustomization.h"
#include "SoftClassPathCustomization.h"
#include "SoftObjectPathCustomization.h"
#include "Sound/SoundNodeDistanceCrossFade.h"
#include "SoundCueDetails.h"
#include "SoundSourceBusDetails.h"
#include "SoundWaveDetails.h"
#include "SourceCodeAccessSettingsDetails.h"
#include "SplineComponentDetails.h"
#include "StaticMeshActorDetails.h"
#include "StaticMeshComponentDetails.h"
#include "StereoLayerComponentCustomization.h"
#include "SubScriptStructOfCustomization.h"
#include "SupportedRangeTypes.h"	// StructsSupportingRangeVisibility
#include "TemplateStringStructCustomization.h"
#include "Templates/SharedPointer.h"
#include "TextCustomization.h"
#include "TextureCubeArrayDetails.h"
#include "TimecodeDetailsCustomization.h"
#include "TimespanStructCustomization.h"
#include "UObject/UnrealNames.h"
#include "Vector4StructCustomization.h"
#include "WindowsTargetSettingsDetails.h"
#include "WorldSettingsDetails.h"

struct FPerPlatformBool;
struct FPerPlatformFloat;
struct FPerPlatformInt;
struct FPerQualityLevelInt;
struct FPerQualityLevelFloat;

IMPLEMENT_MODULE( FDetailCustomizationsModule, DetailCustomizations );

void FDetailCustomizationsModule::StartupModule()
{
	// Load widgets (e.g. ColorGradingWheel) that some customizations depend on
	FModuleManager::Get().LoadModuleChecked<FAdvancedWidgetsModule>("AdvancedWidgets");

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	RegisterPropertyTypeCustomizations();
	RegisterObjectCustomizations();
	RegisterSectionMappings();

	TSharedPtr<FMotionControllerPinFactory> MotionControllerPinFactory = MakeShareable(new FMotionControllerPinFactory());
	FEdGraphUtilities::RegisterVisualPinFactory(MotionControllerPinFactory);

	PropertyModule.NotifyCustomizationModuleChanged();
}


void FDetailCustomizationsModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister all classes customized by name
		for (auto It = RegisteredClassNames.CreateConstIterator(); It; ++It)
		{
			if (It->IsValid())
			{
				PropertyModule.UnregisterCustomClassLayout(*It);
			}
		}

		// Unregister all structures
		for (auto It = RegisteredPropertyTypes.CreateConstIterator(); It; ++It)
		{
			if(It->IsValid())
			{
				PropertyModule.UnregisterCustomPropertyTypeLayout(*It);
			}
		}
	
		PropertyModule.NotifyCustomizationModuleChanged();
	}
}

void FDetailCustomizationsModule::RegisterDevelopmentStatusWarningSuppression(FName ClassName)
{
	SuppressedDevelopmentStatusWarnings.Add(ClassName);
}

void FDetailCustomizationsModule::UnregisterDevelopmentStatusWarningSuppression(FName ClassName)
{
	SuppressedDevelopmentStatusWarnings.Remove(ClassName);
}

bool FDetailCustomizationsModule::IsDevelopmentStatusWarningSuppressed(const UClass* Class) const
{
	const UClass* ClassCursor = Class;
	while(ClassCursor)
	{
		if (SuppressedDevelopmentStatusWarnings.Contains(ClassCursor->GetFName()))
		{
			return true;
		}
		ClassCursor = ClassCursor->GetSuperClass();
	}
	return false;
}

/** Helper that will flag this struct name as supporting the UIMin and UIMax meta data types */
#define REGISTER_UIMINMAX_CUSTOMIZATION( StructName, CallbackFunc ) \
			RangeVisibilityUtils::StructsSupportingRangeVisibility.Add( StructName );		\
			RegisterCustomPropertyTypeLayout( StructName, FOnGetPropertyTypeCustomizationInstance::CreateStatic( CallbackFunc ));

void FDetailCustomizationsModule::RegisterPropertyTypeCustomizations()
{
	RegisterCustomPropertyTypeLayout("SoftObjectPath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoftObjectPathCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SoftClassPath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoftClassPathCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SubScriptStructOf", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSubScriptStructOfCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DataTableRowHandle", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataTableCustomizationLayout::MakeInstance));
	RegisterCustomPropertyTypeLayout("DataTableCategoryHandle", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDataTableCategoryCustomizationLayout::MakeInstance));
	RegisterCustomPropertyTypeLayout("CurveTableRowHandle", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCurveTableCustomizationLayout::MakeInstance));
	RegisterCustomPropertyTypeLayout("LinearColorRamp", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLinearColorRampCustomization::MakeInstance));
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector3f, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector3d, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_IntVector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int32Vector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int64Vector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_UintVector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint32Vector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint64Vector, &FVectorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector4f, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector4d, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector2D, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector2f, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Vector2d, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_IntPoint, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int32Point, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int64Point, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_UintPoint, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint32Point, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint64Point, &FMathStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_IntVector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int32Vector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int64Vector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_UintVector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint32Vector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint64Vector2, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_IntVector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int32Vector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Int64Vector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_UintVector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint32Vector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Uint64Vector4, &FVector4StructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Rotator, &FRotatorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Rotator3f, &FRotatorStructCustomization::MakeInstance);
	REGISTER_UIMINMAX_CUSTOMIZATION(NAME_Rotator3d, &FRotatorStructCustomization::MakeInstance);
	RegisterCustomPropertyTypeLayout(NAME_LinearColor, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FColorStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Color, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FColorStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Matrix, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMatrixStructCustomization<FMatrix::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Matrix44f, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMatrixStructCustomization<FMatrix44f::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Matrix44d, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMatrixStructCustomization<FMatrix44d::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Transform, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTransformStructCustomization<FTransform::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Transform3f, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTransformStructCustomization<FTransform3f::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Transform3d, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTransformStructCustomization<FTransform3d::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Quat, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FQuatStructCustomization<FQuat::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Quat4f, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FQuatStructCustomization<FQuat4f::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout(NAME_Quat4d, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FQuatStructCustomization<FQuat4d::FReal>::MakeInstance));
	RegisterCustomPropertyTypeLayout("SlateColor", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateColorCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("ForceFeedbackAttenuationSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FForceFeedbackAttenuationSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SoundAttenuationSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundAttenuationSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SoundSubmixSendInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundSubmixSendFilterCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SoundSourceBusSendInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundSubmixSendFilterCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("AttenuationSubmixSendSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSoundSubmixSendFilterCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DialogueContext", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDialogueContextStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DialogueWaveParameter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDialogueWaveParameterStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("BodyInstance", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBodyInstanceCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SlateBrush", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateBrushStructCustomization::MakeInstance, true));
	RegisterCustomPropertyTypeLayout("SlateSound", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateSoundStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SlateFontInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSlateFontInfoStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("Guid", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGuidStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("Key", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FKeyStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("FloatRange", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRangeStructCustomization<float>::MakeInstance));
	RegisterCustomPropertyTypeLayout("DoubleRange", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRangeStructCustomization<double>::MakeInstance));
	RegisterCustomPropertyTypeLayout("Int32Range", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRangeStructCustomization<int32>::MakeInstance));
	RegisterCustomPropertyTypeLayout("FloatInterval", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FIntervalStructCustomization<float>::MakeInstance));
	RegisterCustomPropertyTypeLayout("DoubleInterval", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FIntervalStructCustomization<double>::MakeInstance));
	RegisterCustomPropertyTypeLayout("Int32Interval", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FIntervalStructCustomization<int32>::MakeInstance));
	RegisterCustomPropertyTypeLayout("DateTime", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDateTimeStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("Timespan", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTimespanStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("BlackboardEntry", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlackboardEntryDetails::MakeInstance));
	RegisterCustomPropertyTypeLayout("AIDataProviderIntValue", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAIDataProviderValueDetails::MakeInstance));
	RegisterCustomPropertyTypeLayout("AIDataProviderFloatValue", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAIDataProviderValueDetails::MakeInstance));
	RegisterCustomPropertyTypeLayout("AIDataProviderBoolValue", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAIDataProviderValueDetails::MakeInstance));
	RegisterCustomPropertyTypeLayout("RuntimeFloatCurve", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCurveStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("RuntimeVectorCurve", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCurveVectorCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("EnvNamedValue", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEnvQueryParamInstanceCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("NavigationLink", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNavLinkStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("NavigationSegmentLink", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNavLinkStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("NavAgentSelector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNavAgentSelectorCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("Margin", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMarginStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("TextProperty", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTextCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DirectoryPath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDirectoryPathStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("FilePath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFilePathStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("IOSBuildResourceDirectory", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDirectoryPathStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("IOSBuildResourceFilePath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFilePathStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("InputAxisConfigEntry", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInputAxisConfigCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("InputActionKeyMapping", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInputActionMappingCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("InputAxisKeyMapping", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInputAxisMappingCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("RuntimeCurveLinearColor", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCurveColorCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("ParticleSysParam", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FParticleSysParamStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("RawDistributionVector", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRawDistributionVectorStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CollisionProfileName", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCollisionProfileNameCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("AutoReimportDirectoryConfig", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAutoReimportDirectoryCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("AutoReimportWildcard", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAutoReimportWildcardCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DistanceDatum", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDistanceDatumStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("HierarchicalSimplification", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FHierarchicalSimplificationCustomizations::MakeInstance));
	RegisterCustomPropertyTypeLayout("MeshMergingSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMeshMergingSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MeshProxySettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMeshProxySettingsCustomizations::MakeInstance));
	RegisterCustomPropertyTypeLayout("PostProcessSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPostProcessSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("ColorGradingSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FColorGradingSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("AssetImportInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAssetImportDataCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CaptureResolution", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCaptureResolutionCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CompositionGraphCapturePasses", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRenderPassesCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("WeightedBlendable", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWeightedBlendableCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MaterialProxySettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMaterialProxySettingsCustomizations::MakeInstance));
	RegisterCustomPropertyTypeLayout("CompositeReroute", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCompositeRerouteCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CameraFilmbackSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCameraFilmbackSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CameraLensSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCameraLensSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("PlateCropSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCameraCropSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CameraFocusSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCameraFocusSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MovieSceneSequenceLoopCount", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneSequenceLoopCountCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MovieSceneBindingOverrideData", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneBindingOverrideDataCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MovieSceneSectionEvalOptions", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneSectionEvalOptionsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MovieSceneEventParameters", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMovieSceneEventParametersCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("FrameRate", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFrameRateCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("Timecode", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTimecodeDetailsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("LevelSequenceBurnInOptions", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLevelSequenceBurnInOptionsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("LevelSequenceBurnInInitSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLevelSequenceBurnInInitSettingsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CollectionReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCollectionReferenceStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerPlatformInt", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformPropertyCustomization<FPerPlatformInt>::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerPlatformFloat", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformPropertyCustomization<FPerPlatformFloat>::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerPlatformBool", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformPropertyCustomization<FPerPlatformBool>::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerPlatformFrameRate", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformPropertyCustomization<FPerPlatformFrameRate>::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerQualityLevelInt", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerQualityLevelPropertyCustomization<FPerQualityLevelInt>::MakeInstance));
	RegisterCustomPropertyTypeLayout("PerQualityLevelFloat", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerQualityLevelPropertyCustomization<FPerQualityLevelFloat>::MakeInstance));
	RegisterCustomPropertyTypeLayout("SkeletalMeshOptimizationSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSkeletalMeshReductionSettingsDetails::MakeInstance));
	RegisterCustomPropertyTypeLayout("ComponentReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FComponentReferenceCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("SoftComponentReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FComponentReferenceCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("EMaterialShadingModel", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMaterialShadingModelCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("DebugCameraControllerSettingsViewModeIndex", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDebugCameraControllerSettingsViewModeIndexCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("CustomPrimitiveData", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomPrimitiveDataCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("TemplateString", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTemplateStringStructCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("LightingChannels", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLightingChannelsCustomization::MakeInstance));
	RegisterCustomPropertyTypeLayout("MeshDeformer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMeshDeformerCustomization::MakeInstance));
}

#undef REGISTER_UIMINMAX_CUSTOMIZATION

void FDetailCustomizationsModule::RegisterObjectCustomizations()
{
	// Note: By default properties are displayed in script defined order (i.e the order in the header).  These layout detail classes are called in the order seen here which will display properties
	// in the order they are customized.  This is only relevant for inheritance where both a child and a parent have properties that are customized.
	// In the order below, Actor will get a chance to display details first, followed by USceneComponent.

	RegisterCustomClassLayout("Object", FOnGetDetailCustomizationInstance::CreateStatic(&FObjectDetails::MakeInstance));
	RegisterCustomClassLayout("Actor", FOnGetDetailCustomizationInstance::CreateStatic(&FActorDetails::MakeInstance));
	RegisterCustomClassLayout("ActorComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FActorComponentDetails::MakeInstance));
	RegisterCustomClassLayout("SceneComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FSceneComponentDetails::MakeInstance));
	RegisterCustomClassLayout("PrimitiveComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FPrimitiveComponentDetails::MakeInstance));
	RegisterCustomClassLayout("StaticMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FStaticMeshComponentDetails::MakeInstance));
	RegisterCustomClassLayout("InstancedStaticMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FInstancedStaticMeshComponentDetails::MakeInstance));
	RegisterCustomClassLayout("SkeletalMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalMeshComponentDetails::MakeInstance));
	RegisterCustomClassLayout("SkinnedMeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FSkinnedMeshComponentDetails::MakeInstance));
	RegisterCustomClassLayout("SplineComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FSplineComponentDetails::MakeInstance));
	RegisterCustomClassLayout("LightComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FLightComponentDetails::MakeInstance));
	RegisterCustomClassLayout("LocalLightComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FLocalLightComponentDetails::MakeInstance));
	RegisterCustomClassLayout("DirectionalLightComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FDirectionalLightComponentDetails::MakeInstance));
	RegisterCustomClassLayout("StaticMeshActor", FOnGetDetailCustomizationInstance::CreateStatic(&FStaticMeshActorDetails::MakeInstance));
	RegisterCustomClassLayout("MeshComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshComponentDetails::MakeInstance));
	RegisterCustomClassLayout("LevelSequenceActor", FOnGetDetailCustomizationInstance::CreateStatic(&FLevelSequenceActorDetails::MakeInstance));
	RegisterCustomClassLayout("ReflectionCaptureComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FReflectionCaptureDetails::MakeInstance));
	RegisterCustomClassLayout("SceneCaptureComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FSceneCaptureDetails::MakeInstance));
	RegisterCustomClassLayout("SkyLight", FOnGetDetailCustomizationInstance::CreateStatic(&FSkyLightComponentDetails::MakeInstance));
	RegisterCustomClassLayout("Brush", FOnGetDetailCustomizationInstance::CreateStatic(&FBrushDetails::MakeInstance));
	RegisterCustomClassLayout("AmbientSound", FOnGetDetailCustomizationInstance::CreateStatic(&FAmbientSoundDetails::MakeInstance));
	RegisterCustomClassLayout("WorldSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("GeneralProjectSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FGeneralProjectSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("HardwareTargetingSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FHardwareTargetingSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("DocumentationActor", FOnGetDetailCustomizationInstance::CreateStatic(&FDocumentationActorDetails::MakeInstance));

	//@TODO: A2REMOVAL: Rename FSkeletalControlNodeDetails to something more generic
	RegisterCustomClassLayout("K2Node_StructMemberGet", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalControlNodeDetails::MakeInstance));
	RegisterCustomClassLayout("K2Node_StructMemberSet", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalControlNodeDetails::MakeInstance));
	RegisterCustomClassLayout("K2Node_GetClassDefaults", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalControlNodeDetails::MakeInstance));

	RegisterCustomClassLayout("AnimSequence", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimSequenceDetails::MakeInstance));

	RegisterCustomClassLayout("EditorAnimSegment", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimMontageSegmentDetails::MakeInstance));
	RegisterCustomClassLayout("EditorAnimCompositeSegment", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimMontageSegmentDetails::MakeInstance));
	RegisterCustomClassLayout("AnimStateNode", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimStateNodeDetails::MakeInstance));
	RegisterCustomClassLayout("AnimStateAliasNode", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimStateAliasNodeDetails::MakeInstance));
	RegisterCustomClassLayout("AnimStateTransitionNode", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimTransitionNodeDetails::MakeInstance));
	RegisterCustomClassLayout("AnimGraphNode_Trail", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimTrailNodeDetails::MakeInstance));
	RegisterCustomClassLayout("PoseAsset", FOnGetDetailCustomizationInstance::CreateStatic(&FPoseAssetDetails::MakeInstance));
	RegisterCustomClassLayout("AnimationAsset", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimationAssetDetails::MakeInstance));

	RegisterCustomClassLayout("SoundSourceBus", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundSourceBusDetails::MakeInstance));
	RegisterCustomClassLayout("DialogueWave", FOnGetDetailCustomizationInstance::CreateStatic(&FDialogueWaveDetails::MakeInstance));
	RegisterCustomClassLayout("SoundWave", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundWaveDetails::MakeInstance));
	RegisterCustomClassLayout("SoundCue", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundCueDetails::MakeInstance));

	RegisterCustomClassLayout("BodySetup", FOnGetDetailCustomizationInstance::CreateStatic(&FBodySetupDetails::MakeInstance));
	RegisterCustomClassLayout("SkeletalBodySetup", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalBodySetupDetails::MakeInstance));
	RegisterCustomClassLayout("PhysicsConstraintTemplate", FOnGetDetailCustomizationInstance::CreateStatic(&FPhysicsConstraintComponentDetails::MakeInstance));
	RegisterCustomClassLayout("PhysicsConstraintComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FPhysicsConstraintComponentDetails::MakeInstance));
	RegisterCustomClassLayout("CollisionProfile", FOnGetDetailCustomizationInstance::CreateStatic(&FCollisionProfileDetails::MakeInstance));
	RegisterCustomClassLayout("PhysicsSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FPhysicsSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("AudioSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FAudioSettingsDetails::MakeInstance));

	RegisterCustomClassLayout("ParticleModuleRequired", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleRequiredDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleSubUV", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleSubUVDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleAccelerationDrag", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleAccelerationDragDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleAcceleration", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleAccelerationDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleAccelerationDragScaleOverLife", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleAccelerationDragScaleOverLifeDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleCollisionGPU", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleCollisionGPUDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleOrbit", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleOrbitDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleSizeMultiplyLife", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleSizeMultiplyLifeDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleSizeScale", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleSizeScaleDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleVectorFieldScale", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleVectorFieldScaleDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleModuleVectorFieldScaleOverLife", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleModuleVectorFieldScaleOverLifeDetails::MakeInstance));

	RegisterCustomClassLayout("CameraComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FCameraDetails::MakeInstance));
	RegisterCustomClassLayout("DeviceProfile", FOnGetDetailCustomizationInstance::CreateStatic(&FDeviceProfileDetails::MakeInstance));
	RegisterCustomClassLayout("InputSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FInputSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("WindowsTargetSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FWindowsTargetSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("LinuxTargetSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FLinuxTargetSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("MoviePlayerSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FMoviePlayerSettingsDetails::MakeInstance));

	RegisterCustomClassLayout("SourceCodeAccessSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FSourceCodeAccessSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("ParticleSystemComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FParticleSystemComponentDetails::MakeInstance));

	RegisterCustomClassLayout("FbxImportUI", FOnGetDetailCustomizationInstance::CreateStatic(&FFbxImportUIDetails::MakeInstance));
	RegisterCustomClassLayout("FbxSceneImportData", FOnGetDetailCustomizationInstance::CreateStatic(&FFbxSceneImportDataDetails::MakeInstance));

	RegisterCustomClassLayout("ConfigHierarchyPropertyView", FOnGetDetailCustomizationInstance::CreateStatic(&FConfigPropertyHelperDetails::MakeInstance));

	RegisterCustomClassLayout("MovieSceneCapture", FOnGetDetailCustomizationInstance::CreateStatic(&FMovieSceneCaptureCustomization::MakeInstance));

	RegisterCustomClassLayout("AnalyticsPrivacySettings", FOnGetDetailCustomizationInstance::CreateStatic(&FImportantToggleSettingCustomization::MakeInstance));
	RegisterCustomClassLayout("CrashReportsPrivacySettings", FOnGetDetailCustomizationInstance::CreateStatic(&FImportantToggleSettingCustomization::MakeInstance));

	RegisterCustomClassLayout("AssetViewerSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FAssetViewerSettingsCustomization::MakeInstance));

	RegisterCustomClassLayout("MeshMergingSettingsObject", FOnGetDetailCustomizationInstance::CreateStatic(&FMeshMergingSettingsObjectCustomization::MakeInstance));

	RegisterCustomClassLayout("MaterialExpressionGetMaterialAttributes", FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialAttributePropertyDetails::MakeInstance));
	RegisterCustomClassLayout("MaterialExpressionSetMaterialAttributes", FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialAttributePropertyDetails::MakeInstance));
	RegisterCustomClassLayout("MaterialExpressionTextureBase", FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialExpressionTextureBaseDetails::MakeInstance));
	RegisterCustomClassLayout("TextureCubeArray", FOnGetDetailCustomizationInstance::CreateStatic(&FTextureCubeArrayDetails::MakeInstance));
	RegisterCustomClassLayout("MaterialInstanceDynamic", FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialInstanceDynamicDetails::MakeInstance));
	RegisterCustomClassLayout("MaterialParameterCollection", FOnGetDetailCustomizationInstance::CreateStatic(&FMaterialParameterCollectionDetails::MakeInstance));
	RegisterCustomClassLayout("SkeletalMeshLODSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalMeshLODSettingsDetails::MakeInstance));
	RegisterCustomClassLayout("SkeletalMesh", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletalMeshDetails::MakeInstance));
	RegisterCustomClassLayout("Skeleton", FOnGetDetailCustomizationInstance::CreateStatic(&FSkeletonDetails::MakeInstance));

	RegisterCustomClassLayout("MotionControllerComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FMotionControllerDetails::MakeInstance));

	RegisterCustomClassLayout("BoundsCopyComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FBoundsCopyComponentDetailsCustomization::MakeInstance));

	RegisterCustomClassLayout("SoundNodeDistanceCrossFade", FOnGetDetailCustomizationInstance::CreateStatic(&FCrossFadeCustomization::MakeInstance));
	
	RegisterCustomClassLayout("DecalComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FDecalComponentDetails::MakeInstance));
	
	RegisterCustomClassLayout("StereoLayerComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FStereoLayerComponentCustomization::MakeInstance));
}

#define LOCTEXT_NAMESPACE "DetailsSections"

void FDetailCustomizationsModule::RegisterSectionMappings()
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	// Object
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Object", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Default"); // default category for BP instance editable variables
		}
	}

	// Actor
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Actor", "Actor", LOCTEXT("Actor", "Actor"));
			Section->AddCategory("Actor");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Actor", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Cooking");
			Section->AddCategory("Input");
			Section->AddCategory("Replication");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Actor", "Streaming", LOCTEXT("Streaming", "Streaming"));
			Section->AddCategory("World Partition");
			Section->AddCategory("Data Layers");
			Section->AddCategory("HLOD");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Actor", "Audio", LOCTEXT("Audio", "Audio"));
			Section->AddCategory("Audio");
			Section->AddCategory("Sound");
			Section->AddCategory("Sounds");
			Section->AddCategory("MetaSound");
			Section->AddCategory("MetaSounds");
			Section->AddCategory("SFX");
			Section->AddCategory("Music");
			Section->AddCategory("VO");
		}
	}

	// Pawn
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Pawn", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Pawn");
			Section->AddCategory("Camera");
		}
	}

	// ActorComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("ActorComponent", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Asset User Data");
			Section->AddCategory("Cooking");
			Section->AddCategory("Tags");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("ActorComponent", "Audio", LOCTEXT("Audio", "Audio"));
			Section->AddCategory("Audio");
			Section->AddCategory("Sound");
			Section->AddCategory("Sounds");
			Section->AddCategory("MetaSound");
			Section->AddCategory("MetaSounds");
			Section->AddCategory("SFX");
			Section->AddCategory("Music");
			Section->AddCategory("VO");
		}
	}

	// SceneComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SceneComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Transform");
			Section->AddCategory("TransformCommon");
			Section->AddCategory("Mobility");
		}
	}

	// PrimitiveComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PrimitiveComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Materials");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PrimitiveComponent", "LOD", LOCTEXT("LOD", "LOD"));
			Section->AddCategory("HLOD");
			Section->AddCategory("LOD");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PrimitiveComponent", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Navigation");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PrimitiveComponent", "Physics", LOCTEXT("Physics", "Physics"));
			Section->AddCategory("Collision");
			Section->AddCategory("Physics");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PrimitiveComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Lighting");
			Section->AddCategory("Lightmass");
			Section->AddCategory("Materials");
			Section->AddCategory("Mobile");
			Section->AddCategory("Ray Tracing");
			Section->AddCategory("Path Tracing");
			Section->AddCategory("Rendering");
			Section->AddCategory("Texture Streaming");
			Section->AddCategory("Virtual Texture");
		}
	}

	// MeshComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("MeshComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Mesh");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("MeshComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Material Parameters");
		}
	}

	// StaticMeshComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("StaticMeshComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Static Mesh");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("StaticMeshComponent", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Navigation");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("MeshComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Mesh Painting");
		}
	}

	// LightComponentBase
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("LightComponentBase", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Light");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("LightComponentBase", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Light");
			Section->AddCategory("Light Function");
			Section->AddCategory("Light Profiles");
		}
	}

	// LightComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("LightComponent", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Performance");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("LightComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Distance Field Shadows");
			Section->AddCategory("Light Shafts");
		}
	}

	// DirectionalLightComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("DirectionalLightComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Atmosphere and Cloud");
			Section->AddCategory("Cascaded Shadow Maps");
		}
	}

	// SkyLightComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkyLightComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Sky Light");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkyLightComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Atmosphere and Cloud");
			Section->AddCategory("Distance Field Ambient Occlusion");
		}
	}

	// PlayerStart
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PlayerStart", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Object");
		}
	}

	// ShapeComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("ShapeComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Shape");
		}
	}

	// BillboardComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("BillboardComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Sprite");
		}
	}

	// SkinnedMeshComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkinnedMeshComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Mesh");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkinnedMeshComponent", "Animation", LOCTEXT("Animation", "Animation"));
			Section->AddCategory("Skeletal Mesh");
			Section->AddCategory("Deformer");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkinnedMeshComponent", "Misc", LOCTEXT("Misc", "Misc"));
			Section->AddCategory("Optimization");
		}
	}

	// SkeletalMeshComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkeletalMeshComponent", "Animation", LOCTEXT("Animation", "Animation"));
			Section->AddCategory("Animation");
			Section->AddCategory("Animation Rig");
			Section->AddCategory("Leader Pose Component");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkeletalMeshComponent", "Physics", LOCTEXT("Physics", "Physics"));
			Section->AddCategory("Clothing");
		}
	}

	// SkeletalMesh
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("SkeletalMesh", "Animation", LOCTEXT("Animation", "Animation"));
		Section->AddCategory("Skin Weights");
	}

	// AnimInstance
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("AnimInstance", "Movement", LOCTEXT("Movement", "Movement"));
			Section->AddCategory("Root Motion");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("AnimInstance", "Physics", LOCTEXT("Physics", "Physics"));
			Section->AddCategory("Root Motion");
		}
	}

	// Character
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Character", "Movement", LOCTEXT("Movement", "Movement"));
			Section->AddCategory("Camera");
			Section->AddCategory("Character");
			Section->AddCategory("Pawn");
		}
	}

	// CharacterMovementComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CharacterMovementComponent", "Movement", LOCTEXT("Movement", "Movement"));
			Section->AddCategory("Character Movement");
			Section->AddCategory("Character Movement (General Settings)");
			Section->AddCategory("Character Movement (Networking)");
			Section->AddCategory("Character Movement (Rotation Settings)");
			Section->AddCategory("Character Movement: Avoidance");
			Section->AddCategory("Character Movement: Custom Movement");
			Section->AddCategory("Character Movement: Flying");
			Section->AddCategory("Character Movement: Jumping / Falling");
			Section->AddCategory("Character Movement: MovementMode");
			Section->AddCategory("Character Movement: Physics Interaction");
			Section->AddCategory("Character Movement: Swimming");
			Section->AddCategory("Character Movement: Walking");
			Section->AddCategory("Root Motion");
		}

		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CharacterMovementComponent", "Physics", LOCTEXT("Physics", "Physics"));
			Section->AddCategory("Root Motion");
		}
	}

	// CapsuleComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CapsuleComponent", "Actor", LOCTEXT("Actor", "Actor"));
			Section->AddCategory("Shape");
		}
	}

	// Brush
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Brush", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Brush Settings");
		}
	}

	// WorldPartition
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("WorldPartition", "Streaming", LOCTEXT("Streaming", "Streaming"));
			Section->AddCategory("WorldPartition");
		}
	}

	// WorldSettings
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("WorldSettings", "Streaming", LOCTEXT("Streaming", "Streaming"));
			Section->AddCategory("Foliage");
		}
	}

	// Geometry Collections
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("GeometryCollectionComponent", "GC", LOCTEXT("GC", "GC"));
			Section->AddCategory("ChaosPhysics");
		}
	}

	// Post Process Volume
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "General", LOCTEXT("General", "General"));
			Section->AddCategory("PostProcessVolumeSettings");
			Section->RemoveCategory("Brush Settings");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Rendering Features");
			Section->RemoveCategory("Path Tracing");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "ColorGrading", LOCTEXT("ColorGrading", "Color Grading"));
			Section->AddCategory("Color Grading");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "Film", LOCTEXT("Film", "Film"));
			Section->AddCategory("Film");
			Section->AddCategory("Film Grain");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "GlobalIllumination", LOCTEXT("GlobalIllumination", "Global Illumination"));
			Section->AddCategory("Global Illumination");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "Lens", LOCTEXT("Lens", "Lens"));
			Section->AddCategory("Lens");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "PathTracing", LOCTEXT("PathTracing", "Path Tracing"));
			Section->AddCategory("Path Tracing");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("PostProcessVolume", "Reflections", LOCTEXT("Reflections", "Reflections"));
			Section->AddCategory("Reflections");
		}
	}

	// RuntimeVirtualTextureComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("RuntimeVirtualTextureComponent", "Rendering", LOCTEXT("Rendering", "Rendering"));
			Section->AddCategory("Rendering");
			Section->AddCategory("Runtime Virtual Texture");
			Section->AddCategory("Streaming Virtual Texture");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("RuntimeVirtualTextureComponent", "General", LOCTEXT("General", "General"));
			Section->AddCategory("Volume Bounds");
		}
	}

	// CameraComponent
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CameraComponent", "CameraOptions", LOCTEXT("CameraOptions", "Camera Options"));
			Section->AddCategory("Camera Options");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CameraComponent", "Camera", LOCTEXT("Camera", "Camera"));
			Section->AddCategory("Camera Settings");
			Section->AddCategory("Camera");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CameraComponent", "PostProcess", LOCTEXT("PostProcess", "Post Process"));
			Section->AddCategory("Post Process");
		}
	}

	// Cinematic Camera
	{
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CineCameraActor", "CurrentCameraSettings", LOCTEXT("CurrentCameraSettings", "Current Camera Settings"));
			Section->AddCategory("Current Camera Settings");
		}
		{
			TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("CineCameraComponent", "CurrentCameraSettings", LOCTEXT("CurrentCameraSettings", "Current Camera Settings"));
			Section->AddCategory("Current Camera Settings");
		}
	}
}

#undef LOCTEXT_NAMESPACE

void FDetailCustomizationsModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate )
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout( ClassName, DetailLayoutDelegate );
}


void FDetailCustomizationsModule::RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate)
{
	check(PropertyTypeName != NAME_None);

	RegisteredPropertyTypes.Add(PropertyTypeName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomPropertyTypeLayout(PropertyTypeName, PropertyTypeLayoutDelegate);
}
