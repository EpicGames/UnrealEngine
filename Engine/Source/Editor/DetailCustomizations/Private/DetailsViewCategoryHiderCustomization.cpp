// Copyright Epic Games, Inc. All Rights Reserved.
#include "DetailsViewCategoryHiderCustomization.h"

#include "DetailLayoutBuilder.h"

TSharedRef<IDetailCustomization> FDetailsViewCategoryHiderCustomization::MakeInstance(TArray<FName>&& InCategoriesToHide)
{
	return MakeShared<FDetailsViewCategoryHiderCustomization>(MoveTemp(InCategoriesToHide));
}

TSharedRef<IDetailCustomization> FDetailsViewCategoryHiderCustomization::MakeInstance(const TArrayView<FName>& InCategoriesToHide)
{
	return MakeShared<FDetailsViewCategoryHiderCustomization>(InCategoriesToHide);
}

FDetailsViewCategoryHiderCustomization::FDetailsViewCategoryHiderCustomization(TArray<FName>&& InCategoriesToHide) 
	: CategoriesToHide(MoveTemp(InCategoriesToHide))
{
}

FDetailsViewCategoryHiderCustomization::FDetailsViewCategoryHiderCustomization(const TArrayView<FName>& InCategoriesToHide)
	: CategoriesToHide(InCategoriesToHide)
{
}

void FDetailsViewCategoryHiderCustomization::CustomizeDetails(class IDetailLayoutBuilder& DetailLayoutBuilder)
{
	for (const FName& CategoryName : CategoriesToHide)
	{
		DetailLayoutBuilder.HideCategory(CategoryName);
	}
}
