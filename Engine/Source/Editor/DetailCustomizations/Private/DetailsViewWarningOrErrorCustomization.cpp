// Copyright Epic Games, Inc. All Rights Reserved.
#include "DetailsViewWarningOrErrorCustomization.h"

#include "DetailWidgetRow.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IDetailCustomization> FDetailsViewWarningOrErrorCustomization::MakeInstance(
	const FName& InCategoryForInsertion,
	const FName& InRowTag,
	const FText& InWarningOrErrorLabel,
	const EMessageStyle InMessageStyle/* = EMessageStyle::Warning*/,
	ECategoryPriority::Type InCategoryPriority/* = ECategoryPriority::Uncommon*/)
{
	return MakeShared<FDetailsViewWarningOrErrorCustomization>(InCategoryForInsertion, InRowTag, InWarningOrErrorLabel, InMessageStyle, InCategoryPriority);
}

FDetailsViewWarningOrErrorCustomization::FDetailsViewWarningOrErrorCustomization(
	const FName& InCategoryForInsertion, 
	const FName& InRowTag, 
	const FText& InWarningOrErrorLabel, 
	const EMessageStyle InMessageStyle, 
	const ECategoryPriority::Type InCategoryPriority) :
		CategoryForInsertion(InCategoryForInsertion),
		RowTag(InRowTag),
		WarningOrErrorLabel(InWarningOrErrorLabel),
		MessageStyle(InMessageStyle),
		CategoryPriority(InCategoryPriority)
{
}

void FDetailsViewWarningOrErrorCustomization::CustomizeDetails(class IDetailLayoutBuilder& DetailLayoutBuilder)
{
	IDetailCategoryBuilder& TargetCategory = DetailLayoutBuilder.EditCategory(CategoryForInsertion, FText::GetEmpty(), CategoryPriority);

	FDetailWidgetRow& WarningOrErrorRow = TargetCategory
		.AddCustomRow(FText::GetEmpty(), false)
		.RowTag(RowTag)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SWarningOrErrorBox)
						.MessageStyle(MessageStyle)
						.Message(WarningOrErrorLabel)
				]
		];
}
