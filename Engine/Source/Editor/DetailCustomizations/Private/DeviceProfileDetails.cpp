// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceProfileDetails.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "IDetailGroup.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/EnumRange.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/Less.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "TextureLODSettingsDetails.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "DeviceProfileDetails"


////////////////////////////////////////////////
// FDeviceProfileDetails


void FDeviceProfileDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Hide all the properties apart from the Console Variables.
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("DeviceSettings");

	TSharedPtr<IPropertyHandle> DeviceTypeHandle = DetailBuilder.GetProperty("DeviceType");
	DetailBuilder.HideProperty(DeviceTypeHandle);

	TSharedPtr<IPropertyHandle> MeshLODSettingsHandle = DetailBuilder.GetProperty("MeshLODSettings");
	DetailBuilder.HideProperty(MeshLODSettingsHandle);

	// Setup the parent profile panel
	ParentProfileDetails = MakeShareable(new FDeviceProfileParentPropertyDetails(&DetailBuilder));
	ParentProfileDetails->CreateParentPropertyView();

	// Setup the console variable editor
	ConsoleVariablesDetails = MakeShareable(new FDeviceProfileConsoleVariablesPropertyDetails(&DetailBuilder));
	ConsoleVariablesDetails->CreateConsoleVariablesPropertyView();

	TextureLODSettingsDetails = MakeShareable(new FDeviceProfileTextureLODSettingsDetails(&DetailBuilder));
	TextureLODSettingsDetails->CreateTextureLODSettingsPropertyView();
}


////////////////////////////////////////////////
// DeviceProfilePropertyConstants


/** Property layout constants, we will use this for consistent spacing across the details view */
namespace DeviceProfilePropertyConstants
{
	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
	const FMargin CVarSelectionMenuPadding(10.0f, 2.0f);
}


////////////////////////////////////////////////
// DeviceProfileCVarFormatHelper


/** Some helper fucntions to assist us with displaying Console Variables from the CVars property */
namespace DeviceProfileCVarFormatHelper
{
	/** The available Console Variable Categories a CVar will be listed under. */
	enum ECVarGroup
	{
		CVG_Uncategorized = 0,
		CVG_Rendering,
		CVG_Physics,
		CVG_Network,
		CVG_Console,
		CVG_Compatibility,
		CVG_UserInterface,
		CVG_ScalabilityGroups,

		Max_CVarCategories,
	};


	/**
	 *	Convert the CVar group enum enum its display name.
	 *
	 *	@param CatEnum - The ECVarGroup index
	 *
	 *	@return The display name of the group.
	 */
	FText CategoryTextFromEnum(ECVarGroup CatEnum)
	{
		switch(CatEnum)
		{
		case CVG_Rendering:
			return LOCTEXT("RenderingCVarGroupTitle", "Rendering");
		case CVG_Physics:
			return LOCTEXT("PhysicsCVarGroupTitle", "Physics");
		case CVG_Network:
			return LOCTEXT("NetworkCVarGroupTitle", "Network");
		case CVG_Console:
			return LOCTEXT("ConsoleCVarGroupTitle", "Console");
		case CVG_Compatibility:
			return LOCTEXT("CompatibilityCVarGroupTitle", "Compatibility");
		case CVG_UserInterface:
			return LOCTEXT("UICVarGroupTitle", "User Interface");
		case CVG_ScalabilityGroups:
			return LOCTEXT("ScalabilityGroupCVarGroupTitle", "Scalability Group");
		default:
			break;
		}
		return LOCTEXT("UncategorizedCVarGroupTitle", "Uncategorized");
	}


	/**
	 *	Convert the CVar group enum enum its CVar prefix.
	 *
	 *	@param CatEnum - The ECVarGroup index
	 *
	 *	@return The prefix of the group.
	 */
	FString CategoryPrefixFromEnum(ECVarGroup CatEnum)
	{
		switch (CatEnum)
		{
		case CVG_Rendering:
			return TEXT("r");
		case CVG_Physics:
			return TEXT("p");
		case CVG_Network:
			return TEXT("net");
		case CVG_Console:
			return TEXT("con");
		case CVG_Compatibility:
			return TEXT("compat");
		case CVG_UserInterface:
			return TEXT("ui");
		case CVG_ScalabilityGroups:
			return TEXT("sg");
		default:
			break;
		}
		return FString();
	}


	/**
	 *	Convert the CVar prefix to the CVar group enum entry.
	 *
	 *	@param InPrefix - The Prefix of the Console Variable
	 *
	 *	@return The enum entry of the group.
	 */
	ECVarGroup CategoryEnumFromPrefix(const FString& InPrefix)
	{
		if (InPrefix == TEXT("r") || InPrefix == TEXT("r."))
		{
			return CVG_Rendering;
		}
		if (InPrefix == TEXT("p") || InPrefix == TEXT("p."))
		{
			return CVG_Physics;
		}
		if (InPrefix == TEXT("net") || InPrefix == TEXT("net."))
		{
			return CVG_Network;
		}
		if (InPrefix == TEXT("con") || InPrefix == TEXT("con."))
		{
			return CVG_Console;
		}
		if (InPrefix == TEXT("compat") || InPrefix == TEXT("compat."))
		{
			return CVG_Compatibility;
		}
		if (InPrefix == TEXT("ui") || InPrefix == TEXT("ui."))
		{
			return CVG_UserInterface;
		}
		if (InPrefix == TEXT("sg") || InPrefix == TEXT("sg."))
		{
			return CVG_ScalabilityGroups;
		}
		return CVG_Uncategorized;
	}
};

ENUM_RANGE_BY_COUNT(DeviceProfileCVarFormatHelper::ECVarGroup, DeviceProfileCVarFormatHelper::Max_CVarCategories);

////////////////////////////////////////////////
// FConsoleVariablesAvailableVisitor


/**
 * Console variable visitor which collects our desired information from the console manager iterator
 */
class FConsoleVariablesAvailableVisitor
{
public:
	// @param Name must not be 0
	// @param CVar must not be 0
	static void OnConsoleVariable(const TCHAR *Name, IConsoleObject* CVar, TArray<TSharedPtr<FString>>* Sink)
	{
		if(CVar->AsVariable())
		{
			Sink->Add(MakeShareable(new FString(Name)));
		}

	}
};


////////////////////////////////////////////////
// SCVarSelectionPanel


/**
 * Slate Widget to display all available CVars for a given Console Variable group.
 */
class SCVarSelectionPanel : public SCompoundWidget
{
private:

	/** Delegate type to notify listeners that a CVar was selected for add */
	DECLARE_DELEGATE_OneParam(FOnCVarAddedDelegate, const FString&);

public:

	SLATE_BEGIN_ARGS(SCVarSelectionPanel) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnCVarAddedDelegate,OnCVarSelected)
	SLATE_END_ARGS()


	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FString& CVarPrefix);

	/**
	 * Handle the cvar selection from this panel
	 *
	 * @param CVar - The selected CVar
	 */
	FReply HandleCVarSelected(const TSharedPtr<FString> CVar);

	/** 
	 * Row generation widget for the list of available CVars for add
	 *
	 * @param InItem		- The CVar, as a string, we are creating a row for
	 * @param OwnerTable	- The table widget that this row will be added to
	 */
	TSharedRef<ITableRow> GenerateCVarItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Called by Slate when the filter box changes text.
	 *
	 * @param InFilterText - The substring which we should use to filter CVars
	 */
	void OnFilterTextChanged(const FText& InFilterText);


private:

	/** Handle to the list view of selectable console variables */
	TSharedPtr<SListView<TSharedPtr<FString>>> CVarListView;

	/** Text entry to filter console variable strings */
	TSharedPtr<SSearchBox> CVarFilterBox;

	/** The collection of CVars for a groups selection panel*/
	TArray<TSharedPtr<FString>> CVarsToDisplay;

	/** The collection of CVars for a groups selection panel*/
	TArray<TSharedPtr<FString>> AllAvailableCVars;

	/** Delegate used to notify listeners that a CVar was selected for add */
	FOnCVarAddedDelegate OnCVarSelected;


	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;
};


void SCVarSelectionPanel::Construct(const FArguments& InArgs, const FString& CVarPrefix)
{
	OnCVarSelected = InArgs._OnCVarSelected;

	TArray<TSharedPtr<FString>> UnprocessedCVars;

	IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(
		FConsoleObjectVisitor::CreateStatic(
		&FConsoleVariablesAvailableVisitor::OnConsoleVariable,
		&UnprocessedCVars), *CVarPrefix);

	if (DeviceProfileCVarFormatHelper::CategoryEnumFromPrefix(CVarPrefix) == DeviceProfileCVarFormatHelper::CVG_Uncategorized)
	{
		// Make a list of existing prefixes
		TSet<FString> CategoryPrefixes;
		for (DeviceProfileCVarFormatHelper::ECVarGroup PrefixEnum : TEnumRange<DeviceProfileCVarFormatHelper::ECVarGroup>())
		{
			FString Prefix = DeviceProfileCVarFormatHelper::CategoryPrefixFromEnum(PrefixEnum);
			if (Prefix.Len() > 0)
			{
				CategoryPrefixes.Add(Prefix + TEXT("."));
			}
		}
		
		// Add all cvars that *don't* match one of the other groups
		for (const TSharedPtr<FString>& TestCVarPtr : UnprocessedCVars)
		{
			const FString& TestCVar = *TestCVarPtr.Get();

			bool bBelongsInUncategorized = false;

			// Figure out if the prefix on this variable belongs to any other category or if it's missing a prefix
			int32 FirstPeriodIndex = INDEX_NONE;
			TestCVar.FindChar(TEXT('.'), /*out*/ FirstPeriodIndex);

			if (FirstPeriodIndex != INDEX_NONE)
			{
				const FString TestPrefix(TestCVar.Left(FirstPeriodIndex+1));
				bBelongsInUncategorized = !CategoryPrefixes.Contains(TestPrefix);
			}
			else
			{
				// No period means no prefix, so it couldn't be in any other category
				bBelongsInUncategorized = true;
			}

			if (bBelongsInUncategorized)			
			{
				AllAvailableCVars.Add(TestCVarPtr);
			}
		}
	}
	else
	{
		AllAvailableCVars = UnprocessedCVars;
	}

	// Sort the list
	AllAvailableCVars.Sort([](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B) { return *A < *B; });

	// Duplicate the list	
	CVarsToDisplay = AllAvailableCVars;

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(300.f)
		.HeightOverride(512.f)
		.Content()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(4.0f))
			.AutoHeight()
			[
				SAssignNew(CVarFilterBox,SSearchBox)
				.OnTextChanged(this, &SCVarSelectionPanel::OnFilterTextChanged)
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(CVarListView, SListView<TSharedPtr<FString>>)
				.ListItemsSource(&CVarsToDisplay)
				.OnGenerateRow(this, &SCVarSelectionPanel::GenerateCVarItemRow)			
			]
		]
	];
}


FReply SCVarSelectionPanel::HandleCVarSelected(const TSharedPtr<FString> CVar)
{
	OnCVarSelected.ExecuteIfBound(*CVar.Get());

	return FReply::Handled();
}


TSharedRef<ITableRow> SCVarSelectionPanel::GenerateCVarItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText ComposedTooltip = LOCTEXT("CVarSelectionMenuTooltip", "Select a Console Variable to add to the device profile");

	if (IConsoleVariable* Var = IConsoleManager::Get().FindConsoleVariable(**InItem))
	{
		ComposedTooltip = FText::Format(LOCTEXT("CVarSelectionMenuTooltipWithHelp", "{0}\n\n{1}"), ComposedTooltip, FText::FromString(Var->GetHelp()));
	}

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SCVarSelectionPanel::HandleCVarSelected, InItem)
			.ContentPadding(DeviceProfilePropertyConstants::CVarSelectionMenuPadding)
			.ToolTipText(ComposedTooltip)
			[
				SNew(STextBlock)
				.Text(FText::FromString(*InItem))
			]
		];
}


void SCVarSelectionPanel::OnFilterTextChanged(const FText& InFilterText)
{
	const FString CurrentFilterText = InFilterText.ToString();

	// Recreate the list of available CVars using the filter
	CVarsToDisplay.Empty();
	for(auto& NextCVar : AllAvailableCVars)
	{
		if(CurrentFilterText.Len() == 0 || NextCVar->Contains(CurrentFilterText))
		{
			CVarsToDisplay.Add(NextCVar);
		}
	}

	CVarListView->RequestListRefresh();
}


////////////////////////////////////////////////
// FDeviceProfileParentPropertyDetails


FDeviceProfileParentPropertyDetails::FDeviceProfileParentPropertyDetails(IDetailLayoutBuilder* InDetailBuilder)
	: DetailBuilder(InDetailBuilder)
	, ActiveDeviceProfile(nullptr)
{
	ParentPropertyNameHandle = DetailBuilder->GetProperty("BaseProfileName");

	TArray<UObject*> OuterObjects;
	ParentPropertyNameHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		ActiveDeviceProfile = CastChecked<UDeviceProfile>(OuterObjects[0]);
	}
}


void FDeviceProfileParentPropertyDetails::CreateParentPropertyView()
{
	UDeviceProfile* ParentProfile = ActiveDeviceProfile ? Cast<UDeviceProfile>(ActiveDeviceProfile->Parent) : nullptr;
	while(ParentProfile != nullptr)
	{
		ParentProfile->OnCVarsUpdated().BindSP(this, &FDeviceProfileParentPropertyDetails::OnParentPropertyChanged);
		ParentProfile = Cast<UDeviceProfile>(ParentProfile->Parent);
	}

	DetailBuilder->HideProperty(ParentPropertyNameHandle);

	FString CurrentParentName;
	ParentPropertyNameHandle->GetValue(CurrentParentName);

	IDetailCategoryBuilder& ParentDetailCategory = DetailBuilder->EditCategory("ParentDeviceProfile");
	IDetailGroup& ParentNameGroup = ParentDetailCategory.AddGroup(TEXT("ParentProfileName"), LOCTEXT("ParentProfileOptionsGroupTitle", "Parent Profile Name"));

	ParentNameGroup.HeaderRow()
	[
		SNew(SBox)
		.Padding(DeviceProfilePropertyConstants::PropertyPadding)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DeviceProfileSelectParentPropertyTitle", "Selected Parent:"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];

	AvailableParentProfiles.Add(MakeShareable(new FString(LOCTEXT("NoParentSelection", "None").ToString())));
	if(ActiveDeviceProfile != nullptr)
	{
		TArray<UDeviceProfile*> AllPossibleParents;
		UDeviceProfileManager::Get().GetAllPossibleParentProfiles(ActiveDeviceProfile, AllPossibleParents);

		for(auto& NextProfile : AllPossibleParents)
		{
			AvailableParentProfiles.Add(MakeShareable(new FString(NextProfile->GetName())));
		}
	}


	FText ParentNameText = CurrentParentName.Len() > 0 ? FText::FromString(CurrentParentName) : LOCTEXT("NoParentSelection", "None");
	ParentNameGroup.AddWidgetRow()
	[
		SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&AvailableParentProfiles)
		.OnGenerateWidget(this, &FDeviceProfileParentPropertyDetails::HandleDeviceProfileParentComboBoxGenarateWidget)
		.OnSelectionChanged(this, &FDeviceProfileParentPropertyDetails::HandleDeviceProfileParentSelectionChanged)
		.Content()
		[
			SNew(STextBlock)
			.Text(ParentNameText)
		]
	];
	
	// If we have a parent, display Console Variable information
	if(ActiveDeviceProfile != nullptr && ActiveDeviceProfile->BaseProfileName.Len() > 0)
	{
		// Get a list of the current profiles CVar names to use as a filter when showing parent CVars
		TArray<FString> DeviceProfileCVarNames;
		for (auto& NextActiveProfileCVar : ActiveDeviceProfile->CVars)
		{
			FString CVarName;
			FString CVarValue;
			NextActiveProfileCVar.Split(TEXT("="), &CVarName, &CVarValue);
			
			DeviceProfileCVarNames.Add(CVarName);
		}

		TMap<FString, FString> ParentCVarInformation;
		ActiveDeviceProfile->GatherParentCVarInformationRecursively(ParentCVarInformation);

		IDetailGroup* ParentCVarsGroup = nullptr;

		ParentCVarInformation.KeySort(TLess<>());
		for(auto& ParentCVar : ParentCVarInformation)
		{
			FString ParentCVarName;
			FString ParentCVarValue;
			ParentCVar.Value.Split(TEXT("="), &ParentCVarName, &ParentCVarValue);

			// Do not display Parent CVars if the child has them overridden
			bool bDisplayCVar = DeviceProfileCVarNames.Find(ParentCVarName) == INDEX_NONE;
			if(bDisplayCVar)
			{
				if(ParentCVarsGroup == nullptr)
				{
					ParentCVarsGroup = &ParentDetailCategory.AddGroup(TEXT("ParentProfileOptions"), LOCTEXT("ParentConsoleOptionsGroupTitle", "Parent Console Variables"));

					ParentCVarsGroup->HeaderRow()
					[
						SNew(SBox)
						.Padding(DeviceProfilePropertyConstants::PropertyPadding)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DeviceProfileParentCVarsTitle", "Inherited Console Variables"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					];
				}

				ParentCVarsGroup->AddWidgetRow()
				.IsEnabled(true)
				.Visibility(EVisibility::Visible)
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(ParentCVarName))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(ParentCVarValue))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
			}
		}
	}
}


void FDeviceProfileParentPropertyDetails::HandleDeviceProfileParentSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	ActiveDeviceProfile->BaseProfileName = *NewSelection.Get() == LOCTEXT("NoParentSelection", "None").ToString() ? TEXT("") : *NewSelection.Get();
	// -> Refresh the UI of the Details view to display the parent selection
	DetailBuilder->ForceRefreshDetails();

}


TSharedRef<SWidget> FDeviceProfileParentPropertyDetails::HandleDeviceProfileParentComboBoxGenarateWidget(TSharedPtr<FString> InItem)
{
	return SNew(SBox)
	.Padding(DeviceProfilePropertyConstants::CVarSelectionMenuPadding)
	[
		SNew(STextBlock)
		.Text(FText::FromString(*InItem))
	];
}


void FDeviceProfileParentPropertyDetails::OnParentPropertyChanged()
{
	DetailBuilder->ForceRefreshDetails();
}


////////////////////////////////////////////////
// FDeviceProfileConsoleVariablesPropertyDetails


FDeviceProfileConsoleVariablesPropertyDetails::FDeviceProfileConsoleVariablesPropertyDetails(IDetailLayoutBuilder* InDetailBuilder)
: DetailBuilder(InDetailBuilder)
{
	CVarsHandle = DetailBuilder->GetProperty("CVars");
}


void FDeviceProfileConsoleVariablesPropertyDetails::CreateConsoleVariablesPropertyView()
{
	FSimpleDelegate OnCVarPropertyChangedDelegate = FSimpleDelegate::CreateSP(this, &FDeviceProfileConsoleVariablesPropertyDetails::OnCVarPropertyChanged);
	CVarsHandle->SetOnPropertyValueChanged(OnCVarPropertyChangedDelegate);

	DetailBuilder->HideProperty(CVarsHandle);
	TSharedPtr<IPropertyHandleArray> CVarsArrayHandle = CVarsHandle->AsArray();

	IDetailCategoryBuilder& CVarDetailCategory = DetailBuilder->EditCategory("ConsoleVariables");

	uint32 CVarCount = 0;
	ensure(CVarsArrayHandle->GetNumElements(CVarCount) == FPropertyAccess::Success);

	// Sort the properties handles into Categories
	TMap<DeviceProfileCVarFormatHelper::ECVarGroup, TArray<TSharedRef<IPropertyHandle>>> CategoryPropertyMap;

	// Add all the CVar groups, even if these are empty
	for (int32 CategoryIdx = 0; CategoryIdx < (int32)DeviceProfileCVarFormatHelper::Max_CVarCategories; CategoryIdx++)
	{
		CategoryPropertyMap.FindOrAdd((DeviceProfileCVarFormatHelper::ECVarGroup)CategoryIdx);
	}

	for (uint32 CVarPropertyIdx = 0; CVarPropertyIdx < CVarCount; CVarPropertyIdx++)
	{
		// Get the current CVar as a string
		FString CVarValue;
		TSharedRef<IPropertyHandle> CVarElementHandle = CVarsArrayHandle->GetElement(CVarPropertyIdx);
		ensure(CVarElementHandle->GetValue(CVarValue) == FPropertyAccess::Success);

		// Parse the CVar entry and obtain the name and Category Name
		const FString CVarName = CVarValue.Left(CVarValue.Find(TEXT("=")));
		int32 CategoryEndIndex = CVarName.Find(TEXT("."));

		FString CVarAbrv = CVarName.Left(CategoryEndIndex);
		DeviceProfileCVarFormatHelper::ECVarGroup CVarCategory = DeviceProfileCVarFormatHelper::CategoryEnumFromPrefix(CVarAbrv);

		TArray<TSharedRef<IPropertyHandle>>* CurrentPropertyCategoryGroup = CategoryPropertyMap.Find(CVarCategory);
		CurrentPropertyCategoryGroup->Add(CVarElementHandle);
	}


	// Put the property handles into the UI group for the details view.
	for (auto& Current : CategoryPropertyMap)
	{
		TArray<TSharedRef<IPropertyHandle>>& CurrentGroupsProperties = Current.Value;
		CurrentGroupsProperties.Sort([](const TSharedRef<IPropertyHandle>& A, const TSharedRef<IPropertyHandle>& B)
		{ 
			FString AVal;
			A->GetValue(AVal);
			FString BVal;
			B->GetValue(BVal);
			return AVal < BVal;
		});

		const FText GroupName = DeviceProfileCVarFormatHelper::CategoryTextFromEnum(Current.Key);

		FString CVarPrefix = DeviceProfileCVarFormatHelper::CategoryPrefixFromEnum(Current.Key);
		if (CVarPrefix.Len() > 0)
		{
			CVarPrefix += TEXT(".");
		}

		// Find the Property table UI Group for the current CVar
		IDetailGroup& CVarGroup = CVarDetailCategory.AddGroup(*GroupName.ToString(), GroupName);
		CVarDetailGroups.Add(GroupName.ToString(), &CVarGroup);
		CVarGroup.HeaderRow()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(DeviceProfilePropertyConstants::PropertyPadding)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(GroupName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.Padding(DeviceProfilePropertyConstants::PropertyPadding)
				.AutoWidth()
				[
					SNew(SComboButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					]
					.MenuContent()
					[
						SNew(SCVarSelectionPanel, CVarPrefix)
						.OnCVarSelected(this, &FDeviceProfileConsoleVariablesPropertyDetails::HandleCVarAdded)
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(DeviceProfilePropertyConstants::PropertyPadding)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.OnClicked(this, &FDeviceProfileConsoleVariablesPropertyDetails::OnRemoveAllFromGroup, (int32)Current.Key)
					.ContentPadding(4.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Delete"))
					]
				]
			];

		for (auto& Property : CurrentGroupsProperties)
		{
			CreateRowWidgetForCVarProperty(Property, CVarGroup);
		}

		CVarDetailCategory.InitiallyCollapsed(true);
	}
}


void FDeviceProfileConsoleVariablesPropertyDetails::CreateRowWidgetForCVarProperty(TSharedPtr<IPropertyHandle> InProperty, IDetailGroup& InGroup) const
{
	FString UnformattedCVar;
	ensure(InProperty->GetValue(UnformattedCVar) == FPropertyAccess::Success);

	const int32 CVarNameValueSplitIdx = UnformattedCVar.Find(TEXT("="));
	ensure(CVarNameValueSplitIdx != INDEX_NONE);
	const FString CVarName = UnformattedCVar.Left(CVarNameValueSplitIdx);
	const FString CVarValueAsString = UnformattedCVar.Right(UnformattedCVar.Len() - (CVarNameValueSplitIdx + 1));

	FText CVarHelp;
	if (IConsoleVariable* Var = IConsoleManager::Get().FindConsoleVariable(*CVarName))
	{
		CVarHelp = FText::FromString(Var->GetHelp());
	}

	InGroup.AddWidgetRow()
	.IsEnabled(true)
	.Visibility(EVisibility::Visible)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString(CVarName))
		.ToolTipText(CVarHelp)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SEditableTextBox)
			.Text(FText::FromString(CVarValueAsString))
			.SelectAllTextWhenFocused(true)
			.OnTextCommitted(const_cast<FDeviceProfileConsoleVariablesPropertyDetails*>(this), &FDeviceProfileConsoleVariablesPropertyDetails::OnCVarValueCommited, InProperty)
		]
		+ SHorizontalBox::Slot()
		.Padding(DeviceProfilePropertyConstants::PropertyPadding)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(const_cast<FDeviceProfileConsoleVariablesPropertyDetails*>(this), &FDeviceProfileConsoleVariablesPropertyDetails::OnRemoveCVarProperty, InProperty)
			.ContentPadding(4.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.X"))
			]
		]
	];
}


void FDeviceProfileConsoleVariablesPropertyDetails::HandleCVarAdded(const FString& SelectedCVar)
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*SelectedCVar))
	{
		const FString CompleteCVarString = FString::Printf(TEXT("%s=%s"), *SelectedCVar, *CVar->GetString());

		TArray<void*> RawPtrs;
		CVarsHandle->AccessRawData(RawPtrs);

		// Update the CVars with the selection
		{
			CVarsHandle->NotifyPreChange();
			for (void* RawPtr : RawPtrs)
			{
				TArray<FString>& Array = *(TArray<FString>*)RawPtr;

				Array.Add(CompleteCVarString);
			}
			CVarsHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
		}

		// Update the UI with the selection
		// -> Close the selection menu
		FSlateApplication::Get().DismissAllMenus();
	}
}


void FDeviceProfileConsoleVariablesPropertyDetails::OnCVarValueCommited(const FText& CommentText, ETextCommit::Type CommitInfo, TSharedPtr<IPropertyHandle> CVarPropertyHandle)
{
	if (CVarPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = CVarPropertyHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		// Get the current CVar as a string
		FString OldCompleteCVarValue;
		ensure(CVarPropertyHandle->GetValue(OldCompleteCVarValue) == FPropertyAccess::Success);

		// Update the CVar. I.e. MyCVar=1
		const FString CVarLHS = OldCompleteCVarValue.Left(OldCompleteCVarValue.Find(TEXT("=")));
		const FString CVarRHS = CommentText.ToString();
		
		FString NewCompleteCVar = FString::Printf(TEXT("%s=%s"), *CVarLHS, *CVarRHS);

		if(OldCompleteCVarValue!=NewCompleteCVar)
		{
			CVarPropertyHandle->SetValue(NewCompleteCVar);
		}
	}
}


FReply FDeviceProfileConsoleVariablesPropertyDetails::OnRemoveCVarProperty(TSharedPtr<IPropertyHandle> CVarPropertyHandle)
{
	if (CVarPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = CVarPropertyHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		ParentArrayHandle->DeleteItem(CVarPropertyHandle->GetIndexInArray());
	}

	OnCVarPropertyChanged();

	return FReply::Handled();
}


FReply FDeviceProfileConsoleVariablesPropertyDetails::OnRemoveAllFromGroup(int32 CVarCategory)
{
	const TSharedPtr<IPropertyHandleArray> CVarsArrayHandle = CVarsHandle->AsArray();

	uint32 CVarCount = 0;
	ensure(CVarsArrayHandle->GetNumElements(CVarCount) == FPropertyAccess::Success);

	FString CVarPrefix = DeviceProfileCVarFormatHelper::CategoryPrefixFromEnum((DeviceProfileCVarFormatHelper::ECVarGroup)CVarCategory);

	for (int32 CVarPropertyIdx = CVarCount-1; CVarPropertyIdx >= 0; CVarPropertyIdx--)
	{
		// Get the current CVar as a string
		FString CVarValue;
		TSharedRef<IPropertyHandle> CVarElementHandle = CVarsArrayHandle->GetElement(CVarPropertyIdx);
		ensure(CVarElementHandle->GetValue(CVarValue) == FPropertyAccess::Success);

		// Parse the CVar entry and obtain the name and Category Name
		const FString CVarName = CVarValue.Left(CVarValue.Find(TEXT("=")));
		int32 CategoryEndIndex = CVarName.Find(TEXT("."));

		FString CurrentCVarPrefix = CVarName.Left(CategoryEndIndex);
		if(CVarPrefix == CurrentCVarPrefix)
		{
			CVarsArrayHandle->DeleteItem(CVarElementHandle->GetIndexInArray());
		}
	}

	OnCVarPropertyChanged();

	return FReply::Handled();
}


void FDeviceProfileConsoleVariablesPropertyDetails::OnCVarPropertyChanged()
{
	DetailBuilder->ForceRefreshDetails();
}

#undef LOCTEXT_NAMESPACE
