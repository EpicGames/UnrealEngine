// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"

class FDeviceProfileTextureLODSettingsDetails;
class FText;
class IDetailGroup;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

/* FDeviceProfileParentPropertyDetails
 *****************************************************************************/

/**
* Helper which implements details panel customizations for a device profiles parent property
*/
class FDeviceProfileParentPropertyDetails
	: public TSharedFromThis<FDeviceProfileParentPropertyDetails>
{

public:

	/**
	 * Constructor for the parent property details view
	 *
	 * @param InDetailsBuilder Where we are adding our property view to
	 */
	FDeviceProfileParentPropertyDetails(IDetailLayoutBuilder* InDetailBuilder);


	/**
	 * Create the parent property view for the device profile
	 */
	void CreateParentPropertyView();

private:

	/**
	 * Select a parent Combo-box listings
	 *
	 * @param InItem The prospective parent profile name we are creating a listing for
	 * @return The slate widget with the listed profile name
	 */
	TSharedRef<SWidget> HandleDeviceProfileParentComboBoxGenarateWidget(TSharedPtr<FString> InItem);

	/**
	 * Handle a new parent profile being selected for the active device profile
	 *
	 * @param NewSelection The name of the selected parent profile
	 * @param SelectionInfo	The selection type information
	 */
	void HandleDeviceProfileParentSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/**
	* Delegate used when the device profiles parent is updated from any source.
	*/
	void OnParentPropertyChanged();

private:

	/** A handle to the detail view builder */
	IDetailLayoutBuilder* DetailBuilder;

	/** Access to the Parent Property */
	TSharedPtr<IPropertyHandle> ParentPropertyNameHandle;

	/** Collection of possible Device Profiles we can use as a parent for this profile */
	TArray<TSharedPtr<FString>> AvailableParentProfiles;
	
	/** A reference to the object we are showing these properties for */
	class UDeviceProfile* ActiveDeviceProfile;
};


/* FDeviceProfileConsoleVariablesPropertyDetails
 *****************************************************************************/

/**
* Helper which implements details panel customizations for a device profiles console variables property
*/
class FDeviceProfileConsoleVariablesPropertyDetails
	: public TSharedFromThis<FDeviceProfileConsoleVariablesPropertyDetails>
{

public:

	/**
	 * Constructor for the active profiles' console variables property details view
	 *
	 * @param InDetailsBuilder Where we are adding our property view to
	 */
	FDeviceProfileConsoleVariablesPropertyDetails(IDetailLayoutBuilder* InDetailBuilder);


	/**
	 * Create the Console Variables property view for the device profile
	 */
	void CreateConsoleVariablesPropertyView();

private:

	/**
	 * Action when a CVar has been selected for add to the device profile
	 *
	 * @param SelectedCVar The CVar added
	 */
	void HandleCVarAdded(const FString& SelectedCVar);


	/**
	 * Action when a CVar has been selected for add to the device profile
	 *
	 * @param InProperty The property handle of which we are creating a row for.
	 * @param InGroup The group the property is to be listed under.
	 */
	void CreateRowWidgetForCVarProperty(TSharedPtr<IPropertyHandle> InProperty, IDetailGroup& InGroup) const;


	/**
	 * Action when a CVar has been updated on the device profile
	 *
	 * @param NewValue The New value for the given CVar
	 * @param CommitInfo Details of how the action was committed
	 * @param CVarPropertyHandle The CVar property which has been updated
	 */
	void OnCVarValueCommited(const FText& NewValue, ETextCommit::Type CommitInfo, TSharedPtr<IPropertyHandle> CVarPropertyHandle);


	/**
	 * Action when a CVar has removed from the device profile
	 *
	 * @param InProperty The property which has been removed from the profile
	 * @return Whether the event was handled
	 */
	FReply OnRemoveCVarProperty(TSharedPtr<IPropertyHandle> InProperty);


	/**
	 * Action when a group of CVars have been removed from the device profile
	 *
	 * @param CVarCategory - The category of the group all CVars have been removed for (@see ECVarGroup).
	 * @return Whether the event was handled
	 */
	FReply OnRemoveAllFromGroup(int32 CVarCategory);


	/**
	 * Delegate used when the console variables are updated from any source.
	 */
	void OnCVarPropertyChanged();

private:

	/** A handle to the detail view builder */
	IDetailLayoutBuilder* DetailBuilder;

	/** A handle to the CVars array where the entire CVar collection is */
	TSharedPtr<IPropertyHandle> CVarsHandle;

	/** The collection of console variable property groups and their id's*/
	TMap<FString, IDetailGroup*> CVarDetailGroups;
};


/* FDeviceProfileDetails
 *****************************************************************************/

/**
 * Implements details panel customizations for UDeviceProfile fields.
 */
class FDeviceProfileDetails
	: public IDetailCustomization
{
public:

	/**
	 * Makes a new instance of this device profile detail layout class.
	 *
	 * @return The created instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FDeviceProfileDetails);
	}

public:

	// IDetailCustomization interface

	virtual void CustomizeDetails( IDetailLayoutBuilder& InDetailBuilder ) override;

private:

	/** Reference to the parent profile property view */
	TSharedPtr<FDeviceProfileParentPropertyDetails> ParentProfileDetails;

	/** Reference to the console variables property view */
	TSharedPtr<FDeviceProfileConsoleVariablesPropertyDetails> ConsoleVariablesDetails;

	/** Reference to the console variables property view */
	TSharedPtr<FDeviceProfileTextureLODSettingsDetails> TextureLODSettingsDetails;
};
