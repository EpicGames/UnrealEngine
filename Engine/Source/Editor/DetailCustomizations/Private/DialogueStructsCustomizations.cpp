// Copyright Epic Games, Inc. All Rights Reserved.

#include "DialogueStructsCustomizations.h"

#include "AssetThumbnail.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DialogueWaveWidgets.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Sound/DialogueTypes.h"
#include "Sound/DialogueVoice.h"
#include "Sound/DialogueWave.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "DialogueWaveDetails"

TSharedRef<IPropertyTypeCustomization> FDialogueContextStructCustomization::MakeInstance() 
{
	return MakeShareable( new FDialogueContextStructCustomization );
}

void FDialogueContextStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	if( StructPropertyHandle->IsValidHandle() )
	{
		HeaderRow
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("DialogueWaveDetails.HeaderBorder") )
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					StructPropertyHandle->CreatePropertyNameWidget()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SDialogueContextHeaderWidget, StructPropertyHandle, StructCustomizationUtils.GetThumbnailPool().ToSharedRef() )
				]
			]
		];
	}
}

void FDialogueContextStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	if( StructPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = StructPropertyHandle->GetChildHandle("Speaker");
		ChildBuilder.AddProperty(SpeakerPropertyHandle.ToSharedRef());

		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = StructPropertyHandle->GetChildHandle("Targets");
		ChildBuilder.AddProperty(TargetsPropertyHandle.ToSharedRef());
	}
}

class SSpeakerDropDown : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSpeakerDropDown ) {}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:
	void OnSelectionChanged(TSharedPtr<UDialogueVoice*> Speaker, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> MakeComboButtonItemWidget(TSharedPtr<UDialogueVoice*> Speaker);

private:
	TSharedPtr<IPropertyHandle> DialogueWaveParameterPropertyHandle;
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
	TSharedPtr< SComboBox< TSharedPtr<UDialogueVoice*> > > ComboBox;
	TArray< TSharedPtr<UDialogueVoice*> > OptionsSource;
};

void SSpeakerDropDown::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	DialogueWaveParameterPropertyHandle = InPropertyHandle;
	AssetThumbnailPool = InAssetThumbnailPool;

	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");

	TSharedRef<SDialogueVoicePropertyEditor> SpeakerPropertyEditor =
		SNew( SDialogueVoicePropertyEditor, SpeakerPropertyHandle.ToSharedRef(), InAssetThumbnailPool )
		.IsEditable(false)
		.ShouldCenterThumbnail(true);

	FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();

	ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SAssignNew( ComboBox, SComboBox< TSharedPtr<UDialogueVoice*> > )
			.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
			.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.OptionsSource( &OptionsSource )
			.OnGenerateWidget( this, &SSpeakerDropDown::MakeComboButtonItemWidget )
			.OnSelectionChanged( this, &SSpeakerDropDown::OnSelectionChanged )
			[
				SpeakerPropertyEditor
			]
		]
		+SVerticalBox::Slot()
		.Padding( 2.0f )
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			// Voice Description
			SNew( STextBlock )
			.Font( Font )
			.Text( SpeakerPropertyEditor, &SDialogueVoicePropertyEditor::GetDialogueVoiceDescription )
		]
	];
}

void SSpeakerDropDown::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Get dialogue wave.
	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	const UDialogueWave* DialogueWave = NULL;
	if( DialogueWavePropertyHandle->IsValidHandle() )
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		DialogueWave = Cast<UDialogueWave>(Object);
	}

	// Get context.
	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");

	// Get speaker.
	const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
	const UDialogueVoice* Speaker = NULL;
	{
		UObject* Object = NULL;
		SpeakerPropertyHandle->GetValue(Object);
		Speaker = Cast<UDialogueVoice>(Object);
	}

	// Gather unique speaker options.
	TArray<UDialogueVoice*> UniqueSpeakers;
	if(DialogueWave)
	{
		for(int32 i = 0; i < DialogueWave->ContextMappings.Num(); ++i)
		{
			bool bIsValidSpeaker = ( DialogueWave->ContextMappings[i].Context.Speaker != NULL );

			bool bIsValidTargetSet = true;
			for(int32 j = 0; j < DialogueWave->ContextMappings[i].Context.Targets.Num(); ++j)
			{
				bIsValidTargetSet = ( DialogueWave->ContextMappings[i].Context.Targets[j] != NULL );
				if( !bIsValidTargetSet )
				{
					break;
				}
			}

			if(bIsValidSpeaker && bIsValidTargetSet)
			{
				UniqueSpeakers.AddUnique( DialogueWave->ContextMappings[i].Context.Speaker );
			}
		}
	}

	// Check if a refresh is needed.
	bool bNeedsRefresh = false;
	if( OptionsSource.Num() == UniqueSpeakers.Num() )
	{
		for(int32 i = 0; i < UniqueSpeakers.Num(); ++i)
		{
			if( *(OptionsSource[i]) != UniqueSpeakers[i] )
			{
				bNeedsRefresh = true;
				break;
			}
		}
	}
	else
	{
		bNeedsRefresh = true;
	}

	// Refresh if needed.
	if(bNeedsRefresh)
	{
		OptionsSource.Empty();
		if(DialogueWave)
		{
			for(int32 i = 0; i < UniqueSpeakers.Num(); ++i)
			{
				OptionsSource.Add( MakeShareable( new UDialogueVoice*(UniqueSpeakers[i]) ) );
			}
		}
		ComboBox->ClearSelection();
		ComboBox->RefreshOptions();
	}
}

struct FSpeakerMatcher
{
	const UDialogueVoice* Speaker;
	bool operator()(const FDialogueContextMapping& Mapping) const
	{
		return Mapping.Context.Speaker == Speaker;
	}
};

void SSpeakerDropDown::OnSelectionChanged(TSharedPtr<UDialogueVoice*> Speaker, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");

	UDialogueVoice** SpeakerToChange = NULL;
	{
		TArray<void*> RawData;
		SpeakerPropertyHandle->AccessRawData(RawData);
		SpeakerToChange = reinterpret_cast<UDialogueVoice**>(RawData[0]);
	}

	if( SpeakerToChange && Speaker.IsValid() )
	{
		SpeakerPropertyHandle->NotifyPreChange();
		*SpeakerToChange = *Speaker;
		SpeakerPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

TSharedRef<SWidget> SSpeakerDropDown::MakeComboButtonItemWidget(TSharedPtr<UDialogueVoice*> Speaker)
{
	const uint32 ThumbnailSizeX = 64;
	const uint32 ThumbnailSizeY = 64;

	const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable( new FAssetThumbnail( *Speaker, ThumbnailSizeX, ThumbnailSizeY, AssetThumbnailPool ) );

	return
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew( SBox )
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride( static_cast<float>(ThumbnailSizeX) ) 
				.HeightOverride( static_cast<float>(ThumbnailSizeY) )
				[
					AssetThumbnail->MakeThumbnailWidget()
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			// Voice Description
			SNew( STextBlock )
			.Text( ( Speaker.IsValid() && *Speaker ) ? FText::FromString((*Speaker)->GetDesc()) : LOCTEXT("None", "None") )
		];
}

class STargetSetDropDown : public SCompoundWidget
{
public:
	typedef TArray<UDialogueVoice*> FTargetSet;

	SLATE_BEGIN_ARGS( STargetSetDropDown ) {}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:
	void OnSelectionChanged(TSharedPtr<FTargetSet> TargetSet, ESelectInfo::Type SelectInfo);
	float GetPreferredWidthForWrapping() const;
	TSharedRef<SWidget> MakeComboButtonItemWidget(TSharedPtr<FTargetSet> TargetSet);

private:
	TSharedPtr<IPropertyHandle> DialogueWaveParameterPropertyHandle;
	TSharedPtr<FAssetThumbnailPool>  AssetThumbnailPool;
	TSharedPtr< SComboBox< TSharedPtr<FTargetSet> > > ComboBox;
	TArray< TSharedPtr<FTargetSet> > OptionsSource;
	float AllottedWidth;
};

void STargetSetDropDown::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	DialogueWaveParameterPropertyHandle = InPropertyHandle;
	AssetThumbnailPool = InAssetThumbnailPool;

	AllottedWidth = 0.0f;

	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context") ;
	const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets") ;

	TSharedRef<STargetsSummaryWidget> TargetsSummaryWidget =
		SNew( STargetsSummaryWidget, TargetsPropertyHandle.ToSharedRef(), AssetThumbnailPool.ToSharedRef() )
		.IsEditable(false)
		.WrapWidth( this, &STargetSetDropDown::GetPreferredWidthForWrapping);

	FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();

	ChildSlot
	.HAlign(HAlign_Center)
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			SAssignNew( ComboBox, SComboBox< TSharedPtr<FTargetSet> > )
			.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
			.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.OptionsSource( &OptionsSource )
			.OnGenerateWidget( this, &STargetSetDropDown::MakeComboButtonItemWidget )
			.OnSelectionChanged( this, &STargetSetDropDown::OnSelectionChanged )
			[
				TargetsSummaryWidget
			]
		]
		+SVerticalBox::Slot()
		.Padding( 2.0f )
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			// Voice Description
			SNew( STextBlock )
			.Font( Font )
			.Text( TargetsSummaryWidget, &STargetsSummaryWidget::GetDialogueVoiceDescription )
		]
	];
}

void STargetSetDropDown::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	AllottedWidth = AllottedGeometry.Size.X;

	// Get dialogue wave.
	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	const UDialogueWave* DialogueWave = NULL;
	if( DialogueWavePropertyHandle->IsValidHandle() )
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		DialogueWave = Cast<UDialogueWave>(Object);
	}

	// Get context.
	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");

	// Get speaker.
	const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
	const UDialogueVoice* Speaker = NULL;
	{
		UObject* Object = NULL;
		SpeakerPropertyHandle->GetValue(Object);
		Speaker = Cast<UDialogueVoice>(Object);
	}

	// Get target set.
	const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
	FTargetSet* TargetSet = NULL;
	{
		TArray<void*> RawData;
		TargetsPropertyHandle->AccessRawData(RawData);
		TargetSet = reinterpret_cast<FTargetSet*>(RawData[0]);
	}

	// Gather unique target set options.
	TArray<FTargetSet> UniqueTargetSets;
	if( DialogueWave && Speaker )
	{
		for(int32 i = 0; i < DialogueWave->ContextMappings.Num(); ++i)
		{
			if( DialogueWave->ContextMappings[i].Context.Speaker == Speaker )
			{
				bool bIsValidTargetSet = true;

				for(int32 j = 0; j < DialogueWave->ContextMappings[i].Context.Targets.Num(); ++j)
				{
					bIsValidTargetSet = ( DialogueWave->ContextMappings[i].Context.Targets[j] != NULL );
					if( !bIsValidTargetSet )
					{
						break;
					}
				}

				if( bIsValidTargetSet )
				{
					UniqueTargetSets.AddUnique( DialogueWave->ContextMappings[i].Context.Targets );
				}
			}
		}
	}

	// Check if a refresh is needed.
	bool bNeedsRefresh = false;
	if( OptionsSource.Num() == UniqueTargetSets.Num() )
	{
		for(int32 i = 0; i < UniqueTargetSets.Num(); ++i)
		{
			if( *(OptionsSource[i]) != UniqueTargetSets[i] )
			{
				bNeedsRefresh = true;
				break;
			}
		}
	}
	else
	{
		bNeedsRefresh = true;
	}

	// Refresh if needed.
	if(bNeedsRefresh)
	{
		OptionsSource.Empty();
		if(DialogueWave)
		{
			for(int32 i = 0; i < UniqueTargetSets.Num(); ++i)
			{
				OptionsSource.Add( MakeShareable( new FTargetSet(UniqueTargetSets[i]) ) );
			}
		}
		ComboBox->ClearSelection();
		ComboBox->RefreshOptions();
	}
}

void STargetSetDropDown::OnSelectionChanged(TSharedPtr<FTargetSet> TargetSet, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");

	FTargetSet* TargetSetToChange = NULL;
	{
		TArray<void*> RawData;
		TargetsPropertyHandle->AccessRawData(RawData);
		TargetSetToChange = reinterpret_cast<FTargetSet*>(RawData[0]);
	}

	if( TargetSet.IsValid() )
	{
		TargetsPropertyHandle->NotifyPreChange();
		*TargetSetToChange = *TargetSet;
		TargetsPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

float STargetSetDropDown::GetPreferredWidthForWrapping() const
{
	return AllottedWidth;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STargetSetDropDown::MakeComboButtonItemWidget(TSharedPtr<FTargetSet> TargetSet)
{
	TSharedPtr<SWidget> Result;

	const uint32 ThumbnailSizeX = 64;
	const uint32 ThumbnailSizeY = 64;

	FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();

	if( TargetSet->Num() > 1 )
	{
		const TSharedRef<SWrapBox> WrapBox =
			SNew( SWrapBox )
			.PreferredSize( this, &STargetSetDropDown::GetPreferredWidthForWrapping );

		for(int32 i = 0; i < TargetSet->Num(); ++i)
		{
			const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable( new FAssetThumbnail( (*TargetSet)[i], ThumbnailSizeX, ThumbnailSizeY, AssetThumbnailPool ) );

			WrapBox->AddSlot()
			.Padding(2.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew( SBox )
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew( SBox )
					.WidthOverride( static_cast<float>(ThumbnailSizeX) ) 
					.HeightOverride( static_cast<float>(ThumbnailSizeY) )
					[
						AssetThumbnail->MakeThumbnailWidget()
					]
				]
			];
		}

		Result =
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				WrapBox
			]
			+SVerticalBox::Slot()
			.Padding( 2.0f )
			.HAlign(HAlign_Center)
			.AutoHeight()
			[
				// Voice Description
				SNew( STextBlock )
				.Font( Font )
				.Text( LOCTEXT("Multiple", "Multiple") )
			];
	}
	else if( TargetSet->Num() == 1 )
	{
		UDialogueVoice* Target = (*TargetSet)[0];
		const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable( new FAssetThumbnail( Target, ThumbnailSizeX, ThumbnailSizeY, AssetThumbnailPool ) );

		SAssignNew( Result, SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew( SBox )
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride( static_cast<float>(ThumbnailSizeX) ) 
				.HeightOverride( static_cast<float>(ThumbnailSizeY) )
				[
					AssetThumbnail->MakeThumbnailWidget()
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			// Voice Description
			SNew( STextBlock )
			.Font( Font )
			.Text( Target ? FText::FromString(Target->GetDesc()) : LOCTEXT("None", "None") )
		];
	}
	else
	{
		SAssignNew( Result, SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			SNew( SBox )
			.WidthOverride( static_cast<float>(ThumbnailSizeX) )
			.HeightOverride( static_cast<float>(ThumbnailSizeY) )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
		]
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.AutoHeight()
		[
			// Voice Description
			SNew( STextBlock )
			.Font( Font )
			.Text( LOCTEXT( "NoTargets", "No One" ) )
		];
	}

	return Result.ToSharedRef();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

class SValidatedDialogueContextHeaderWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SValidatedDialogueContextHeaderWidget ) {}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:
	bool IsDialogueWaveValid() const;
	bool IsSpeakerValid() const;
	bool IsTargetSetValid() const;

	bool ShouldSpeakerDropDownBeEnabled() const;
	EVisibility GetSpeakerErrorVisibility() const;

	bool ShouldTargetsDropDownBeEnabled() const;
	EVisibility GetTargetsErrorVisibility() const;

private:
	const UDialogueWave* CurrentDialogueWave;
	TSharedPtr<IPropertyHandle> DialogueWaveParameterPropertyHandle;
	TSharedPtr<SErrorHint> ContextErrorHint;
	TSharedPtr<SErrorHint> SpeakerErrorHint;
	TSharedPtr<SErrorText> SpeakerErrorText;
	TSharedPtr<SErrorHint> TargetsErrorHint;
	TSharedPtr<SErrorText> TargetsErrorText;
};

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SValidatedDialogueContextHeaderWidget::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	DialogueWaveParameterPropertyHandle = InPropertyHandle;

	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	CurrentDialogueWave = NULL;
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		CurrentDialogueWave = Cast<UDialogueWave>(Object);
	}

	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	if( ContextPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");

		ChildSlot
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("DialogueWaveDetails.HeaderBorder") )
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SAssignNew( ContextErrorHint, SErrorHint )
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						ContextPropertyHandle->CreatePropertyNameWidget()
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNullWidget::NullWidget
					]
				]
				+SVerticalBox::Slot()
				.Padding( FMargin( 4.0f, 2.0f, 4.0f, 4.0f ) )
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.FillWidth(1.0f)
							[
								SAssignNew( SpeakerErrorHint, SErrorHint )
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SpeakerPropertyHandle->CreatePropertyNameWidget()
							]
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNullWidget::NullWidget
							]
						]
						+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						.FillHeight(1.0f)
						[
							SNew( SOverlay )
							+SOverlay::Slot()
							[
								SNew( SSpeakerDropDown, DialogueWaveParameterPropertyHandle.ToSharedRef(), InAssetThumbnailPool )
								.IsEnabled( this, &SValidatedDialogueContextHeaderWidget::ShouldSpeakerDropDownBeEnabled )
							]
							+SOverlay::Slot()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew( SBox )
								.Visibility( this, &SValidatedDialogueContextHeaderWidget::GetSpeakerErrorVisibility )
								[
									SAssignNew( SpeakerErrorText, SErrorText )
								]
							]
						]
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(4.0f)
					.AutoWidth()
					[
						SNew( SImage )
						.Image( FAppStyle::GetBrush("DialogueWaveDetails.SpeakerToTarget") )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.FillWidth(1.0f)
							[
								SAssignNew( TargetsErrorHint, SErrorHint )
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								TargetsPropertyHandle->CreatePropertyNameWidget( LOCTEXT("DirectedAt", "Directed At") )
							]
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								SNullWidget::NullWidget
							]
						]
						+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.FillHeight(1.0f)
						[
							SNew( SOverlay )
							+SOverlay::Slot()
							[
								SNew( STargetSetDropDown, DialogueWaveParameterPropertyHandle.ToSharedRef(), InAssetThumbnailPool )
								.IsEnabled( this, &SValidatedDialogueContextHeaderWidget::ShouldTargetsDropDownBeEnabled )
							]
							+SOverlay::Slot()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew( SBox )
								.Visibility( this, &SValidatedDialogueContextHeaderWidget::GetTargetsErrorVisibility )
								[
									SAssignNew( TargetsErrorText, SErrorText )
								]
							]
						]
					]
				]
			]
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SValidatedDialogueContextHeaderWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( DialogueWaveParameterPropertyHandle.IsValid() && DialogueWaveParameterPropertyHandle->IsValidHandle() )
	{
		// Get dialogue wave.
		const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
		const UDialogueWave* DialogueWave = NULL;
		if( DialogueWavePropertyHandle.IsValid() && DialogueWavePropertyHandle->IsValidHandle() )
		{
			UObject* Object = NULL;
			DialogueWavePropertyHandle->GetValue(Object);
			DialogueWave = Cast<UDialogueWave>(Object);
		}

		// Get context.
		const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");

		// Get speaker.
		const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
		UDialogueVoice* Speaker = NULL;
		if( SpeakerPropertyHandle.IsValid() && SpeakerPropertyHandle->IsValidHandle())
		{
			UObject* Object = NULL;
			SpeakerPropertyHandle->GetValue(Object);
			Speaker = Cast<UDialogueVoice>(Object);
		}

		// Get target set.
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
		STargetSetDropDown::FTargetSet* TargetSet = NULL;
		if( TargetsPropertyHandle.IsValid() && TargetsPropertyHandle->IsValidHandle())
		{
			TArray<void*> RawData;
			TargetsPropertyHandle->AccessRawData(RawData);
			TargetSet = reinterpret_cast<STargetSetDropDown::FTargetSet*>(RawData[0]);
		}

		bool bDidDialogueWaveChange = (CurrentDialogueWave != DialogueWave);
		if( bDidDialogueWaveChange )
		{
			CurrentDialogueWave = DialogueWave;

			// Check if the speaker needs to be reset.
			bool bSpeakerNeedsReset = true;
			bool bTargetSetNeedsReset = true;
			if( DialogueWave )
			{
				for(int32 i = 0; i < DialogueWave->ContextMappings.Num(); ++i)
				{
					if( bSpeakerNeedsReset )
					{
						bSpeakerNeedsReset = ( DialogueWave->ContextMappings[i].Context.Speaker != Speaker );
					}

					if( bTargetSetNeedsReset && TargetSet )
					{
						bTargetSetNeedsReset = ( ToRawPtrTArrayUnsafe(DialogueWave->ContextMappings[i].Context.Targets) != *TargetSet );
					}

					if( !bSpeakerNeedsReset && !bTargetSetNeedsReset)
					{
						break;
					}
				}
			}

			// Don't try resets if there are no valid contexts.
			if( DialogueWave && DialogueWave->ContextMappings.Num() > 0)
			{
				// Speaker reset if needed and possible.
				if( bSpeakerNeedsReset && SpeakerPropertyHandle.IsValid() && SpeakerPropertyHandle->IsValidHandle() )
				{
					const UObject* Object = DialogueWave->ContextMappings[0].Context.Speaker;
					SpeakerPropertyHandle->SetValue( Object );
				}
				// Target set reset if needed.
				if(bTargetSetNeedsReset)
				{
					// Reset if possible.
					if( TargetSet && TargetsPropertyHandle.IsValid() && TargetsPropertyHandle->IsValidHandle() )
					{
						TargetsPropertyHandle->NotifyPreChange();
						*TargetSet = DialogueWave->ContextMappings[0].Context.Targets;
						TargetsPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
					}
				}
			}
		}
	}

	if( !IsDialogueWaveValid() )
	{
		if( ContextErrorHint.IsValid() )	{ ContextErrorHint->SetError( LOCTEXT("InvalidDialogueWaveError", "Invalid dialogue wave.") ); }
																		  
		if( SpeakerErrorHint.IsValid() )	{ SpeakerErrorHint->SetError( FText::GetEmpty() ); }
		if( SpeakerErrorText.IsValid() )	{ SpeakerErrorText->SetError( LOCTEXT("SelectDialogueWaveError", "Select a valid dialogue wave.") ); }
		if( TargetsErrorHint.IsValid() )	{ TargetsErrorHint->SetError( FText::GetEmpty() ); }
		if( TargetsErrorText.IsValid() )	{ TargetsErrorText->SetError( LOCTEXT("SelectDialogueWaveError", "Select a valid dialogue wave.") ); }
	}																	  
	else if( !IsSpeakerValid() )										  
	{																	  
		if( ContextErrorHint.IsValid() )	{ ContextErrorHint->SetError( FText::GetEmpty() ); }
																		  
		if( SpeakerErrorHint.IsValid() )	{ SpeakerErrorHint->SetError( LOCTEXT("InvalidSpeakerError", "Invalid speaker for dialogue wave.") ); }
		if( SpeakerErrorText.IsValid() )	{ SpeakerErrorText->SetError( FText::GetEmpty() ); }
		if( TargetsErrorHint.IsValid() )	{ TargetsErrorHint->SetError( FText::GetEmpty() ); }
		if( TargetsErrorText.IsValid() )	{ TargetsErrorText->SetError( LOCTEXT("SelectSpeakerError", "Select a valid speaker.") ); }
	}																	  
	else if( !IsTargetSetValid() )										  
	{																	  
		if( ContextErrorHint.IsValid() )	{ ContextErrorHint->SetError( FText::GetEmpty() ); }
																		  
		if( SpeakerErrorHint.IsValid() )	{ SpeakerErrorHint->SetError( FText::GetEmpty() ); }
		if( SpeakerErrorText.IsValid() )	{ SpeakerErrorText->SetError( FText::GetEmpty() ); }
		if( TargetsErrorHint.IsValid() )	{ TargetsErrorHint->SetError( LOCTEXT("SelectTargetsError", "Select a valid target set.") ); }
		if( TargetsErrorText.IsValid() )	{ TargetsErrorText->SetError( FText::GetEmpty() ); }
	}																	  
	else																  
	{																	  
		if( ContextErrorHint.IsValid() )	{ ContextErrorHint->SetError( FText::GetEmpty() ); }
																		  
		if( SpeakerErrorHint.IsValid() )	{ SpeakerErrorHint->SetError( FText::GetEmpty() ); }
		if( SpeakerErrorText.IsValid() )	{ SpeakerErrorText->SetError( FText::GetEmpty() ); }
		if( TargetsErrorHint.IsValid() )	{ TargetsErrorHint->SetError( FText::GetEmpty() ); }
		if( TargetsErrorText.IsValid() )	{ TargetsErrorText->SetError( FText::GetEmpty() ); }
	}
}

bool SValidatedDialogueContextHeaderWidget::IsDialogueWaveValid() const
{
	const UDialogueWave* DialogueWave = NULL;

	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	if( DialogueWavePropertyHandle->IsValidHandle() )
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		DialogueWave = Cast<UDialogueWave>(Object);
	}

	return DialogueWave != NULL && DialogueWave->ContextMappings.Num() > 0;
}

bool SValidatedDialogueContextHeaderWidget::IsSpeakerValid() const
{
	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	const UDialogueWave* DialogueWave = NULL;
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		DialogueWave = Cast<UDialogueWave>(Object);
	}

	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
	const UDialogueVoice* Speaker = NULL;
	{
		UObject* Object = NULL;
		SpeakerPropertyHandle->GetValue(Object);
		Speaker = Cast<UDialogueVoice>(Object);
	}

	bool bSpeakerIsValid = false;
	if( DialogueWave )
	{
		FSpeakerMatcher SpeakerMatcher = { Speaker };
		if (DialogueWave->ContextMappings.IndexOfByPredicate(SpeakerMatcher) != INDEX_NONE)
		{
			bSpeakerIsValid = true;
		}
	}

	return bSpeakerIsValid;
}

bool SValidatedDialogueContextHeaderWidget::IsTargetSetValid() const
{
	const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("DialogueWave");
	const UDialogueWave* DialogueWave = NULL;
	{
		UObject* Object = NULL;
		DialogueWavePropertyHandle->GetValue(Object);
		DialogueWave = Cast<UDialogueWave>(Object);
	}

	const TSharedPtr<IPropertyHandle> ContextPropertyHandle = DialogueWaveParameterPropertyHandle->GetChildHandle("Context");
	const FDialogueContext* DialogueContext = NULL;
	{
		TArray<void*> RawData;
		ContextPropertyHandle->AccessRawData(RawData);
		DialogueContext = reinterpret_cast<FDialogueContext*>(RawData[0]);
	}

	return DialogueWave && DialogueContext && DialogueWave->SupportsContext(*DialogueContext);
}

bool SValidatedDialogueContextHeaderWidget::ShouldSpeakerDropDownBeEnabled() const
{
	return IsDialogueWaveValid();
}

EVisibility SValidatedDialogueContextHeaderWidget::GetSpeakerErrorVisibility() const
{
	return SpeakerErrorText.IsValid() && SpeakerErrorText->HasError() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SValidatedDialogueContextHeaderWidget::ShouldTargetsDropDownBeEnabled() const
{
	return IsSpeakerValid();
}

EVisibility SValidatedDialogueContextHeaderWidget::GetTargetsErrorVisibility() const
{
	return TargetsErrorText.IsValid() && TargetsErrorText->HasError() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<IPropertyTypeCustomization> FDialogueWaveParameterStructCustomization::MakeInstance() 
{
	return MakeShareable( new FDialogueWaveParameterStructCustomization );
}

void FDialogueWaveParameterStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
}

void FDialogueWaveParameterStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	if( StructPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> DialogueWavePropertyHandle = StructPropertyHandle->GetChildHandle("DialogueWave");
		ChildBuilder.AddProperty(DialogueWavePropertyHandle.ToSharedRef());

		const TSharedPtr<IPropertyHandle> ContextPropertyHandle = StructPropertyHandle->GetChildHandle("Context");
		ChildBuilder.AddCustomRow( ContextPropertyHandle->GetPropertyDisplayName() )
		[
			SNew( SValidatedDialogueContextHeaderWidget, StructPropertyHandle, StructCustomizationUtils.GetThumbnailPool().ToSharedRef() )
		];
	}
}

#undef LOCTEXT_NAMESPACE
