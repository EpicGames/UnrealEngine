// Copyright Epic Games, Inc. All Rights Reserved.

#include "DialogueWaveDetails.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DialogueWaveWidgets.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Sound/DialogueTypes.h"
#include "Sound/DialogueWave.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "DialogueWaveDetails"

FDialogueContextMappingNodeBuilder::FDialogueContextMappingNodeBuilder(IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	: DetailLayoutBuilder(InDetailLayoutBuilder)
	, ContextMappingPropertyHandle(InPropertyHandle)
	, LocalizationKeyFormatPropertyHandle(ContextMappingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDialogueContextMapping, LocalizationKeyFormat)))
	, LastLocalizationKeyErrorUpdateTimestamp(0.0)
{
	check(LocalizationKeyFormatPropertyHandle.IsValid());
}

void FDialogueContextMappingNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	if (ContextMappingPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ContextPropertyHandle = ContextMappingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDialogueContextMapping, Context));
		if (ContextPropertyHandle->IsValidHandle())
		{
			const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDialogueContext, Speaker));
			const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDialogueContext, Targets));

			const TSharedPtr<IPropertyHandle> ParentHandle = ContextMappingPropertyHandle->GetParentHandle();
			const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

			uint32 ContextCount;
			ParentArrayHandle->GetNumElements(ContextCount);

			TSharedRef<SWidget> ClearButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FDialogueContextMappingNodeBuilder::RemoveContextButton_OnClick),
				ContextCount > 1 ? LOCTEXT("RemoveContextToolTip", "Remove context.") : LOCTEXT("RemoveContextDisabledToolTip", "Cannot remove context - a dialogue wave must have at least one context."),
				ContextCount > 1);

			NodeRow
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("DialogueWaveDetails.HeaderBorder"))
					[
						SNew(SDialogueContextHeaderWidget, ContextPropertyHandle.ToSharedRef(), DetailLayoutBuilder->GetThumbnailPool().ToSharedRef())
					]
				]
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					ClearButton
				]
			];
		}
	}
}

void FDialogueContextMappingNodeBuilder::Tick(float DeltaTime)
{
	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();
	if ((CurrentTime - LastLocalizationKeyErrorUpdateTimestamp) >= 1.0)
	{
		LocalizationKeyFormatEditableText_UpdateErrorText();
	}
}

void FDialogueContextMappingNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (ContextMappingPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> SoundWavePropertyHandle = ContextMappingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDialogueContextMapping, SoundWave));
		ChildrenBuilder.AddProperty(SoundWavePropertyHandle.ToSharedRef());

		ChildrenBuilder.AddCustomRow(LocalizationKeyFormatPropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			LocalizationKeyFormatPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.MaxDesiredWidth(TOptional<float>())
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(LocalizationKeyFormatEditableText, SEditableTextBox)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(this, &FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_GetText)
				.ToolTipText(LocalizationKeyFormatPropertyHandle->GetToolTipText())
				.OnTextCommitted(this, &FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_OnTextCommitted)
				.IsReadOnly(this, &FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_IsReadyOnly)
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 30.0f, 0.0f))
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text(this, &FDialogueContextMappingNodeBuilder::LocalizationKey_GetText)
				.ToolTipText(LOCTEXT("LocalizationKeyToolTipText", "The localization key used by this context."))
			]
		];

		LocalizationKeyFormatEditableText_UpdateErrorText();
	}
}

void FDialogueContextMappingNodeBuilder::RemoveContextButton_OnClick()
{
	if (ContextMappingPropertyHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = ContextMappingPropertyHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		uint32 ContextCount;
		ParentArrayHandle->GetNumElements(ContextCount);
		if (ContextCount != 1) // Mustn't remove the only context.
		{
			ParentArrayHandle->DeleteItem(ContextMappingPropertyHandle->GetIndexInArray());
			DetailLayoutBuilder->ForceRefreshDetails();
		}
	}
}

FText FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_GetText() const
{
	if (LocalizationKeyFormatPropertyHandle->IsValidHandle())
	{
		FString ValueStr;
		if (LocalizationKeyFormatPropertyHandle->GetValue(ValueStr) == FPropertyAccess::Success)
		{
			return FText::FromString(MoveTemp(ValueStr));
		}
	}

	return FText::GetEmpty();
}

void FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_OnTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (LocalizationKeyFormatPropertyHandle->IsValidHandle())
	{
		LocalizationKeyFormatPropertyHandle->SetValue(InNewText.ToString());
	}
}

bool FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_IsReadyOnly() const
{
	return !LocalizationKeyFormatPropertyHandle->IsValidHandle() || LocalizationKeyFormatPropertyHandle->IsEditConst();
}

void FDialogueContextMappingNodeBuilder::LocalizationKeyFormatEditableText_UpdateErrorText()
{
	if (LocalizationKeyFormatEditableText.IsValid())
	{
		LastLocalizationKeyErrorUpdateTimestamp = FSlateApplication::Get().GetCurrentTime();

		FText NewLocalizationKeyErrorMsg;

		// Check for duplicates in the localization keys
		const TArray<TWeakObjectPtr<UObject>>& RootObjects = DetailLayoutBuilder->GetSelectedObjects();
		if (RootObjects.Num() == 1)
		{
			auto DialogueWave = Cast<const UDialogueWave>(RootObjects[0].Get());
			if (DialogueWave && ContextMappingPropertyHandle->IsValidHandle())
			{
				TArray<const void*> RawData;
				ContextMappingPropertyHandle->AccessRawData(RawData);
				check(RawData.Num() == 1);

				auto ContextMappingPtr = static_cast<const FDialogueContextMapping*>(RawData[0]);
				const FString OurLocalizationKey = (ContextMappingPtr) ? DialogueWave->GetContextLocalizationKey(*ContextMappingPtr) : FString();

				bool bIsDuplicate = false;
				if (ContextMappingPropertyHandle->IsValidHandle())
				{
					const TSharedPtr<IPropertyHandle> ParentHandle = ContextMappingPropertyHandle->GetParentHandle();
					const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

					uint32 ContextCount = 0;
					ParentArrayHandle->GetNumElements(ContextCount);
					for (uint32 j = 0; j < ContextCount && !bIsDuplicate; ++j)
					{
						if (ContextMappingPropertyHandle->GetIndexInArray() == j)
						{
							continue;
						}

						const TSharedPtr<IPropertyHandle> OtherContextMappingPropertyHandle = ParentArrayHandle->GetElement(j);

						TArray<const void*> OtherRawData;
						OtherContextMappingPropertyHandle->AccessRawData(OtherRawData);
						check(OtherRawData.Num() == 1);

						auto OtherContextMappingPtr = static_cast<const FDialogueContextMapping*>(OtherRawData[0]);
						const FString OtherLocalizationKey = (OtherContextMappingPtr) ? DialogueWave->GetContextLocalizationKey(*OtherContextMappingPtr) : FString();

						bIsDuplicate = OurLocalizationKey.Equals(OtherLocalizationKey, ESearchCase::CaseSensitive);
					}
				}

				if (bIsDuplicate)
				{
					NewLocalizationKeyErrorMsg = LOCTEXT("LocKeyDuplicationError", "The localization key for this context is being used on more than one context. Please ensure that each context has a unique localization key.");
				}
			}
		}

		if (!NewLocalizationKeyErrorMsg.ToString().Equals(LocalizationKeyErrorMsg.ToString(), ESearchCase::CaseSensitive))
		{
			// Only set the error once to avoid flickering
			LocalizationKeyErrorMsg = NewLocalizationKeyErrorMsg;
			LocalizationKeyFormatEditableText->SetError(LocalizationKeyErrorMsg);
		}
	}
}

FText FDialogueContextMappingNodeBuilder::LocalizationKey_GetText() const
{
	const TArray<TWeakObjectPtr<UObject>>& RootObjects = DetailLayoutBuilder->GetSelectedObjects();
	if (RootObjects.Num() == 1)
	{
		auto DialogueWave = Cast<const UDialogueWave>(RootObjects[0].Get());
		if (DialogueWave && ContextMappingPropertyHandle->IsValidHandle())
		{
			TArray<const void*> RawData;
			ContextMappingPropertyHandle->AccessRawData(RawData);
			check(RawData.Num() == 1);
			
			auto ContextMappingPtr = static_cast<const FDialogueContextMapping*>(RawData[0]);
			if (ContextMappingPtr)
			{
				return FText::FromString(DialogueWave->GetContextLocalizationKey(*ContextMappingPtr));
			}
		}
	}

	return FText::GetEmpty();
}

TSharedRef<IDetailCustomization> FDialogueWaveDetails::MakeInstance()
{
	return MakeShareable(new FDialogueWaveDetails);
}

void FDialogueWaveDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;

	IDetailCategoryBuilder& ContextMappingsDetailCategoryBuilder = DetailBuilder.EditCategory("DialogueContexts", FText::GetEmpty(), ECategoryPriority::Important);

	// Add Context Button
	ContextMappingsDetailCategoryBuilder.AddCustomRow(LOCTEXT("AddDialogueContext", "Add Dialogue Context"))
	[
		SNew(SBox)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("AddDialogueContext", "Add Dialogue Context"))
			.ToolTipText(LOCTEXT("AddDialogueContextToolTip", "Adds a new context for dialogue based on speakers, those spoken to, and the associated soundwave."))
			.OnClicked(FOnClicked::CreateSP(this, &FDialogueWaveDetails::AddDialogueContextMapping_OnClicked))
		]
	];

	// Individual Context Mappings
	const TSharedPtr<IPropertyHandle> ContextMappingsPropertyHandle = DetailLayoutBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UDialogueWave, ContextMappings), UDialogueWave::StaticClass());
	ContextMappingsPropertyHandle->MarkHiddenByCustomization();

	const TSharedPtr<IPropertyHandleArray> ContextMappingsPropertyArrayHandle = ContextMappingsPropertyHandle->AsArray();

	uint32 DialogueContextMappingCount;
	ContextMappingsPropertyArrayHandle->GetNumElements(DialogueContextMappingCount);
	for (uint32 j = 0; j < DialogueContextMappingCount; ++j)
	{
		const TSharedPtr<IPropertyHandle> ChildContextMappingPropertyHandle = ContextMappingsPropertyArrayHandle->GetElement(j);

		const TSharedRef<FDialogueContextMappingNodeBuilder> DialogueContextMapping = MakeShareable(new FDialogueContextMappingNodeBuilder(DetailLayoutBuilder, ChildContextMappingPropertyHandle));
		ContextMappingsDetailCategoryBuilder.AddCustomBuilder(DialogueContextMapping);
	}
}

FReply FDialogueWaveDetails::AddDialogueContextMapping_OnClicked()
{
	const TSharedPtr<IPropertyHandle> ContextMappingsPropertyHandle = DetailLayoutBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UDialogueWave, ContextMappings), UDialogueWave::StaticClass());
	const TSharedPtr<IPropertyHandleArray> ContextMappingsPropertyArrayHandle = ContextMappingsPropertyHandle->AsArray();
	ContextMappingsPropertyArrayHandle->AddItem();

	DetailLayoutBuilder->ForceRefreshDetails();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
