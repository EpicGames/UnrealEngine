// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;

class FDialogueContextMappingNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FDialogueContextMappingNodeBuilder>
{
public:
	FDialogueContextMappingNodeBuilder(IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle);

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { OnRebuildChildren = InOnRebuildChildren; }
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick(float DeltaTime) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual bool InitiallyCollapsed() const override { return true; }
	virtual FName GetName() const override { return NAME_None; }

private:
	void RemoveContextButton_OnClick();

	FText LocalizationKeyFormatEditableText_GetText() const;
	void LocalizationKeyFormatEditableText_OnTextCommitted(const FText& InNewText, ETextCommit::Type InCommitType);
	bool LocalizationKeyFormatEditableText_IsReadyOnly() const;
	void LocalizationKeyFormatEditableText_UpdateErrorText();

	FText LocalizationKey_GetText() const;

private:
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Associated detail layout builder */
	IDetailLayoutBuilder* DetailLayoutBuilder;

	/** Property handle to associated context mapping */
	TSharedPtr<IPropertyHandle> ContextMappingPropertyHandle;

	/** Property handle to the localization key format property within this context mapping */
	TSharedPtr<IPropertyHandle> LocalizationKeyFormatPropertyHandle;

	/** Pointer to the editable text box used to edit the localization key format string */
	TSharedPtr<SEditableTextBox> LocalizationKeyFormatEditableText;

	/** Timestamp the last time the error information for the localization key was updated */
	double LastLocalizationKeyErrorUpdateTimestamp;

	/** The error message that the localized key is currently showing */
	FText LocalizationKeyErrorMsg;
};

class FDialogueWaveDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;

private:
	FReply AddDialogueContextMapping_OnClicked();

private:
	/** Associated detail layout builder */
	IDetailLayoutBuilder* DetailLayoutBuilder;
};
