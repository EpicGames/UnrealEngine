// Copyright Epic Games, Inc. All Rights Reserved.

#include "DialogueWaveWidgets.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetThumbnail.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "SAssetDropTarget.h"
#include "Selection.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Sound/DialogueVoice.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Notifications/SErrorHint.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

class FDragDropEvent;
class UClass;

#define LOCTEXT_NAMESPACE "DialogueWaveDetails"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDialogueVoicePropertyEditor::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	DialogueVoicePropertyHandle = InPropertyHandle;
	AssetThumbnailPool = InAssetThumbnailPool;
	IsEditable = InArgs._IsEditable;
	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;

	if( DialogueVoicePropertyHandle->IsValidHandle() )
	{
		const UDialogueVoice* DialogueVoice = NULL;
		if( DialogueVoicePropertyHandle->IsValidHandle() )
		{
			UObject* Object = NULL;
			DialogueVoicePropertyHandle->GetValue(Object);
			DialogueVoice = Cast<UDialogueVoice>(Object);
		}

		const uint32 ThumbnailSizeX = 64;
		const uint32 ThumbnailSizeY = 64;
		AssetThumbnail = MakeShareable( new FAssetThumbnail( DialogueVoice, ThumbnailSizeX, ThumbnailSizeY, AssetThumbnailPool ) );

		TSharedRef<SWidget> AssetWidget =
			SNew( SAssetDropTarget )
			.ToolTipText( this, &SDialogueVoicePropertyEditor::OnGetToolTip )
			.OnAreAssetsAcceptableForDrop( this, &SDialogueVoicePropertyEditor::OnIsAssetAcceptableForDrop )
			.OnAssetsDropped( this, &SDialogueVoicePropertyEditor::OnAssetDropped )
			[
				SNew( SBox )
				.WidthOverride( static_cast<float>(ThumbnailSizeX) ) 
				.HeightOverride( static_cast<float>(ThumbnailSizeY) )
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(0.0f)
				[
					AssetThumbnail->MakeThumbnailWidget()
				]
			];

		if(IsEditable)
		{
			const TSharedRef<SWidget> UseButton = PropertyCustomizationHelpers::MakeUseSelectedButton( FSimpleDelegate::CreateSP( this, &SDialogueVoicePropertyEditor::OnUseSelectedDialogueVoice ) );
			UseButton->SetEnabled( TAttribute<bool>::Create( TAttribute<bool>::FGetter::CreateSP( this, &SDialogueVoicePropertyEditor::CanUseSelectedAsset) ) );

			const TSharedRef<SWidget> BrowseButton = PropertyCustomizationHelpers::MakeBrowseButton( FSimpleDelegate::CreateSP( this, &SDialogueVoicePropertyEditor::OnBrowseToDialogueVoice ) );
			BrowseButton->SetEnabled( TAttribute<bool>::Create( TAttribute<bool>::FGetter::CreateSP( this, &SDialogueVoicePropertyEditor::CanBrowseToAsset) ) );

			TSharedRef<SWidget> ButtonsColumnWidget =
				SNew( SVerticalBox )
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(1.0f)
				.AutoHeight()
				[
					UseButton
				]
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(1.0f)
				.AutoHeight()
				[
					BrowseButton
				];

			struct Local
			{
				static FVector2D GetDesiredSize(TSharedRef<SWidget> Widget)
				{
					return Widget->GetDesiredSize();
				}
			};

			TSharedRef<SHorizontalBox> HorizontalBox = SNew( SHorizontalBox );

			if(InArgs._ShouldCenterThumbnail)
			{
				HorizontalBox->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				.FillWidth(1.0f)
				[
					SNew( SSpacer )
					.Size_Static( &Local::GetDesiredSize, ButtonsColumnWidget )
				];
			}

			// Thumbnail
			HorizontalBox->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew( ComboButton, SComboButton )
					.ToolTipText( this, &SDialogueVoicePropertyEditor::OnGetToolTip )
					.ButtonStyle( FAppStyle::Get(), "PropertyEditor.AssetComboStyle" )
					.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
					.OnGetMenuContent( this, &SDialogueVoicePropertyEditor::OnGetMenuContent )
					.ContentPadding(2.0f)
					.ButtonContent()
					[
						AssetWidget
					]
				];

			// Path Property Buttons
			if(InArgs._ShouldCenterThumbnail)
			{
				HorizontalBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					// Redundant Horizontal Box Slot exists to AutoWidth the contents here - avoids squishing nested button images.
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.Padding(0.0f) // Don't influence positioning - we're only here to correct sizing.
					.AutoWidth()
					[
						ButtonsColumnWidget
					]
				];
			}
			else
			{
				HorizontalBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					ButtonsColumnWidget
				];
			}

			ChildSlot
			.VAlign(VAlign_Fill)
			[
				HorizontalBox
			];
		}
		else
		{
			ChildSlot
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				AssetWidget
			];
		}
	}
}

void SDialogueVoicePropertyEditor::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	UDialogueVoice* CurrentDialogueVoice = NULL;
	if( DialogueVoicePropertyHandle->IsValidHandle() )
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		CurrentDialogueVoice = Cast<UDialogueVoice>(Object);
	}

	if( AssetThumbnail->GetAsset() != CurrentDialogueVoice )
	{
		AssetThumbnail->SetAsset( CurrentDialogueVoice );
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SDialogueVoicePropertyEditor::OnGetToolTip() const
{
	FText ToolTipText;

	const FAssetData& AssetData = AssetThumbnail->GetAssetData();
	if( AssetData.IsValid() )
	{
		
		ToolTipText = FText::FromName(AssetData.PackageName);
	}

	return ToolTipText;
}

TSharedRef<SWidget> SDialogueVoicePropertyEditor::OnGetMenuContent()
{
	TArray<const UClass*> AllowedClasses;
	AllowedClasses.Add(UDialogueVoice::StaticClass());

	UDialogueVoice* DialogueVoice = NULL;
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		DialogueVoice = Cast<UDialogueVoice>(Object);
	}

	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
		DialogueVoice,
		false,
		AllowedClasses,
		PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
		OnShouldFilterAsset,
		FOnAssetSelected::CreateSP(this, &SDialogueVoicePropertyEditor::OnAssetSelectedFromPicker),
		FSimpleDelegate::CreateSP(this, &SDialogueVoicePropertyEditor::CloseMenu));
}

void SDialogueVoicePropertyEditor::CloseMenu()
{
	ComboButton->SetIsOpen(false);
}

bool SDialogueVoicePropertyEditor::OnIsAssetAcceptableForDrop( TArrayView<FAssetData> InAssets ) const
{
	// Only dialogue voice can be dropped 
	return Cast<UDialogueVoice>( InAssets[0].GetAsset() ) != nullptr;
}

void SDialogueVoicePropertyEditor::OnAssetDropped( const FDragDropEvent&, TArrayView<FAssetData> InAssets )
{
	ReplaceDialogueVoice( CastChecked<UDialogueVoice>(InAssets[0].GetAsset()) );
}

FText SDialogueVoicePropertyEditor::GetDialogueVoiceDescription() const
{
	UDialogueVoice* DialogueVoice = NULL;
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		DialogueVoice = Cast<UDialogueVoice>(Object);
	}

	return DialogueVoice ? FText::FromString(DialogueVoice->GetDesc()) : LOCTEXT("None", "None");
}

bool SDialogueVoicePropertyEditor::CanUseSelectedAsset()
{
	bool Result = false;

	// Load selected assets
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	// Get the first dialogue voice selected
	USelection* DialogueVoiceSelection = GEditor->GetSelectedObjects();
	if (DialogueVoiceSelection && DialogueVoiceSelection->Num() == 1)
	{
		UDialogueVoice* DialogueVoiceToAssign = DialogueVoiceSelection->GetTop<UDialogueVoice>();
		if( DialogueVoiceToAssign && ( !OnShouldFilterAsset.IsBound() || !OnShouldFilterAsset.Execute(DialogueVoiceToAssign) ) )
		{
			Result = true;
		}
	}

	return Result;
}

void SDialogueVoicePropertyEditor::OnUseSelectedDialogueVoice()
{
	// Load selected assets
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	// Get the first dialogue voice selected
	USelection* DialogueVoiceSelection = GEditor->GetSelectedObjects();
	if (DialogueVoiceSelection && DialogueVoiceSelection->Num() == 1)
	{
		UDialogueVoice* DialogueVoiceToAssign = DialogueVoiceSelection->GetTop<UDialogueVoice>();
		if( DialogueVoiceToAssign )
		{
			ReplaceDialogueVoice( DialogueVoiceToAssign );
		}
	}
}

void SDialogueVoicePropertyEditor::ReplaceDialogueVoice( const UDialogueVoice* const NewDialogueVoice )
{
	if( !OnShouldFilterAsset.IsBound() || !OnShouldFilterAsset.Execute(NewDialogueVoice) )
	{
		const UDialogueVoice* PrevDialogueVoice = NULL;

		const UDialogueVoice* DialogueVoice = NULL;
		{
			UObject* Object = NULL;
			DialogueVoicePropertyHandle->GetValue(Object);
			DialogueVoice = Cast<UDialogueVoice>(Object);
		}

		if( DialogueVoice )
		{
			PrevDialogueVoice = DialogueVoice;
		}

		if( NewDialogueVoice != PrevDialogueVoice )
		{
			// Replace the dialogue voice
			DialogueVoicePropertyHandle->SetValue( NewDialogueVoice );
		}
	}
}

bool SDialogueVoicePropertyEditor::CanBrowseToAsset()
{
	const UDialogueVoice* DialogueVoice = NULL;
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		DialogueVoice = Cast<UDialogueVoice>(Object);
	}

	return DialogueVoice != NULL;
}

void SDialogueVoicePropertyEditor::OnBrowseToDialogueVoice()
{
	UDialogueVoice* DialogueVoice = NULL;
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		DialogueVoice = Cast<UDialogueVoice>(Object);
	}

	if( DialogueVoice )
	{	
		// Find the item in the content browser
		GoToAssetInContentBrowser(MakeWeakObjectPtr(DialogueVoice));
	}
}

void SDialogueVoicePropertyEditor::OnAssetSelectedFromPicker( const FAssetData& InAssetData )
{
	UDialogueVoice* NewDialogueVoice = Cast<UDialogueVoice>(InAssetData.GetAsset());
	if( NewDialogueVoice )
	{
		ReplaceDialogueVoice( NewDialogueVoice );
	}
}

/**
	* Called to get the dialogue voice path that should be displayed
	*/
FText SDialogueVoicePropertyEditor::GetDialogueVoicePath() const
{
	const UDialogueVoice* EdittingDialogueVoice = NULL;
	{
		UObject* Object = NULL;
		DialogueVoicePropertyHandle->GetValue(Object);
		EdittingDialogueVoice = Cast<UDialogueVoice>(Object);
	}

	if( EdittingDialogueVoice )
	{
		return FText::FromString( EdittingDialogueVoice->GetOuter()->GetPathName() );
	}
	else
	{
		return FText::GetEmpty();		
	}
}

/**
 * Called when the dialogue voice path is changed by a user
 */
void SDialogueVoicePropertyEditor::OnDialogueVoicePathChanged( const FText& NewText, ETextCommit::Type TextCommitType )
{
	const FString& NewString = NewText.ToString();

	if( !NewText.EqualTo( GetDialogueVoicePath() ) && NewString.Len() < NAME_SIZE )
	{	
		const UDialogueVoice* PrevDialogueVoice = NULL;

		const UDialogueVoice* DialogueVoice = NULL;
		{
			UObject* Object = NULL;
			DialogueVoicePropertyHandle->GetValue(Object);
			DialogueVoice = Cast<UDialogueVoice>(Object);
		}

		if( DialogueVoice )
		{
			PrevDialogueVoice = DialogueVoice;
		}

		const UDialogueVoice* DialogueVoiceToAssign = NULL;
			
		if( !NewString.IsEmpty() )
		{
			if( NewString.Contains( TEXT(".") ) )
			{
				// Formatted text string, use the exact path instead of any package
				DialogueVoiceToAssign = FindObject<UDialogueVoice>(nullptr, *NewString);
			}
			else
			{
				DialogueVoiceToAssign = FindFirstObject<UDialogueVoice>(*NewString, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
			}
	
			if( !DialogueVoiceToAssign )
			{
				DialogueVoiceToAssign = Cast<UDialogueVoice>( StaticLoadObject( UDialogueVoice::StaticClass(), NULL, *NewString ) );
			}

			if( !DialogueVoiceToAssign )
			{
				// If we still don't have the dialogue voice attempt to find it via the asset registry
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>( "AssetRegistry" );

				// Collect a full list of assets with the specified class
				TArray<FAssetData> AssetData;
				
				AssetRegistryModule.Get().GetAssetsByPackageName( FName(*NewString), AssetData );

				if( AssetData.Num() > 0 )
				{
					// There should really only be one dialogue voice found
					DialogueVoiceToAssign = Cast<UDialogueVoice>( AssetData[0].GetAsset() );
				}
			}
		}

		if( NewString.IsEmpty() || DialogueVoiceToAssign )
		{
			ReplaceDialogueVoice( DialogueVoiceToAssign );
		}
	}
}

void SDialogueVoicePropertyEditor::GoToAssetInContentBrowser( TWeakObjectPtr<UObject> Object )
{
	if( Object.IsValid() )
	{
		TArray< UObject* > Objects;
		Objects.Add( Object.Get() );
		GEditor->SyncBrowserToObjects( Objects );
	}
}

void STargetsSummaryWidget::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	TargetsPropertyHandle = InPropertyHandle;
	AssetThumbnailPool = InAssetThumbnailPool;
	IsEditable = InArgs._IsEditable;
	WrapWidth = InArgs._WrapWidth;

	AllottedWidth = 0.0f;

	GenerateContent();
}

float STargetsSummaryWidget::GetPreferredWidthForWrapping() const
{
	return WrapWidth.IsBound() ? WrapWidth.Get() : AllottedWidth;
}

void STargetsSummaryWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	AllottedWidth = AllottedGeometry.Size.X;

	if( TargetsPropertyHandle->IsValidHandle() )
	{
		uint32 TargetCount;
		TargetsPropertyHandle->GetNumChildren(TargetCount);

		if( TargetCount != DisplayedTargets.Num() )
		{
			// The array sizes differ so we need to refresh the list
			GenerateContent();
		}
	}
}

FText STargetsSummaryWidget::GetDialogueVoiceDescription() const
{
	FText Result = LOCTEXT( "NoTargets", "No One" );

	if( TargetsPropertyHandle->IsValidHandle() )
	{
		uint32 TargetCount;
		TargetsPropertyHandle->GetNumChildren(TargetCount);

		if( TargetCount > 1 )
		{
			Result = LOCTEXT("Multiple", "Multiple");
		}
		else if( TargetCount == 1 )
		{
			const TSharedPtr<IPropertyHandle> SingleTargetPropertyHandle = TargetsPropertyHandle->GetChildHandle(0);

			UDialogueVoice* DialogueVoice = NULL;
			if( SingleTargetPropertyHandle->IsValidHandle() )
			{
				UObject* Object = NULL;
				SingleTargetPropertyHandle->GetValue(Object);
				DialogueVoice = Cast<UDialogueVoice>(Object);
			}

			Result = DialogueVoice ? FText::FromString(DialogueVoice->GetDesc()) : LOCTEXT("None", "None");
		}
	}

	return Result;
}

bool STargetsSummaryWidget::FilterTargets( const struct FAssetData& InAssetData )
{
	bool ShouldAssetBeFilteredOut = false;

	if( TargetsPropertyHandle->IsValidHandle() )
	{
		uint32 TargetCount;
		TargetsPropertyHandle->GetNumChildren(TargetCount);

		// Show tiles only.
		for(uint32 i = 0; i < TargetCount; ++i)
		{
			const TSharedPtr<IPropertyHandle> TargetPropertyHandle = TargetsPropertyHandle->GetChildHandle(i);

			const UDialogueVoice* DialogueVoice = NULL;
			if( TargetPropertyHandle->IsValidHandle() )
			{
				UObject* Object = NULL;
				TargetPropertyHandle->GetValue(Object);
				DialogueVoice = Cast<UDialogueVoice>(Object);
			}

			if( DialogueVoice == InAssetData.GetAsset() )
			{
				ShouldAssetBeFilteredOut = true;
				break;
			}
		}
	}

	return ShouldAssetBeFilteredOut;
}

class SRemovableDialogueVoicePropertyEditor : public SDialogueVoicePropertyEditor
{
public:
	SRemovableDialogueVoicePropertyEditor()
		: bIsPressed(false)
	{

	}
protected:
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:
	void DoRemove();

private:
	bool bIsPressed;
};

FReply SRemovableDialogueVoicePropertyEditor::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();

	if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
	{
		bIsPressed = true;

		//we need to capture the mouse for MouseUp events
		Reply = FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), EFocusCause::Mouse);
	}

	//return the constructed reply
	return Reply;
}

FReply SRemovableDialogueVoicePropertyEditor::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseButtonDown( InMyGeometry, InMouseEvent );
}

FReply SRemovableDialogueVoicePropertyEditor::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();
	if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton )
	{
		bIsPressed = false;

		const bool bIsUnderMouse = MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());
		if( bIsUnderMouse )
		{
			// If we were asked to allow the button to be clicked on mouse up, regardless of whether the user
			// pressed the button down first, then we'll allow the click to proceed without an active capture
			if( HasMouseCapture() )
			{
				DoRemove();
				Reply = FReply::Handled();
			}
		}

		//If the user of the button didn't handle this click, then the button's
		//default behavior handles it.
		if(Reply.IsEventHandled() == false)
		{
			Reply = FReply::Handled();
		}

		//If the user hasn't requested a new mouse captor, then the default
		//behavior of the button is to release mouse capture.
		if(Reply.GetMouseCaptor().IsValid() == false)
		{
			Reply.ReleaseMouseCapture();
		}
	}

	return Reply;
}

void SRemovableDialogueVoicePropertyEditor::DoRemove()
{
	if( IsEditable )
	{
		const TSharedPtr<IPropertyHandle> ParentPropertyHandle = DialogueVoicePropertyHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentPropertyArrayHandle = ParentPropertyHandle->AsArray();
		ParentPropertyArrayHandle->DeleteItem( DialogueVoicePropertyHandle->GetIndexInArray() );
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STargetsSummaryWidget::GenerateContent()
{
	if( TargetsPropertyHandle->IsValidHandle() )
	{
		DisplayedTargets.Empty();

		uint32 TargetCount;
		TargetsPropertyHandle->GetNumChildren(TargetCount);

		FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();

		if( TargetCount > 1 )
		{
			const TSharedRef<SWrapBox> WrapBox =
				SNew( SWrapBox )
				.PreferredSize(this, &STargetsSummaryWidget::GetPreferredWidthForWrapping);

			// Show tiles only.
			for(uint32 i = 0; i < TargetCount; ++i)
			{
				const TSharedPtr<IPropertyHandle> TargetPropertyHandle = TargetsPropertyHandle->GetChildHandle(i);

				const UDialogueVoice* DialogueVoice = NULL;
				if( TargetPropertyHandle->IsValidHandle() )
				{
					UObject* Object = NULL;
					TargetPropertyHandle->GetValue(Object);
					DialogueVoice = Cast<UDialogueVoice>(Object);
				}

				DisplayedTargets.Add(DialogueVoice);


				WrapBox->AddSlot()
				.Padding(2.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew( SRemovableDialogueVoicePropertyEditor, TargetPropertyHandle.ToSharedRef(), AssetThumbnailPool.ToSharedRef() )
					.IsEditable( IsEditable )
					.OnShouldFilterAsset( this, &STargetsSummaryWidget::FilterTargets)
				];
			}

			ChildSlot
			.HAlign(HAlign_Center)
			[
				WrapBox
			];
		}
		else if( TargetCount == 1 )
		{
			const TSharedPtr<IPropertyHandle> SingleTargetPropertyHandle = TargetsPropertyHandle->GetChildHandle(0);

			const UDialogueVoice* DialogueVoice = NULL;
			if( SingleTargetPropertyHandle->IsValidHandle() )
			{
				UObject* Object = NULL;
				SingleTargetPropertyHandle->GetValue(Object);
				DialogueVoice = Cast<UDialogueVoice>(Object);
			}

			DisplayedTargets.Add(DialogueVoice);

			TSharedRef<SWidget> TargetPropertyEditor =
				SNew( SRemovableDialogueVoicePropertyEditor, SingleTargetPropertyHandle.ToSharedRef(), AssetThumbnailPool.ToSharedRef() )
				.IsEditable( IsEditable )
				.OnShouldFilterAsset( this, &STargetsSummaryWidget::FilterTargets)
				.ShouldCenterThumbnail(true);

			ChildSlot
			.HAlign(HAlign_Center)
			[
				SNew( SBox )
				.Padding(2.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					TargetPropertyEditor
				]
			];
		}
		else
		{
			const float ThumbnailSizeX = 64.0f;
			const float ThumbnailSizeY = 64.0f;

			ChildSlot
			.HAlign(HAlign_Center)
			[
				SNew( SBox )
				.Padding(2.0f)
				.WidthOverride( ThumbnailSizeX )
				.HeightOverride( ThumbnailSizeY )
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			];
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDialogueContextHeaderWidget::Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool )
{
	ContextPropertyHandle = InPropertyHandle;
	if( ContextPropertyHandle->IsValidHandle() )
	{
		FSlateFontInfo Font = IDetailLayoutBuilder::GetDetailFont();

		const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");

		TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton( FSimpleDelegate::CreateSP( this, &SDialogueContextHeaderWidget::AddTargetButton_OnClick ) );
		TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeRemoveButton( FSimpleDelegate::CreateSP( this, &SDialogueContextHeaderWidget::RemoveTargetButton_OnClick ) );
		TSharedRef<SWidget> EmptyButton = PropertyCustomizationHelpers::MakeEmptyButton( FSimpleDelegate::CreateSP( this, &SDialogueContextHeaderWidget::EmptyTargetsButton_OnClick ) );

		TSharedRef<SDialogueVoicePropertyEditor> SpeakerPropertyEditor =
			SNew( SDialogueVoicePropertyEditor, SpeakerPropertyHandle.ToSharedRef(), InAssetThumbnailPool )
			.IsEditable(true)
			.ShouldCenterThumbnail(true);

		TSharedRef<STargetsSummaryWidget> TargetsSummaryWidget =
			SNew( STargetsSummaryWidget, TargetsPropertyHandle.ToSharedRef(), InAssetThumbnailPool );

		ChildSlot
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SAssignNew( SpeakerErrorHint, SErrorHint )
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SpeakerPropertyHandle->CreatePropertyNameWidget()
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNullWidget::NullWidget
					]
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.FillHeight(1.0f)
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Fill)
					.AutoHeight()
					[
						SpeakerPropertyEditor
					]
					+SVerticalBox::Slot()
					.Padding( 2.0f )
					.HAlign(HAlign_Center)
					.AutoHeight()
					[
						// Voice Description
						SNew( STextBlock )
						.Font( Font )
						.Text( SpeakerPropertyEditor, &SDialogueVoicePropertyEditor::GetDialogueVoiceDescription )
					]
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew( SImage )
				.Image( FAppStyle::GetBrush("DialogueWaveDetails.SpeakerToTarget") )
				.ColorAndOpacity( FSlateColor::UseForeground() )
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SHorizontalBox )
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SAssignNew( TargetsErrorHint, SErrorHint )
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							TargetsPropertyHandle->CreatePropertyNameWidget( LOCTEXT("DirectedAt", "Directed At") )
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( SHorizontalBox )
							+SHorizontalBox::Slot()
							.Padding(1.0f)
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								AddButton
							]
							+SHorizontalBox::Slot()
							.Padding(1.0f)
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								RemoveButton
							]
							+SHorizontalBox::Slot()
							.Padding(1.0f)
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								EmptyButton
							]
						]
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNullWidget::NullWidget
					]
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.FillHeight(1.0f)
				[
					SNew( SVerticalBox )
					+SVerticalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Fill)
					.AutoHeight()
					[
						TargetsSummaryWidget
					]
					+SVerticalBox::Slot()
					.Padding( 2.0f )
					.HAlign(HAlign_Center)
					.AutoHeight()
					[
						// Voice Description
						SNew( STextBlock )
						.Font( Font )
						.Text( TargetsSummaryWidget, &STargetsSummaryWidget::GetDialogueVoiceDescription )
					]
				]
			]
		];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SDialogueContextHeaderWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( !IsSpeakerValid() )
	{
		if( SpeakerErrorHint.IsValid() ) { SpeakerErrorHint->SetError( LOCTEXT("NullSpeakerError", "Speaker can not be \"None\".") ); }
	}
	else
	{
		if( SpeakerErrorHint.IsValid() ) { SpeakerErrorHint->SetError( FText::GetEmpty() ); }
	}

	if( !IsTargetSetValid() )
	{
		if( TargetsErrorHint.IsValid() ) { TargetsErrorHint->SetError( LOCTEXT("NullTargetError", "Target set can not contain \"None\".") ); }
	}
	else
	{
		if( TargetsErrorHint.IsValid() ) { TargetsErrorHint->SetError( FText::GetEmpty() ); }
	}
}

bool SDialogueContextHeaderWidget::IsSpeakerValid() const
{
	bool Result = false;

	if( ContextPropertyHandle.IsValid() && ContextPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> SpeakerPropertyHandle = ContextPropertyHandle->GetChildHandle("Speaker");

		const UDialogueVoice* Speaker = nullptr;
		if( SpeakerPropertyHandle.IsValid() && SpeakerPropertyHandle->IsValidHandle() )
		{
			UObject* Object = nullptr;
			SpeakerPropertyHandle->GetValue(Object);
			Speaker = Cast<UDialogueVoice>(Object);
		}

		Result = ( Speaker != nullptr );
	}

	return Result;
}

bool SDialogueContextHeaderWidget::IsTargetSetValid() const
{
	bool Result = false;

	if( ContextPropertyHandle.IsValid() && ContextPropertyHandle->IsValidHandle() )
	{
		Result = true;

		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
		const TSharedPtr<IPropertyHandleArray> TargetsArrayPropertyHandle = TargetsPropertyHandle->AsArray();

		uint32 TargetCount;
		TargetsArrayPropertyHandle->GetNumElements(TargetCount);

		for(uint32 i = 0; i < TargetCount; ++i)
		{
			TSharedPtr<IPropertyHandle> TargetPropertyHandle = TargetsArrayPropertyHandle->GetElement(i);

			const UDialogueVoice* Target = nullptr;
			if( TargetPropertyHandle.IsValid() && TargetPropertyHandle->IsValidHandle() )
			{
				UObject* Object = nullptr;
				TargetPropertyHandle->GetValue(Object);
				Target = Cast<UDialogueVoice>(Object);
			}

			if ( Target == nullptr )
			{
				Result = false;
				break;
			}
		}
	}
	
	return Result;
}

void SDialogueContextHeaderWidget::AddTargetButton_OnClick()
{
	if( ContextPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
		const TSharedPtr<IPropertyHandleArray> TargetsArrayPropertyHandle = TargetsPropertyHandle->AsArray();

		TargetsArrayPropertyHandle->AddItem();
	}
}

void SDialogueContextHeaderWidget::RemoveTargetButton_OnClick()
{
	if( ContextPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
		const TSharedPtr<IPropertyHandleArray> TargetsArrayPropertyHandle = TargetsPropertyHandle->AsArray();

		uint32 TargetCount;
		TargetsArrayPropertyHandle->GetNumElements(TargetCount);
		if( TargetCount > 0 )
		{
			TargetsArrayPropertyHandle->DeleteItem( TargetCount - 1 );
		}
	}
}

void SDialogueContextHeaderWidget::EmptyTargetsButton_OnClick()
{
	if( ContextPropertyHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> TargetsPropertyHandle = ContextPropertyHandle->GetChildHandle("Targets");
		const TSharedPtr<IPropertyHandleArray> TargetsArrayPropertyHandle = TargetsPropertyHandle->AsArray();

		TargetsArrayPropertyHandle->EmptyArray();
	}
}

#undef LOCTEXT_NAMESPACE
