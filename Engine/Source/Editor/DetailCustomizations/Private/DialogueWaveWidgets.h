// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FAssetThumbnail;
class FAssetThumbnailPool;
class FDragDropEvent;
class IPropertyHandle;
class SComboButton;
class SErrorHint;
class SWidget;
class UClass;
class UDialogueVoice;
class UObject;
struct FAssetData;
struct FGeometry;

class SDialogueVoicePropertyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDialogueVoicePropertyEditor )
		: _IsEditable(true)
		, _ShouldCenterThumbnail(false)
		{}
		SLATE_ARGUMENT( bool, IsEditable )
		SLATE_ARGUMENT( bool, ShouldCenterThumbnail )
		SLATE_EVENT( FOnShouldFilterAsset, OnShouldFilterAsset )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	FText GetDialogueVoiceDescription() const;

private:
	FText OnGetToolTip() const;
	TSharedRef<SWidget> OnGetMenuContent();
	void CloseMenu();

	/**
	 * Called when an asset is being dragged over this item
	 *
	 * @param InObject	The asset being dragged over
	 */
	bool OnIsAssetAcceptableForDrop( TArrayView<FAssetData> InAssets ) const;

	/**
	 * Called when an asset dropped onto the list item
	 *
	 * @param InObject	The asset being dropped
	 */
	void OnAssetDropped( const FDragDropEvent&, TArrayView<FAssetData> InAssets );

	bool CanUseSelectedAsset();
	void OnUseSelectedDialogueVoice();

	void ReplaceDialogueVoice( const UDialogueVoice* const NewDialogueVoice );

	bool CanBrowseToAsset();
	void OnBrowseToDialogueVoice();

	void OnGetAllowedClassesForAssetPicker( TArray<const UClass*>& OutClasses );

	void OnAssetSelectedFromPicker( const FAssetData& InAssetData );

	/**
	 * Called to get the dialogue voice path that should be displayed
	 */
	FText GetDialogueVoicePath() const;

	/**
	 * Called when the dialogue voice path is changed by a user
	 */
	void OnDialogueVoicePathChanged( const FText& NewText, ETextCommit::Type TextCommitType );

	/**
	 * Finds the asset in the content browser
	 */
	void GoToAssetInContentBrowser( TWeakObjectPtr<UObject> Object );

protected:
	TSharedPtr<IPropertyHandle> DialogueVoicePropertyHandle;
	bool IsEditable;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
	TSharedPtr<SComboButton> ComboButton;
	FOnShouldFilterAsset OnShouldFilterAsset;
};

class STargetsSummaryWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STargetsSummaryWidget )
		: _IsEditable(true)
		, _WrapWidth(0.0f)
		{}
		SLATE_ARGUMENT( bool, IsEditable )
		SLATE_ATTRIBUTE( float, WrapWidth )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	FText GetDialogueVoiceDescription() const;

private:
	float GetPreferredWidthForWrapping() const;
	bool FilterTargets( const struct FAssetData& InAssetData );
	void GenerateContent();

private:
	TSharedPtr<IPropertyHandle> TargetsPropertyHandle;
	bool IsEditable;
	TAttribute<float> WrapWidth;
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
	TArray<const UDialogueVoice*> DisplayedTargets;
	float AllottedWidth;
};

class SDialogueContextHeaderWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDialogueContextHeaderWidget ) {}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedRef<FAssetThumbnailPool>& InAssetThumbnailPool );
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:
	bool IsSpeakerValid() const;
	bool IsTargetSetValid() const;
	void AddTargetButton_OnClick();
	void RemoveTargetButton_OnClick();
	void EmptyTargetsButton_OnClick();

private:
	TSharedPtr<IPropertyHandle> ContextPropertyHandle;

	TSharedPtr<SErrorHint> SpeakerErrorHint;
	TSharedPtr<SErrorHint> TargetsErrorHint;
};
