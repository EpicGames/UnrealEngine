// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectionalLightComponentDetails.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/SceneComponent.h"
#include "RenderUtils.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "DirectionalLightComponentDetails"


TSharedRef<IDetailCustomization> FDirectionalLightComponentDetails::MakeInstance()
{
	return MakeShareable( new FDirectionalLightComponentDetails );
}

void FDirectionalLightComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TSharedPtr<IPropertyHandle> MovableShadowRadiusPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceMovableLight));
	TSharedPtr<IPropertyHandle> StationaryShadowRadiusPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDirectionalLightComponent, DynamicShadowDistanceStationaryLight));

	if(!IsStaticLightingAllowed())
	{
		// If static lighting is not allowed, hide DynamicShadowDistanceStationaryLight and rename DynamicShadowDistanceMovableLight to "Dynamic Shadow Distance"
		MovableShadowRadiusPropertyHandle->SetPropertyDisplayName(LOCTEXT("DynamicShadowDistanceDisplayName", "Dynamic Shadow Distance"));
		StationaryShadowRadiusPropertyHandle->MarkHiddenByCustomization();
	}

	TSharedPtr<IPropertyHandle> LightIntensityProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponentBase, Intensity), ULightComponentBase::StaticClass());
	// Point lights need to override the ui min and max for units of lumens, so we have to undo that
	LightIntensityProperty->SetInstanceMetaData("UIMin",TEXT("0.0f"));
	LightIntensityProperty->SetInstanceMetaData("UIMax",TEXT("150.0f"));
	LightIntensityProperty->SetInstanceMetaData("SliderExponent", TEXT("2.0f"));
	LightIntensityProperty->SetInstanceMetaData("Units", TEXT("lux"));
	LightIntensityProperty->SetToolTipText(LOCTEXT("DirectionalLightIntensityToolTipText", "Maximum illumination from the light in lux"));

}

#undef LOCTEXT_NAMESPACE
