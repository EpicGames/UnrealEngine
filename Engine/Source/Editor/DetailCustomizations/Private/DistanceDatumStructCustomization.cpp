// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistanceDatumStructCustomization.h"

#include "Containers/Array.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "Sound/SoundNodeDistanceCrossFade.h"
#include "Sound/SoundNodeParamCrossFade.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "DistanceDatumStructCustomization"

TSharedRef<IPropertyTypeCustomization> FDistanceDatumStructCustomization::MakeInstance()
{
	return MakeShared<FDistanceDatumStructCustomization>(FDistanceDatumStructCustomization::FPrivateToken());
}

FDistanceDatumStructCustomization::~FDistanceDatumStructCustomization()
{
}

FDistanceDatumStructCustomization::FDistanceDatumStructCustomization(FPrivateToken)
{
}

void FDistanceDatumStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InStructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FDistanceDatumStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	InStructPropertyHandle->GetNumChildren(NumChildren);

	TArray<UObject*> OuterObjects;
	InStructPropertyHandle->GetOuterObjects(OuterObjects);

	// Check whether we are dealing with param cross fade nodes
	bool bAllParamNodes = true;
	for (UObject* OuterObject : OuterObjects)
	{
		if (!OuterObject->IsA(USoundNodeParamCrossFade::StaticClass()))
		{
			bAllParamNodes = false;
		}
	}

	// How to describe the parameter depending on how many are selected
	FText ParamDesc = LOCTEXT("Param", "Param");
	if (bAllParamNodes)
	{
		if (OuterObjects.Num() == 1)
		{
			FName ParamName = CastChecked<USoundNodeParamCrossFade>(OuterObjects[0])->ParamName;
			if (ParamName != NAME_None)
			{
				ParamDesc = FText::FromName(ParamName);
			}
		}
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> Child = InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		IDetailPropertyRow& ChildRow = StructBuilder.AddProperty(Child);

		if (bAllParamNodes)
		{
			if (Child->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FDistanceDatum, FadeInDistanceStart))
			{
				ChildRow.DisplayName(FText::Format(LOCTEXT("FadeInStart", "Fade In {0} Start"), ParamDesc));
				ChildRow.ToolTip(LOCTEXT("FadeInStartText", "The param value at which to start hearing this sound."));
			}
			else if (Child->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FDistanceDatum, FadeInDistanceEnd))
			{
				ChildRow.DisplayName(FText::Format(LOCTEXT("FadeInEnd", "Fade In {0} End"), ParamDesc));
				ChildRow.ToolTip(LOCTEXT("FadeInEndText", "The param value at which this sound has faded in completely."));
			}
			else if (Child->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FDistanceDatum, FadeOutDistanceStart))
			{
				ChildRow.DisplayName(FText::Format(LOCTEXT("FadeOutStart", "Fade Out {0} Start"), ParamDesc));
				ChildRow.ToolTip(LOCTEXT("FadeOutStartText", "The param value at which this sound starts fading out"));
			}
			else if (Child->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FDistanceDatum, FadeOutDistanceEnd))
			{
				ChildRow.DisplayName(FText::Format(LOCTEXT("FadeOutEnd", "Fade Out {0} End"), ParamDesc));
				ChildRow.ToolTip(LOCTEXT("FadeOutEndText", "The param value at which this sound is no longer audible."));
			}
		}
	}
}

void FCrossFadeCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	
	TArray< TWeakObjectPtr< UObject > > Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	//Restrict this to one object at a time
	if (Objects.Num() != 1)
	{
		return;
	}

	//We only need to change the tooltips if we're working with a SoundNodeParamCrossFade
	if (Objects[0]->IsA(USoundNodeParamCrossFade::StaticClass()))
	{
		TSharedRef< IPropertyHandle > CrossFadeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USoundNodeDistanceCrossFade, CrossFadeInput));
		ensure(CrossFadeHandle->IsValidHandle());

		CrossFadeHandle->SetToolTipText(LOCTEXT("CrossFadeInputText", "Each input needs to have the correct data filled in so the SoundNodeParamCrossFade is able to determine which sounds to play"));
	}

}

TSharedRef< IDetailCustomization > FCrossFadeCustomization::MakeInstance()
{
	return MakeShared<FCrossFadeCustomization>();
}


#undef LOCTEXT_NAMESPACE
