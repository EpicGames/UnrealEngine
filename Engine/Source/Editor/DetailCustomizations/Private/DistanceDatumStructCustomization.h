// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "IDetailCustomization.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

/**
 * Customizes a Distance Datum struct to improve naming when used as a parameter
 */
class FDistanceDatumStructCustomization : public IPropertyTypeCustomization
{
private:
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/**
	 * Destructor
	 */
	virtual ~FDistanceDatumStructCustomization();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	/**
	 * Constructor
	 */
	explicit FDistanceDatumStructCustomization(FPrivateToken);
};

class FCrossFadeCustomization : public IDetailCustomization
{
public:
	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//

	static TSharedRef< IDetailCustomization > MakeInstance();
};