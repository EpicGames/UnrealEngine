// Copyright Epic Games, Inc. All Rights Reserved.

#include "DocumentationActorDetails.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/DocumentationActor.h"
#include "Internationalization/Internationalization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

class UObject;


#define LOCTEXT_NAMESPACE "DocumentationActorDetails"


TSharedRef<IDetailCustomization> FDocumentationActorDetails::MakeInstance()
{
	return MakeShareable(new FDocumentationActorDetails);
}

void FDocumentationActorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	EDocumentationActorType::Type SelectedType = EDocumentationActorType::None;
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() > 0)
	{
		SelectedDocumentationActor = Cast<ADocumentationActor>(Objects[0].Get());
	}
	PropertyHandle = DetailBuilder.GetProperty("DocumentLink");

	// Add a button we can click on to open the documentation
	IDetailCategoryBuilder& HelpCategory = DetailBuilder.EditCategory("Help Data");
	HelpCategory.AddCustomRow(LOCTEXT("HelpDocumentation_Filter", "Help Documentation"))
		[
			SNew(SButton)
			.Text(this, &FDocumentationActorDetails::OnGetButtonText)
			.ToolTipText(this, &FDocumentationActorDetails::OnGetButtonTooltipText)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FDocumentationActorDetails::OnHelpButtonClicked)
			.IsEnabled(this, &FDocumentationActorDetails::IsButtonEnabled)
		];
	
}

FReply FDocumentationActorDetails::OnHelpButtonClicked()
{
	FReply Handled = FReply::Unhandled();
	
	FString DocumentLink;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(DocumentLink);
	if (Result == FPropertyAccess::Success)
	{
		if (SelectedDocumentationActor.IsValid() == true)
		{
			Handled = SelectedDocumentationActor->OpenDocumentLink() == true ? FReply::Handled() : FReply::Unhandled();
		}
	}
	return Handled;
}

FText FDocumentationActorDetails::OnGetButtonText() const
{
	FText ButtonText;
	if (IsButtonEnabled() == true)
	{
		EDocumentationActorType::Type LinkType = SelectedDocumentationActor->GetLinkType();
		if (LinkType == EDocumentationActorType::UDNLink)
		{
			ButtonText = LOCTEXT("HelpDocumentation", "Open Help Documentation");
		}
		else if (LinkType == EDocumentationActorType::URLLink)
		{
			ButtonText = LOCTEXT("HelpDocumentationURL", "Open Help URL");
		}
	}
	return ButtonText;
}

FText FDocumentationActorDetails::OnGetButtonTooltipText() const
{
	FText Tooltip;
	EDocumentationActorType::Type LinkType = SelectedDocumentationActor->GetLinkType();
	switch (LinkType)
	{
		case EDocumentationActorType::UDNLink:
			Tooltip = FText::Format(LOCTEXT("OpenUDNLink", "Opens the help link:{0}"), FText::FromString(SelectedDocumentationActor->DocumentLink));
			break;
		case EDocumentationActorType::URLLink:
			Tooltip = FText::Format(LOCTEXT("OpenURLLink", "Opens the Web page:{0}"), FText::FromString(SelectedDocumentationActor->DocumentLink));
			break;			
		case EDocumentationActorType::None:
			Tooltip = LOCTEXT("OpenNoString", "Enter a string in the link field to have this button open it");
			break;
		case EDocumentationActorType::InvalidLink:
			Tooltip = LOCTEXT("OpenInvalid", "The link field contains invalid data");
			break;			
	}
	return Tooltip;
}

bool FDocumentationActorDetails::IsButtonEnabled() const
{
	bool bResult = false;
	if ((SelectedDocumentationActor.IsValid() == true) && (SelectedDocumentationActor->HasValidDocumentLink() == true))
	{
		EDocumentationActorType::Type LinkType = SelectedDocumentationActor->GetLinkType();
		bResult = ((LinkType == EDocumentationActorType::UDNLink) || (LinkType == EDocumentationActorType::URLLink)) ? true : false;
	}

	return bResult;
}

#undef LOCTEXT_NAMESPACE
