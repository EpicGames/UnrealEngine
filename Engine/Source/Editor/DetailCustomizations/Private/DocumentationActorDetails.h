// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ADocumentationActor;
class IDetailLayoutBuilder;
class IPropertyHandle;

class FDocumentationActorDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
	/* Handler for clicking the help button */
	FReply OnHelpButtonClicked();

	/* Handler to get the text for the button */
	FText OnGetButtonText() const;
	
	/* Handler to get the text for the button tooltip  */
	FText OnGetButtonTooltipText() const;

	/* Handler to determine if the button is enabled (link is valid)  */
	bool IsButtonEnabled() const;

private:
	/* The first documentation actor that we are showing in the details panel */
	TWeakObjectPtr<ADocumentationActor>	SelectedDocumentationActor;

	/* Handle to the string property */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
