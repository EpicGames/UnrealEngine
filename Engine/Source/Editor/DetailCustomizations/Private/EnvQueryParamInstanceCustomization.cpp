// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnvQueryParamInstanceCustomization.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EnvironmentQuery/EnvQueryTypes.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FEnvQueryCustomization"

TSharedRef<IPropertyTypeCustomization> FEnvQueryParamInstanceCustomization::MakeInstance( )
{
	return MakeShareable(new FEnvQueryParamInstanceCustomization);
}

void FEnvQueryParamInstanceCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	NameProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvNamedValue,ParamName));
	TypeProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvNamedValue,ParamType));
	ValueProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FEnvNamedValue,Value));

	FSimpleDelegate OnTypeChangedDelegate = FSimpleDelegate::CreateSP( this, &FEnvQueryParamInstanceCustomization::OnTypeChanged );
	TypeProp->SetOnPropertyValueChanged(OnTypeChangedDelegate);
	InitCachedTypes();
	OnTypeChanged();

	// create struct header
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(STextBlock) 
		.Text(this, &FEnvQueryParamInstanceCustomization::GetHeaderDesc)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

void FEnvQueryParamInstanceCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	StructBuilder.AddProperty(NameProp.ToSharedRef());
	StructBuilder.AddProperty(TypeProp.ToSharedRef());
	
	StructBuilder.AddCustomRow(LOCTEXT("ValueLabel", "Value"))
		.NameContent()
		[
			ValueProp->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f, 5.0f, 2.0f)
			[
				SNew(SNumericEntryBox<float>)
				.AllowSpin(false)
				.Visibility(this, &FEnvQueryParamInstanceCustomization::GetParamNumValueVisibility)
				.Value(this, &FEnvQueryParamInstanceCustomization::GetParamNumValue)
				.OnValueChanged(this, &FEnvQueryParamInstanceCustomization::OnParamNumValueChanged)
			]
			+SHorizontalBox::Slot()
			.Padding(0.0f, 2.0f, 5.0f, 2.0f)
			[
				SNew(SCheckBox)
				.Visibility(this, &FEnvQueryParamInstanceCustomization::GetParamBoolValueVisibility)
				.IsChecked(this, &FEnvQueryParamInstanceCustomization::GetParamBoolValue )
				.OnCheckStateChanged(this, &FEnvQueryParamInstanceCustomization::OnParamBoolValueChanged )
			]
		];
}

TOptional<float> FEnvQueryParamInstanceCustomization::GetParamNumValue() const
{
	// Gets the actual aspect ratio property value
	if (ParamType == EAIParamType::Float)
	{
		float FloatValue = 0.0f;
		FPropertyAccess::Result PropResult = ValueProp->GetValue(FloatValue);
		if (PropResult == FPropertyAccess::Success)
		{
			return FloatValue;
		}
	}
	else if (ParamType == EAIParamType::Int)
	{
		float StoreValue = 0.0f;
		FPropertyAccess::Result PropResult = ValueProp->GetValue(StoreValue);
		if (PropResult == FPropertyAccess::Success)
		{
			const int32 IntValue = *((int32*)&StoreValue);
			return static_cast<float>(IntValue);
		}
	}

	return TOptional<float>();
}

void FEnvQueryParamInstanceCustomization::OnParamNumValueChanged(float FloatValue) const
{
	if (ParamType == EAIParamType::Float)
	{
		ValueProp->SetValue(FloatValue);
		CachedFloat = FloatValue;
	}
	else if (ParamType == EAIParamType::Int)
	{
		const int32 IntValue = FMath::TruncToInt(FloatValue);
		const float StoreValue = *((float*)&IntValue);
		ValueProp->SetValue(StoreValue);
		CachedInt = IntValue;
	}
}

ECheckBoxState FEnvQueryParamInstanceCustomization::GetParamBoolValue() const
{
	if (ParamType == EAIParamType::Bool)
	{
		float StoreValue = 0.0f;
		FPropertyAccess::Result PropResult = ValueProp->GetValue(StoreValue);
		if (PropResult == FPropertyAccess::Success)
		{
			return (StoreValue > 0.0f) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	
	return ECheckBoxState::Undetermined;
}

void FEnvQueryParamInstanceCustomization::OnParamBoolValueChanged(ECheckBoxState BoolValue) const
{
	if (ParamType == EAIParamType::Bool)
	{
		const float StoreValue = (BoolValue == ECheckBoxState::Checked) ? 1.0f : -1.0f;
		ValueProp->SetValue(StoreValue);
		CachedBool = (BoolValue == ECheckBoxState::Checked);
	}
}

EVisibility FEnvQueryParamInstanceCustomization::GetParamNumValueVisibility() const
{
	return (ParamType == EAIParamType::Int || ParamType == EAIParamType::Float) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FEnvQueryParamInstanceCustomization::GetParamBoolValueVisibility() const
{
	return (ParamType == EAIParamType::Bool) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FEnvQueryParamInstanceCustomization::GetHeaderDesc() const
{
	FString ParamName;
	if (NameProp->GetValue(ParamName) == FPropertyAccess::Success)
	{
		switch (ParamType)
		{
		case EAIParamType::Float:	return FText::FromString(FString::Printf(TEXT("%s = %s"), *ParamName, *FString::SanitizeFloat(CachedFloat)));
		case EAIParamType::Int:	return FText::FromString(FString::Printf(TEXT("%s = %d"), *ParamName, CachedInt));
		case EAIParamType::Bool:	return FText::FromString(FString::Printf(TEXT("%s = %s"), *ParamName, CachedBool ? TEXT("true") : TEXT("false")));
		}
	}

	return FText::GetEmpty();
}

void FEnvQueryParamInstanceCustomization::InitCachedTypes()
{
	CachedBool = false;
	CachedFloat = 0.0f;
	CachedInt = 0;

	uint8 EnumValue = 0;
	FPropertyAccess::Result PropResult = TypeProp->GetValue(EnumValue);
	if (PropResult == FPropertyAccess::Success)
	{
		ParamType = (EAIParamType)EnumValue;
		switch (ParamType)
		{
		case EAIParamType::Float:	CachedFloat = GetParamNumValue().GetValue(); break;
		case EAIParamType::Int:	CachedInt = FMath::TruncToInt(GetParamNumValue().GetValue()); break;
		case EAIParamType::Bool:	CachedBool = (GetParamBoolValue() == ECheckBoxState::Checked); break;
		}
	}
}

void FEnvQueryParamInstanceCustomization::OnTypeChanged()
{
	uint8 EnumValue = 0;
	FPropertyAccess::Result PropResult = TypeProp->GetValue(EnumValue);
	if (PropResult == FPropertyAccess::Success)
	{
		ParamType = (EAIParamType)EnumValue;
		switch (ParamType)
		{
		case EAIParamType::Float:	OnParamNumValueChanged(CachedFloat); break;
		case EAIParamType::Int:	OnParamNumValueChanged(static_cast<float>(CachedInt)); break;
		case EAIParamType::Bool:	OnParamBoolValueChanged(CachedBool ? ECheckBoxState::Checked : ECheckBoxState::Unchecked); break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
