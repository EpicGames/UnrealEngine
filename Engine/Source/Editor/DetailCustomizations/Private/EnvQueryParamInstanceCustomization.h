// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

enum class EAIParamType : uint8;
enum class ECheckBoxState : uint8;

class FEnvQueryParamInstanceCustomization : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance( );

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:

	TSharedPtr<IPropertyHandle> NameProp;
	TSharedPtr<IPropertyHandle> ValueProp;
	TSharedPtr<IPropertyHandle> TypeProp;

	TOptional<float> GetParamNumValue() const;
	void OnParamNumValueChanged(float FloatValue) const;
	EVisibility GetParamNumValueVisibility() const;

	ECheckBoxState GetParamBoolValue() const;
	void OnParamBoolValueChanged(ECheckBoxState BoolValue) const;
	EVisibility GetParamBoolValueVisibility() const;

	FText GetHeaderDesc() const;
	void OnTypeChanged();
	void InitCachedTypes();

	EAIParamType ParamType;
	mutable uint8 CachedBool : 1;
	mutable float CachedFloat;
	mutable int32 CachedInt;
};
