// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxImportUIDetails.h"

#include "AssetToolsModule.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxMeshImportData.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Materials/MaterialParameters.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/NamePermissionList.h"
#include "Misc/StringFormatArg.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SEnumCombo.h"
#include "SlotBase.h"
#include "StaticMeshResources.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FbxImportUIDetails)

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "FbxImportUIDetails"

#define MinimumLodNumberID 0
#define LodNumberID 1

static FString DoNotOverrideString = FText(LOCTEXT("BaseColorPropertyDoNotOverride", "Do Not Override")).ToString();

//If the String is contain in the StringArray, it return the index. Otherwise return INDEX_NONE
int FindString(const TArray<TSharedPtr<FString>> &StringArray, const FString &String) {
	for (int i = 0; i < StringArray.Num(); i++)
	{
		if (String.Equals(*StringArray[i].Get()))
		{
			return i;
		}
	}
	return INDEX_NONE;
}

FFbxImportUIDetails::FFbxImportUIDetails()
{
	CachedDetailBuilder = nullptr;

	LODGroupNames.Reset();
	UStaticMesh::GetLODGroups(LODGroupNames);
	for (int32 GroupIndex = 0; GroupIndex < LODGroupNames.Num(); ++GroupIndex)
	{
		LODGroupOptions.Add(MakeShareable(new FString(LODGroupNames[GroupIndex].GetPlainNameString())));
	}

	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}
}

FFbxImportUIDetails::~FFbxImportUIDetails()
{
	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->UnregisterForUndo(this);
	}
}

void FFbxImportUIDetails::RefreshCustomDetail()
{
	if (CachedDetailBuilder)
	{
		CachedDetailBuilder->ForceRefreshDetails();
	}
}

void FFbxImportUIDetails::PostUndo(bool bSuccess)
{
	//Refresh the UI
	RefreshCustomDetail();
}

void FFbxImportUIDetails::PostRedo(bool bSuccess)
{
	//Refresh the UI
	RefreshCustomDetail();
}


TSharedRef<IDetailCustomization> FFbxImportUIDetails::MakeInstance()
{
	return MakeShareable( new FFbxImportUIDetails );
}

bool SkipImportProperty(TSharedPtr<IPropertyHandle> Handle, const FString &MetaData, const bool bImportGeoOnly, const bool bImportRigOnly)
{
	TArray<FString> Types;
	MetaData.ParseIntoArray(Types, TEXT("|"), 1);
	if (bImportRigOnly && Types.Contains(TEXT("GeoOnly")))
	{
		return true;
	}
	if (bImportGeoOnly && Types.Contains(TEXT("RigOnly")))
	{
		return true;
	}
	if ((bImportGeoOnly || bImportRigOnly) && Types.Contains(TEXT("RigAndGeo")))
	{
		return true;
	}
	return false;
}

bool FFbxImportUIDetails::ShowCompareResult()
{
	bool bHasMaterialConflict = false;
	ImportCompareHelper::ECompareResult SkeletonCompareResult = ImportCompareHelper::ECompareResult::SCR_None;
	bool bShowCompareResult = ImportUI->bIsReimport && ImportUI->ReimportMesh != nullptr && ImportUI->OnUpdateCompareFbx.IsBound();
	if (bShowCompareResult)
	{
		//Always update the compare data with the current option
		ImportUI->OnUpdateCompareFbx.Execute();
		bHasMaterialConflict = ImportUI->MaterialCompareData.HasConflict();
		SkeletonCompareResult = ImportUI->SkeletonCompareData.CompareResult;
		if (bHasMaterialConflict || SkeletonCompareResult != ImportCompareHelper::ECompareResult::SCR_None)
		{
			FName ConflictCategoryName = TEXT("Conflicts");
			IDetailCategoryBuilder& CategoryBuilder = CachedDetailBuilder->EditCategory(ConflictCategoryName, LOCTEXT("CategoryConflictsName", "Conflicts"), ECategoryPriority::Important);
			auto BuildConflictRow = [this, &CategoryBuilder](const FText& CategoryName, const FText& Conflict_NameContent, const FText& Conflict_ButtonTooltip, const FText& Conflict_ButtonText, EConflictDialogType DialogType, const FSlateBrush* Brush, const FText& Conflict_IconTooltip)
			{
				CategoryBuilder.AddCustomRow(CategoryName).WholeRowContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.AutoWidth()
					.Padding(2.0f, 2.0f, 5.0f, 2.0f)
					[
						SNew(SImage)
						.ToolTipText(Conflict_IconTooltip)
						.Image(Brush)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(Conflict_NameContent)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(Conflict_ButtonTooltip)
						.OnClicked(this, &FFbxImportUIDetails::ShowConflictDialog, DialogType)
						.Content()
						[
							SNew(STextBlock)
							.Text(Conflict_ButtonText)
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]

				];
			};

			if (bHasMaterialConflict)
			{
				BuildConflictRow(LOCTEXT("MaterialConflict_RowFilter", "Material conflict"),
					LOCTEXT("MaterialConflict_NameContent", "Unmatched Materials"),
					LOCTEXT("MaterialConflict_ButtonShowTooltip", "Show a detailed view of the materials conflict."),
					LOCTEXT("MaterialConflict_ButtonShow", "Show Conflict"),
					EConflictDialogType::Conflict_Material,
					FAppStyle::GetBrush("Icons.Error"),
					LOCTEXT("MaterialConflict_IconTooltip", "There is one or more material(s) that do not match.")
				);
			}

			if (SkeletonCompareResult != ImportCompareHelper::ECompareResult::SCR_None)
			{
				FText IconTooltip;
				if ((SkeletonCompareResult & ImportCompareHelper::ECompareResult::SCR_SkeletonBadRoot) > ImportCompareHelper::ECompareResult::SCR_None)
				{
					IconTooltip = (LOCTEXT("SkeletonConflictBadRoot_IconTooltip", "(Error) Root bone: The root bone of the incoming fbx do not match the root bone of the current skeletalmesh asset. Import will probably fail!"));
				}
				else if ((SkeletonCompareResult & ImportCompareHelper::ECompareResult::SCR_SkeletonMissingBone) > ImportCompareHelper::ECompareResult::SCR_None)
				{
					IconTooltip = (LOCTEXT("SkeletonConflictDeletedBones_IconTooltip", "(Warning) Deleted bones: Some bones of the of the current skeletalmesh asset are not use by the incoming fbx."));
				}
				else
				{
					IconTooltip = (LOCTEXT("SkeletonConflictAddedBones_IconTooltip", "(Info) Added bones: Some bones in the incoming fbx do not exist in the current skeletalmesh asset."));
				}
				 
				const FSlateBrush* Brush = (SkeletonCompareResult & ImportCompareHelper::ECompareResult::SCR_SkeletonBadRoot) > ImportCompareHelper::ECompareResult::SCR_None ? FAppStyle::GetBrush("Icons.Error")
					: (SkeletonCompareResult & ImportCompareHelper::ECompareResult::SCR_SkeletonMissingBone) > ImportCompareHelper::ECompareResult::SCR_None ? FAppStyle::GetBrush("Icons.Warning")
					: FAppStyle::GetBrush("Icons.Info");

				BuildConflictRow(LOCTEXT("SkeletonConflict_RowFilter", "Skeleton conflict"),
					LOCTEXT("SkeletonConflict_NameContent", "Unmatched Skeleton joints"),
					LOCTEXT("SkeletonConflict_ButtonShowTooltip", "Show a detailed view of the skeleton joints conflict."),
					LOCTEXT("SkeletonConflict_ButtonShow", "Show Conflict"),
					EConflictDialogType::Conflict_Skeleton,
					Brush,
					IconTooltip);
			}
		}
	}
	return bShowCompareResult;
}

void FFbxImportUIDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	CachedDetailBuilder = &DetailBuilder;
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	ImportUI = Cast<UFbxImportUI>(EditingObjects[0].Get());
	
	auto GetClassPermission = [](UClass* Class, EAssetClassAction ClassAction)
		{
			IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
			TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(ClassAction);
			if (Class && AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
			{
				return AssetClassPermissionList->PassesFilter(Class->GetPathName());
			}
			return true;
		};
	bool bShowPhysicsAssetOptions = GetClassPermission(UPhysicsAsset::StaticClass(), EAssetClassAction::ImportAsset);

	bool bShowCompareResult = ShowCompareResult();

	auto AddRefreshCustomDetailEvent = [this](TSharedPtr<IPropertyHandle> Handle)
	{
		Handle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::RefreshCustomDetail));
	};

	auto SetupRefreshForHandle = [this, &bShowCompareResult, &AddRefreshCustomDetailEvent](TSharedPtr<IPropertyHandle> Handle)
	{
		if (bShowCompareResult && Handle->GetProperty() != nullptr)
		{
			FProperty* Property = Handle->GetProperty();
			if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxImportUI, Skeleton) ||
				Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportRigidMesh) ||
				Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, bImportMeshesInBoneHierarchy) ||
				Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, bCombineMeshes)
				)
			{
				AddRefreshCustomDetailEvent(Handle);
			}
		}
	};
	
	bool bImportGeoOnly = false;
	bool bImportRigOnly = false;

	// Handle mesh category
	IDetailCategoryBuilder& MeshCategory = DetailBuilder.EditCategory("Mesh", FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory("Transform");
	TArray<TSharedRef<IPropertyHandle>> CategoryDefaultProperties;
	TArray<TSharedPtr<IPropertyHandle>> ExtraProperties;

	// Grab and hide per-type import options
	TSharedRef<IPropertyHandle> StaticMeshDataProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, StaticMeshImportData));
	TSharedRef<IPropertyHandle> SkeletalMeshDataProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, SkeletalMeshImportData));
	TSharedRef<IPropertyHandle> AnimSequenceDataProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, AnimSequenceImportData));
	DetailBuilder.HideProperty(StaticMeshDataProp);
	DetailBuilder.HideProperty(SkeletalMeshDataProp);
	DetailBuilder.HideProperty(AnimSequenceDataProp);

	TSharedPtr<IPropertyHandle> SK_ImportContent_DataProp = SkeletalMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, ImportContentType));
	AddRefreshCustomDetailEvent(SK_ImportContent_DataProp);

	TSharedRef<IPropertyHandle> ImportMaterialsProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportMaterials));
	ImportMaterialsProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::ImportMaterialsChanged));

	TSharedRef<IPropertyHandle> ImportAutoComputeLodDistancesProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bAutoComputeLodDistances));
	ImportAutoComputeLodDistancesProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::ImportAutoComputeLodDistancesChanged));

	TSharedPtr<IPropertyHandle> ImportMeshLODsProp = StaticMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, bImportMeshLODs));
	ImportMeshLODsProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::RefreshCustomDetail));

	MeshCategory.GetDefaultProperties(CategoryDefaultProperties);

	
	switch(ImportUI->MeshTypeToImport)
	{
		case FBXIT_StaticMesh:
			{
				//Validate static mesh input
				TSharedRef<IPropertyHandle> MinimumLodNumberProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, MinimumLodNumber));
				MinimumLodNumberProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::ValidateLodSettingsChanged, MinimumLodNumberID));
				//Validate static mesh input
				TSharedRef<IPropertyHandle> LodNumberProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, LodNumber));
				LodNumberProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::ValidateLodSettingsChanged, LodNumberID));

				CollectChildPropertiesRecursive(StaticMeshDataProp, ExtraProperties);
			}
			break;
		case FBXIT_SkeletalMesh:
			{
				bImportGeoOnly = ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_Geometry;
				bImportRigOnly = ImportUI->SkeletalMeshImportData->ImportContentType == EFBXImportContentType::FBXICT_SkinningWeights;
				if (ImportUI->bImportMesh)
				{
					CollectChildPropertiesRecursive(SkeletalMeshDataProp, ExtraProperties);
				}
				else
				{
					ImportUI->MeshTypeToImport = FBXIT_Animation;
				}
			}
			break;
		default:
			break;
	}
	EFBXImportType ImportType = ImportUI->MeshTypeToImport;

	//Hide LodDistance property if we do not need them
	if (ImportUI->bIsReimport || ImportType != FBXIT_StaticMesh || !ImportUI->StaticMeshImportData->bImportMeshLODs)
	{
		DetailBuilder.HideCategory(FName(TEXT("LodSettings")));
	}
	else
	{
		int32 ShowMaxLodIndex = (ImportUI->bAutoComputeLodDistances ? 0 : ImportUI->LodNumber > 0 ? ImportUI->LodNumber : MAX_STATIC_MESH_LODS) - 1;
		for (int32 LodIndex = 0; LodIndex < MAX_STATIC_MESH_LODS; ++LodIndex)
		{
			if (LodIndex <= ShowMaxLodIndex)
			{
				continue;
			}
			TArray<FStringFormatArg> Args;
			Args.Add(TEXT("LodDistance"));
			Args.Add(FString::FromInt(LodIndex));
			FString LodDistancePropertyName = FString::Format(TEXT("{0}{1}"), Args);
			TSharedRef<IPropertyHandle> Handle = DetailBuilder.GetProperty(FName(*LodDistancePropertyName));
			FProperty* Property = Handle->GetProperty();
			if (Property != nullptr && Property->GetName().Compare(LodDistancePropertyName) == 0)
			{
				DetailBuilder.HideProperty(Handle);
			}
		}
	}

	if(ImportType != FBXIT_Animation)
	{
		{
			TSharedRef<IPropertyHandle> Prop = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportAsSkeletal));
			if (!ImportUI->bIsReimport)
			{
				Prop->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::MeshImportModeChanged));
				MeshCategory.AddProperty(Prop);
			}
			else
			{
				DetailBuilder.HideProperty(Prop);
			}
		}
	}

	TSharedRef<IPropertyHandle> ImportMeshProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportMesh));
	if(ImportUI->OriginalImportType == FBXIT_SkeletalMesh && ImportType != FBXIT_StaticMesh && !ImportUI->bIsReimport)
	{
		ImportMeshProp->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::ImportMeshToggleChanged));
		MeshCategory.AddProperty(ImportMeshProp);
	}
	else
	{
		DetailBuilder.HideProperty(ImportMeshProp);
	}

	for(TSharedRef<IPropertyHandle> Handle : CategoryDefaultProperties)
	{
		FString MetaData = Handle->GetMetaData(TEXT("ImportType"));
		if(!IsImportTypeMetaDataValid(ImportType, MetaData))
		{
			DetailBuilder.HideProperty(Handle);
		}
		else if (ImportUI->bIsReimport && Handle->GetBoolMetaData(TEXT("ReimportRestrict")))
		{
			DetailBuilder.HideProperty(Handle);
		}
		else
		{
			SetupRefreshForHandle(Handle);
		}
	}

	if (!bShowPhysicsAssetOptions)
	{
		TSharedRef<IPropertyHandle> CreatePhysicsAssetProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bCreatePhysicsAsset));
		DetailBuilder.HideProperty(CreatePhysicsAssetProperty);
		ImportUI->bCreatePhysicsAsset = false;
	}

	TMap<FString, TArray<TSharedPtr<IPropertyHandle>>> SubCategoriesProperties;
	TMap<FString, bool > SubCategoriesAdvanced;
	TMap<FString, FText > SubCategoriesTooltip;

	for(TSharedPtr<IPropertyHandle> Handle : ExtraProperties)
	{
		FProperty* Property = Handle->GetProperty();
		FString ImportTypeMetaData = Handle->GetMetaData(TEXT("ImportType"));
		const FString& CategoryMetaData = Handle->GetMetaData(TEXT("ImportCategory"));
		const FString& SubCategoryData = Handle->GetMetaData(TEXT("SubCategory"));
		bool bSkip = (bImportGeoOnly || bImportRigOnly) && SkipImportProperty(Handle, ImportTypeMetaData, bImportGeoOnly, bImportRigOnly);
		if (!ImportUI->bAllowContentTypeImport && Property != nullptr && Property == SK_ImportContent_DataProp->GetProperty())
		{
			bSkip = true;
		}

		//Skip the variable that is ReimportRestrick when we are in reimport mode
		bSkip |= ImportUI->bIsReimport && Handle->GetBoolMetaData(TEXT("ReimportRestrict"));
		
		if(!bSkip && IsImportTypeMetaDataValid(ImportType, ImportTypeMetaData))
		{
			// Decide on category
			if(!CategoryMetaData.IsEmpty())
			{
				// Populate custom categories.
				IDetailCategoryBuilder& CustomCategory = DetailBuilder.EditCategory(*CategoryMetaData);
				CustomCategory.AddProperty(Handle);
			}
			else if (!SubCategoryData.IsEmpty())
			{
				//Localize the sub category
				FString SubCategoryLocalizeName = SubCategoryData;
				if (SubCategoryData.Equals(TEXT("Thresholds")))
				{
					SubCategoryLocalizeName = LOCTEXT("Thresholds_subcategory_name", "Thresholds").ToString();
				}
				TArray<TSharedPtr<IPropertyHandle> >& SubCategoryProperties = SubCategoriesProperties.FindOrAdd(SubCategoryLocalizeName);
				SubCategoryProperties.Add(Handle);
				bool& SubCategoryAdvanced = SubCategoriesAdvanced.FindOrAdd(SubCategoryLocalizeName);
				FText& SubCategoryTooltip = SubCategoriesTooltip.FindOrAdd(SubCategoryLocalizeName);
				if (SubCategoryData.Equals(TEXT("Thresholds")))
				{
					SubCategoryAdvanced = true;
					SubCategoryTooltip = LOCTEXT("Thresholds_subcategory_tooltip", "Thresholds for when a vertex is considered the same as another vertex");
				}
			}
			else
			{
				// No override, add to default mesh category
				IDetailPropertyRow& PropertyRow = MeshCategory.AddProperty(Handle);

				if (Property != nullptr)
				{
					if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, StaticMeshLODGroup))
					{
						//We cannot change the LODGroup when re-importing so hide the option
						if (ImportUI->bIsReimport)
						{
							PropertyRow.Visibility(EVisibility::Collapsed);
						}
						else
						{
							SetStaticMeshLODGroupWidget(PropertyRow, Handle);
						}
					}

					if (ImportType == FBXIT_StaticMesh)
					{
						if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, VertexOverrideColor))
						{
							// Cache the VertexColorImportOption property
							VertexColorImportOptionHandle = StaticMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, VertexColorImportOption));

							PropertyRow.IsEnabled(TAttribute<bool>(this, &FFbxImportUIDetails::GetVertexOverrideColorEnabledState));
						}
					}
					else
					{
						if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, VertexOverrideColor))
						{
							// Cache the VertexColorImportOption property
							SkeletalMeshVertexColorImportOptionHandle = SkeletalMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, VertexColorImportOption));

							PropertyRow.IsEnabled(TAttribute<bool>(this, &FFbxImportUIDetails::GetSkeletalMeshVertexOverrideColorEnabledState));
						}
					}
					
				}
			}
			//Add refresh callback
			SetupRefreshForHandle(Handle);
		}
	}

	//Lets add all "Mesh" sub category we found
	AddSubCategory(DetailBuilder, "Mesh", SubCategoriesProperties, SubCategoriesAdvanced, SubCategoriesTooltip);

	// Animation Category
	IDetailCategoryBuilder& AnimCategory = DetailBuilder.EditCategory("Animation", FText::GetEmpty(), ECategoryPriority::Important);

	CategoryDefaultProperties.Empty();
	AnimCategory.GetDefaultProperties(CategoryDefaultProperties);
	for(TSharedRef<IPropertyHandle> Handle : CategoryDefaultProperties)
	{
		FString MetaData = Handle->GetMetaData(TEXT("ImportType"));
		bool bSkip = (bImportGeoOnly || bImportRigOnly) && SkipImportProperty(Handle, MetaData, bImportGeoOnly, bImportRigOnly);
		if(!IsImportTypeMetaDataValid(ImportType, MetaData))
		{
			DetailBuilder.HideProperty(Handle);
		}
	}

	if(ImportType == FBXIT_Animation || (ImportType == FBXIT_SkeletalMesh && !bImportGeoOnly && !ImportUI->bIsReimport))
	{
		ExtraProperties.Empty();
		CollectChildPropertiesRecursive(AnimSequenceDataProp, ExtraProperties);

		// Before we add the import data properties we need to re-add any properties we want to appear above them in the UI
		TSharedRef<IPropertyHandle> ImportAnimProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportAnimations));
		// If we're importing an animation file we really don't need to ask this
		DetailBuilder.HideProperty(ImportAnimProp);
		if(ImportType == FBXIT_Animation)
		{
			ImportUI->bImportAnimations = true;
		}
		else
		{
			AnimCategory.AddProperty(ImportAnimProp);
		}

		for(TSharedPtr<IPropertyHandle> Handle : ExtraProperties)
		{
			const FString& CategoryMetaData = Handle->GetMetaData(TEXT("ImportCategory"));
			if(Handle->GetProperty()->GetOwner<UObject>() == UFbxAnimSequenceImportData::StaticClass()
			   && CategoryMetaData.IsEmpty())
			{
				// Add to default anim category if no override specified
				IDetailPropertyRow& PropertyRow = AnimCategory.AddProperty(Handle);
			}
			else if(ImportType == FBXIT_Animation && !CategoryMetaData.IsEmpty())
			{
				// Override category is available
				IDetailCategoryBuilder& CustomCategory = DetailBuilder.EditCategory(*CategoryMetaData);
				CustomCategory.AddProperty(Handle);
			}
		}
	}
	else
	{
		// Hide animation options
		CategoryDefaultProperties.Empty();
		AnimCategory.GetDefaultProperties(CategoryDefaultProperties);

		for(TSharedRef<IPropertyHandle> Handle : CategoryDefaultProperties)
		{
			DetailBuilder.HideProperty(Handle);
		}
	}

	// Material Category
	IDetailCategoryBuilder& MaterialCategory = DetailBuilder.EditCategory("Material");
	if(ImportUI->bIsReimport || ImportType == FBXIT_Animation || bImportRigOnly)
	{
		// hide the material category
		DetailBuilder.HideCategory("Material");
	}
	else
	{
		TSharedRef<IPropertyHandle> ImportMaterialPropHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, bImportMaterials));

		TSharedRef<IPropertyHandle> TextureDataProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxImportUI, TextureImportData));
		DetailBuilder.HideProperty(TextureDataProp);

		ExtraProperties.Empty();
		CollectChildPropertiesRecursive(TextureDataProp, ExtraProperties);
		

		TSharedPtr<IPropertyHandle> MaterialLocationPropHandle;
		for (TSharedPtr<IPropertyHandle> Handle : ExtraProperties)
		{
			// We ignore base import data for this window.
			if (Handle->GetProperty()->GetOwner<UObject>() == UFbxTextureImportData::StaticClass())
			{
				if (Handle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxTextureImportData, MaterialSearchLocation))
				{
					MaterialLocationPropHandle = Handle;
				}
			}
		}

		//The order is
		// Search Location
		// Import Materials
		// [Base Material Name]
		// [All Base Material Parameter]
		// 
		DetailBuilder.HideProperty(MaterialLocationPropHandle);
		MaterialCategory.AddProperty(MaterialLocationPropHandle);
		DetailBuilder.HideProperty(ImportMaterialPropHandle);
		ConstructMaterialImportMethod(ImportMaterialPropHandle, MaterialCategory);

		for(TSharedPtr<IPropertyHandle> Handle : ExtraProperties)
		{
			// We ignore base import data for this window.
			if(Handle->GetProperty()->GetOwner<UObject>() == UFbxTextureImportData::StaticClass())
			{
				if (Handle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UFbxTextureImportData, BaseMaterialName))
				{
					if (ImportUI->bImportMaterials && ImportUI->TextureImportData->bUseBaseMaterial)
					{
						ConstructBaseMaterialUI(Handle, MaterialCategory);
					}
				}
				else if (Handle != MaterialLocationPropHandle)
				{
					MaterialCategory.AddProperty(Handle);
				}
			}
		}

		TSharedPtr<IPropertyHandle> ReorderMaterialToFbxOrderProp = nullptr;
		if (ImportUI->MeshTypeToImport == FBXIT_StaticMesh)
		{
			ReorderMaterialToFbxOrderProp = StaticMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxStaticMeshImportData, bReorderMaterialToFbxOrder));
		}
		else if (ImportUI->MeshTypeToImport == FBXIT_SkeletalMesh)
		{
			ReorderMaterialToFbxOrderProp = SkeletalMeshDataProp->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, bReorderMaterialToFbxOrder));
		}
		//Add the advance reorder option at the end
		MaterialCategory.AddProperty(ReorderMaterialToFbxOrderProp);
	}

	//Information category
	IDetailCategoryBuilder& InformationCategory = DetailBuilder.EditCategory("FbxFileInformation", FText::GetEmpty());
	CategoryDefaultProperties.Empty();
	InformationCategory.GetDefaultProperties(CategoryDefaultProperties);
	for (TSharedRef<IPropertyHandle> Handle : CategoryDefaultProperties)
	{
		FString MetaData = Handle->GetMetaData(TEXT("ImportType"));
		DetailBuilder.HideProperty(Handle);
		if (IsImportTypeMetaDataValid(ImportType, MetaData))
		{
			FDetailWidgetRow& WidgetRow = DetailBuilder.AddCustomRowToCategory(Handle, Handle->GetPropertyDisplayName());
			FText PropertyValue;
			Handle->GetValueAsDisplayText(PropertyValue);
			WidgetRow.NameContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Margin(FMargin(2.0f, 2.0f))
					.Text(Handle->GetPropertyDisplayName())
					.ToolTipText(Handle->GetToolTipText())
				]
			];
			WidgetRow.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Margin(FMargin(2.0f, 2.0f))
					.Text(PropertyValue)
					.ToolTipText(PropertyValue)
				]
			];
		}
	}
}

void FFbxImportUIDetails::AddSubCategory(IDetailLayoutBuilder& DetailBuilder, FName MainCategoryName, TMap<FString, TArray<TSharedPtr<IPropertyHandle>>>& SubCategoriesProperties, TMap<FString, bool >& SubCategoriesAdvanced, TMap<FString, FText >& SubCategoriesTooltip)
{
	IDetailCategoryBuilder& MainCategory = DetailBuilder.EditCategory(MainCategoryName);
	//If we found some sub category we can add them to the group
	for (auto Kvp : SubCategoriesProperties)
	{
		FString& SubCategoryName = Kvp.Key;
		TArray<TSharedPtr<IPropertyHandle>>& SubCategoryProperties = Kvp.Value;
		bool SubCategoryAdvanced = SubCategoriesAdvanced[Kvp.Key];
		IDetailGroup& Group = MainCategory.AddGroup(FName(*SubCategoryName), FText::FromString(SubCategoryName), SubCategoryAdvanced);
		for (int32 PropertyIndex = 0; PropertyIndex < SubCategoryProperties.Num(); ++PropertyIndex)
		{
			TSharedPtr<IPropertyHandle>& PropertyHandle = SubCategoryProperties[PropertyIndex];
			DetailBuilder.HideProperty(PropertyHandle);
			Group.AddPropertyRow(PropertyHandle.ToSharedRef());
		}
		const FText& SubCategoryTooltip = SubCategoriesTooltip[Kvp.Key];
		if (!SubCategoryTooltip.IsEmpty())
		{
			FDetailWidgetRow& GroupHeaderRow = Group.HeaderRow();
			GroupHeaderRow.NameContent().Widget = SNew(SBox)
			[
				SNew(STextBlock)
				.Text(FText::FromString(SubCategoryName))
				.ToolTipText(SubCategoryTooltip)
			];
		}
	}
}

void FFbxImportUIDetails::ConstructMaterialImportMethod(TSharedPtr<IPropertyHandle> ImportMaterialPropHandle, IDetailCategoryBuilder& MaterialCategory)
{
	if(ImportUI->TextureImportData->BaseMaterialName.IsValid())
	{
		//When we load the UI the first time we set this boolean to true in case the BaseMaterialName is valid.
		ImportUI->TextureImportData->bUseBaseMaterial = true;
	}

	const UEnum* MaterialImportMethodEnum = StaticEnum< EMaterialImportMethod >();
	SelectedMaterialImportMethod = ImportUI->bImportMaterials
		? (ImportUI->TextureImportData->bUseBaseMaterial ? EMaterialImportMethod::CreateNewInstancedMaterials : EMaterialImportMethod::CreateNewMaterials) 
		: EMaterialImportMethod::DoNotCreateMaterialString;

	MaterialCategory.AddCustomRow(LOCTEXT("MaterialImportMethod", "Material Import Method"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("MaterialImportMethod", "Material Import Method"))
		.ToolTipText(LOCTEXT("MaterialImportMethodToolTip", "How materials are created when the importer cannot found it using the search location."))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			[
				SNew(SEnumComboBox, MaterialImportMethodEnum)
				.OnEnumSelectionChanged(SEnumComboBox::FOnEnumSelectionChanged::CreateSP(this, &FFbxImportUIDetails::OnMaterialImportMethodChanged))
				.CurrentValue(this, &FFbxImportUIDetails::GetMaterialImportMethodValue)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];
}

void FFbxImportUIDetails::ConstructBaseMaterialUI(TSharedPtr<IPropertyHandle> Handle, IDetailCategoryBuilder& MaterialCategory)
{
	IDetailPropertyRow &MaterialPropertyRow = MaterialCategory.AddProperty(Handle);
	Handle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FFbxImportUIDetails::BaseMaterialChanged));
	UMaterialInterface *MaterialInstanceProperty = Cast<UMaterialInterface>(ImportUI->TextureImportData->BaseMaterialName.TryLoad());
	if (MaterialInstanceProperty == nullptr)
	{
		return;
	}
	UMaterial *Material = MaterialInstanceProperty->GetMaterial();
	if (Material == nullptr)
	{
		return;
	}

	BaseColorNames.Empty();
	BaseTextureNames.Empty();
	BaseColorNames.Add(MakeShareable(new FString(DoNotOverrideString)));
	BaseTextureNames.Add(MakeShareable(new FString(DoNotOverrideString)));
	TArray<FMaterialParameterInfo> OutParameterInfo;
	TArray<FGuid> Guids;
	float MinDesiredWidth = 150.0f;
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	MaterialPropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
	int InitialSelect = INDEX_NONE;

	// base color properties, only used when there is no texture in the diffuse map
	Material->GetAllVectorParameterInfo(OutParameterInfo, Guids);
	if (OutParameterInfo.Num() > 0)
	{
		for (FMaterialParameterInfo &ParameterInfo : OutParameterInfo)
		{
			BaseColorNames.Add(MakeShareable(new FString(ParameterInfo.Name.ToString())));
		}
	}
	OutParameterInfo.Empty();
	Material->GetAllTextureParameterInfo(OutParameterInfo, Guids);
	if (OutParameterInfo.Num() > 0)
	{
		for (FMaterialParameterInfo &ParameterInfo : OutParameterInfo)
		{
			BaseTextureNames.Add(MakeShareable(new FString(ParameterInfo.Name.ToString())));
		}
	}
	if (BaseColorNames.Num() > 1)
	{
		InitialSelect = FindString(BaseColorNames, ImportUI->TextureImportData->BaseColorName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseColorProperty", "Base Color Property"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseColorProperty", "Base Color Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseColorNames)
					.ToolTip(SNew(SToolTip).Text(LOCTEXT("BaseColorFBXImportToolTip", "When there is no diffuse texture in the imported material this color property will be used to fill a contant color value instead.")))
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnBaseColor)
					.InitiallySelectedItem(BaseColorNames[InitialSelect])
				]
			]
		];
	}
	// base texture properties
	if(BaseTextureNames.Num() > 1)
	{
		InitialSelect = FindString(BaseTextureNames, ImportUI->TextureImportData->BaseDiffuseTextureName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseTextureProperty", "Base Texture Property")).NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseTextureProperty", "Base Texture Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseTextureNames)
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnDiffuseTextureColor)
					.InitiallySelectedItem(BaseTextureNames[InitialSelect])
				]
			]
		];

		// base normal properties
		InitialSelect = FindString(BaseTextureNames, ImportUI->TextureImportData->BaseNormalTextureName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseNormalTextureProperty", "Base Normal Texture Property")).NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseNormalTextureProperty", "Base Normal Texture Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseTextureNames)
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnNormalTextureColor)
					.InitiallySelectedItem(BaseTextureNames[InitialSelect])
				]
			]
		];
	}

	if(BaseColorNames.Num() > 1)
	{
		// base emissive color properties, only used when there is no texture in the emissive map
		InitialSelect = FindString(BaseColorNames, ImportUI->TextureImportData->BaseEmissiveColorName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseEmissiveColorProperty", "Base Emissive Color Property"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseEmissiveColorProperty", "Base Emissive Color Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseColorNames)
					.ToolTip(SNew(SToolTip).Text(LOCTEXT("BaseEmissiveColorFBXImportToolTip", "When there is no emissive texture in the imported material this emissive color property will be used to fill a contant color value instead.")))
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnEmissiveColor)
					.InitiallySelectedItem(BaseColorNames[InitialSelect])
				]
			]
		];
	}

	if (BaseTextureNames.Num() > 1)
	{
		// base emmisive properties
		InitialSelect = FindString(BaseTextureNames, ImportUI->TextureImportData->BaseEmmisiveTextureName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseEmissiveTextureProperty", "Base Emissive Texture Property")).NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseEmissiveTextureProperty", "Base Emissive Texture Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseTextureNames)
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnEmmisiveTextureColor)
					.InitiallySelectedItem(BaseTextureNames[InitialSelect])
				]
			]
		];

		// base specular properties
		InitialSelect = FindString(BaseTextureNames, ImportUI->TextureImportData->BaseSpecularTextureName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseSpecularTextureProperty", "Base Specular Texture Property")).NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseSpecularTextureProperty", "Base Specular Texture Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseTextureNames)
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnSpecularTextureColor)
					.InitiallySelectedItem(BaseTextureNames[InitialSelect])
				]
			]
		];

		// base opacity properties
		InitialSelect = FindString(BaseTextureNames, ImportUI->TextureImportData->BaseOpacityTextureName);
		InitialSelect = InitialSelect == INDEX_NONE ? 0 : InitialSelect; // default to the empty string located at index 0
		MaterialCategory.AddCustomRow(LOCTEXT("BaseOpacityTextureProperty", "Base Opacity Texture Property")).NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseOpacityTextureProperty", "Base Opacity Texture Property"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidth)
				[
					SNew(STextComboBox)
					.OptionsSource(&BaseTextureNames)
					.OnSelectionChanged(this, &FFbxImportUIDetails::OnOpacityTextureColor)
					.InitiallySelectedItem(BaseTextureNames[InitialSelect])
				]
			]
		];
	}

	if (BaseTextureNames.Num() > 1 || BaseColorNames.Num() > 1)
	{
		MaterialCategory.AddCustomRow(LOCTEXT("BaseParamPropertyClearAll", "Clear All Properties"))
		.ValueContent()
		[
			SNew(SButton)
			.OnClicked(this, &FFbxImportUIDetails::MaterialBaseParamClearAllProperties)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BaseParamPropertyClearAll", "Clear All Properties"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

void FFbxImportUIDetails::SetStaticMeshLODGroupWidget(IDetailPropertyRow& PropertyRow, const TSharedPtr<IPropertyHandle>& Handle)
{
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	FName InitialValue;
	ensure(Handle->GetValue(InitialValue) == FPropertyAccess::Success);
	int32 GroupIndex = LODGroupNames.Find(InitialValue);
	if (GroupIndex == INDEX_NONE && LODGroupNames.Num() > 0)
	{
		GroupIndex = 0;
	}
	check(GroupIndex != INDEX_NONE);
	StaticMeshLODGroupPropertyHandle = Handle;
	TWeakPtr<IPropertyHandle> HandlePtr = Handle;

	const bool bShowChildren = true;
	PropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		.VAlign(VAlign_Center)
		[
			SNew(STextComboBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OptionsSource(&LODGroupOptions)
			.InitiallySelectedItem(LODGroupOptions[GroupIndex])
			.OnSelectionChanged(this, &FFbxImportUIDetails::OnLODGroupChanged, HandlePtr)
		];
}

void FFbxImportUIDetails::OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandlePtr)
{
	TSharedPtr<IPropertyHandle> Handle = HandlePtr.Pin();
	if (Handle.IsValid())
	{
		int32 GroupIndex = LODGroupOptions.Find(NewValue);
		check(GroupIndex != INDEX_NONE);
		ensure(Handle->SetValue(LODGroupNames[GroupIndex]) == FPropertyAccess::Success);
	}
}

bool FFbxImportUIDetails::GetVertexOverrideColorEnabledState() const
{
	uint8 VertexColorImportOption;
	check(VertexColorImportOptionHandle.IsValid());
	ensure(VertexColorImportOptionHandle->GetValue(VertexColorImportOption) == FPropertyAccess::Success);

	return (VertexColorImportOption == EVertexColorImportOption::Override);
}


bool FFbxImportUIDetails::GetSkeletalMeshVertexOverrideColorEnabledState() const
{
	uint8 VertexColorImportOption;
	check(SkeletalMeshVertexColorImportOptionHandle.IsValid());
	ensure(SkeletalMeshVertexColorImportOptionHandle->GetValue(VertexColorImportOption) == FPropertyAccess::Success);

	return (VertexColorImportOption == EVertexColorImportOption::Override);
}

void FFbxImportUIDetails::CollectChildPropertiesRecursive(TSharedPtr<IPropertyHandle> Node, TArray<TSharedPtr<IPropertyHandle>>& OutProperties)
{
	uint32 NodeNumChildren = 0;
	Node->GetNumChildren(NodeNumChildren);

	for(uint32 ChildIdx = 0 ; ChildIdx < NodeNumChildren ; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = Node->GetChildHandle(ChildIdx);
		CollectChildPropertiesRecursive(ChildHandle, OutProperties);

		if(ChildHandle->GetProperty())
		{
			OutProperties.AddUnique(ChildHandle);
		}
	}
}

bool FFbxImportUIDetails::IsImportTypeMetaDataValid(EFBXImportType& ImportType, FString& MetaData)
{
	TArray<FString> Types;
	MetaData.ParseIntoArray(Types, TEXT("|"), 1);
	switch(ImportType)
	{
		case FBXIT_StaticMesh:
			return Types.Contains(TEXT("StaticMesh")) || Types.Contains(TEXT("Mesh"));
		case FBXIT_SkeletalMesh:
			return Types.Contains(TEXT("SkeletalMesh")) || Types.Contains(TEXT("Mesh"));
		case FBXIT_Animation:
			return Types.Contains(TEXT("Animation"));
		default:
			return false;
	}
}

void FFbxImportUIDetails::ImportAutoComputeLodDistancesChanged()
{
	//We need to update the LOD distance UI
	RefreshCustomDetail();
}

void FFbxImportUIDetails::ValidateLodSettingsChanged(int32 MemberID)
{
	//This feature is supported only for staticmesh
	if (ImportUI->MeshTypeToImport != FBXIT_StaticMesh)
	{
		return;
	}

	if (ImportUI->MinimumLodNumber < 0 || ImportUI->MinimumLodNumber >= MAX_STATIC_MESH_LODS)
	{
		ImportUI->MinimumLodNumber = FMath::Clamp<int32>(ImportUI->MinimumLodNumber, 0, MAX_STATIC_MESH_LODS -1);
	}
	if (ImportUI->LodNumber < 0 || ImportUI->LodNumber >= MAX_STATIC_MESH_LODS)
	{
		ImportUI->LodNumber = FMath::Clamp<int32>(ImportUI->LodNumber, 0, MAX_STATIC_MESH_LODS);
	}

	if (ImportUI->LodNumber > 0 && ImportUI->MinimumLodNumber >= ImportUI->LodNumber)
	{
		ImportUI->MinimumLodNumber = FMath::Clamp<int32>(ImportUI->MinimumLodNumber, 0, ImportUI->LodNumber - 1);
	}
	
	if (!ImportUI->bAutoComputeLodDistances && MemberID == LodNumberID)
	{
		RefreshCustomDetail();
	}
}

void FFbxImportUIDetails::ImportMaterialsChanged()
{
	//We need to update the Base Material UI
	RefreshCustomDetail();
}

void FFbxImportUIDetails::MeshImportModeChanged()
{
	ImportUI->SetMeshTypeToImport();
	RefreshCustomDetail();
}

void FFbxImportUIDetails::ImportMeshToggleChanged()
{
	if(ImportUI->bImportMesh)
	{
		ImportUI->SetMeshTypeToImport();
	}
	else
	{
		ImportUI->MeshTypeToImport = FBXIT_Animation;
	}
	RefreshCustomDetail();
}

void FFbxImportUIDetails::BaseMaterialChanged()
{
	RefreshCustomDetail();
}

void GetSelectionParameterString(TSharedPtr<FString> Selection, FString &OutParameterName)
{
	OutParameterName = *Selection.Get();
	if (OutParameterName.Equals(DoNotOverrideString))
	{
		OutParameterName.Empty();
	}
}

void FFbxImportUIDetails::OnBaseColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseColorName);
}

void FFbxImportUIDetails::OnDiffuseTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseDiffuseTextureName);
}

void FFbxImportUIDetails::OnNormalTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseNormalTextureName);
}

void FFbxImportUIDetails::OnEmmisiveTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseEmmisiveTextureName);
}

void FFbxImportUIDetails::OnEmissiveColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseEmissiveColorName);
}

void FFbxImportUIDetails::OnSpecularTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseSpecularTextureName);
}

void FFbxImportUIDetails::OnOpacityTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	GetSelectionParameterString(Selection, ImportUI->TextureImportData->BaseOpacityTextureName);
}

FReply FFbxImportUIDetails::MaterialBaseParamClearAllProperties()
{
	ImportUI->TextureImportData->BaseColorName.Empty();
	ImportUI->TextureImportData->BaseDiffuseTextureName.Empty();
	ImportUI->TextureImportData->BaseNormalTextureName.Empty();
	ImportUI->TextureImportData->BaseEmmisiveTextureName.Empty();
	ImportUI->TextureImportData->BaseEmissiveColorName.Empty();
	ImportUI->TextureImportData->BaseSpecularTextureName.Empty();
	ImportUI->TextureImportData->BaseOpacityTextureName.Empty();
	//Need to refresh the custom detail since we do not have any pointer on the combo box
	RefreshCustomDetail();
	return FReply::Handled();
}

int32 FFbxImportUIDetails::GetMaterialImportMethodValue() const
{
	return static_cast<int32>(SelectedMaterialImportMethod);
}

void FFbxImportUIDetails::OnMaterialImportMethodChanged(int32 Value, ESelectInfo::Type SelectInfo)
{
	SelectedMaterialImportMethod = static_cast<EMaterialImportMethod>(Value);

	switch (SelectedMaterialImportMethod)
	{
	case EMaterialImportMethod::CreateNewMaterials:
		ImportUI->bImportMaterials = true;
		//Reset the base material and the UseBaseMaterial flag to hide the base material name property
		ImportUI->TextureImportData->bUseBaseMaterial = false;
		ImportUI->TextureImportData->BaseMaterialName.Reset();
		break;
	case EMaterialImportMethod::CreateNewInstancedMaterials:
		ImportUI->bImportMaterials = true;
		ImportUI->TextureImportData->bUseBaseMaterial = true;
		break;
	case EMaterialImportMethod::DoNotCreateMaterialString:
		ImportUI->bImportMaterials = false;
		//Reset the base material and the UseBaseMaterial flag to hide the base material name property
		ImportUI->TextureImportData->bUseBaseMaterial = false;
		ImportUI->TextureImportData->BaseMaterialName.Reset();
		break;
	default:
		break;
	}
	RefreshCustomDetail();
}

FReply FFbxImportUIDetails::ShowConflictDialog(EConflictDialogType DialogType)
{
	if (DialogType == Conflict_Material)
	{
		ImportUI->OnShowMaterialConflictDialog.ExecuteIfBound();
	}
	else if (DialogType == Conflict_Skeleton)
	{
		ImportUI->OnShowSkeletonConflictDialog.ExecuteIfBound();
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
