// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EditorUndoClient.h"
#include "Factories/FbxImportUI.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "FbxImportUIDetails.generated.h"

class FString;
class FText;
class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;

enum EConflictDialogType
{
	Conflict_Material,
	Conflict_Skeleton
};

UENUM()
enum class EMaterialImportMethod : int32 
{
	CreateNewMaterials  UMETA(DisplayName = "Create New Materials", ToolTip = "A new material will be created from the imported data."),
	CreateNewInstancedMaterials UMETA(DisplayName = "Create New Instanced Materials", Tooltip = "A new material instance of the specified base material will be created and set with the imported material data."),
	DoNotCreateMaterialString UMETA(DisplayName = "Do Not Create Material", Tooltip = "No materials will be created from the import data."),
};

class FFbxImportUIDetails : public IDetailCustomization, public FEditorUndoClient
{
public:
	~FFbxImportUIDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	/** End IDetailCustomization interface */

	void AddSubCategory(IDetailLayoutBuilder& DetailBuilder, FName MainCategoryName, TMap<FString, TArray<TSharedPtr<IPropertyHandle>>>& SubCategoriesProperties, TMap<FString, bool >& SubCategoriesAdvanced, TMap<FString, FText >& SubCategoriesTooltip);

	void RefreshCustomDetail();

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	/** End FEditorUndoClient interface */

	void CollectChildPropertiesRecursive(TSharedPtr<IPropertyHandle> Node, TArray<TSharedPtr<IPropertyHandle>>& OutProperties);
	
	void ConstructMaterialImportMethod(TSharedPtr<IPropertyHandle> ImportMaterialPropHandle, class IDetailCategoryBuilder& MaterialCategory);

	void ConstructBaseMaterialUI(TSharedPtr<IPropertyHandle> Handle, class IDetailCategoryBuilder& MaterialCategory);

	/** Checks whether a metadata string is valid for a given import type 
	* @param ImportType the type of mesh being imported
	* @param MetaData the metadata string to validate
	*/
	bool IsImportTypeMetaDataValid(EFBXImportType& ImportType, FString& MetaData);
	
	/** Called if the bAutoComputeLodDistances changes */
	void ImportAutoComputeLodDistancesChanged();

	/** Called when the LODSettings changes */
	void ValidateLodSettingsChanged(int32 MemberID);

	/** Called if the bImportMaterials changes */
	void ImportMaterialsChanged();

	/** Called if the mesh mode (static / skeletal) changes */
	void MeshImportModeChanged();

	/** Called if the import mesh option for skeletal meshes is changed */
	void ImportMeshToggleChanged();

	/** Called when the base material is changed */
	void BaseMaterialChanged();

	/** Called user chooses base material properties */
	void OnBaseColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnDiffuseTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnNormalTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnEmmisiveTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnEmissiveColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnSpecularTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnOpacityTextureColor(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	FReply MaterialBaseParamClearAllProperties();

	int32 GetMaterialImportMethodValue() const;
	void OnMaterialImportMethodChanged(int32 Value, ESelectInfo::Type SelectInfo);
	
	TWeakObjectPtr<UFbxImportUI> ImportUI;		// The UI data object being customised
	IDetailLayoutBuilder* CachedDetailBuilder;	// The detail builder for this cusomtomisation

private:

	/** Use MakeInstance to create an instance of this class */
	FFbxImportUIDetails();

	/** Sets a custom widget for the StaticMeshLODGroup property */
	void SetStaticMeshLODGroupWidget(IDetailPropertyRow& PropertyRow, const TSharedPtr<IPropertyHandle>& Handle);

	/** Called when the StaticMeshLODGroup spinbox is changed */
	void OnLODGroupChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandlePtr);

	/** Called to determine the visibility of the VertexOverrideColor property */
	bool GetVertexOverrideColorEnabledState() const;
	bool GetSkeletalMeshVertexOverrideColorEnabledState() const;

	FReply ShowConflictDialog(EConflictDialogType DialogType);
	bool ShowCompareResult();

	/** LOD group options. */
	TArray<FName> LODGroupNames;
	TArray<TSharedPtr<FString>> LODGroupOptions;

	/** Cached StaticMeshLODGroup property handle */
	TSharedPtr<IPropertyHandle> StaticMeshLODGroupPropertyHandle;

	/** Cached VertexColorImportOption property handle */
	TSharedPtr<IPropertyHandle> VertexColorImportOptionHandle;
	TSharedPtr<IPropertyHandle> SkeletalMeshVertexColorImportOptionHandle;

	EMaterialImportMethod SelectedMaterialImportMethod;
	bool bShowBaseMaterialUI;

	TArray< TSharedPtr< FString > > BaseColorNames;
	TArray< TSharedPtr< FString > > BaseTextureNames;
};
