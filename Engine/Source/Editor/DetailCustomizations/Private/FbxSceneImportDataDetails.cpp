// Copyright Epic Games, Inc. All Rights Reserved.

#include "FbxSceneImportDataDetails.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Factories/FbxSceneImportData.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericWindow.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDesktopPlatform.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

class UObject;

#define LOCTEXT_NAMESPACE "FbxSceneImportDataDetails"


FFbxSceneImportDataDetails::FFbxSceneImportDataDetails()
{
	ImportData = nullptr;
}

TSharedRef<IDetailCustomization> FFbxSceneImportDataDetails::MakeInstance()
{
	return MakeShareable( new FFbxSceneImportDataDetails);
}

void FFbxSceneImportDataDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	check(EditingObjects.Num() == 1);

	IDetailCategoryBuilder &ImportSettingsCategory = DetailBuilder.EditCategory("ImportSettings");

	ImportData = Cast<UFbxSceneImportData>(EditingObjects[0].Get());

	// Grab and hide per-type import options
	SourceFileFbxHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFbxSceneImportData, SourceFbxFile));
	SourceFileFbxHandle->MarkHiddenByCustomization();

	IDetailPropertyRow &SourceFileFbxRow = ImportSettingsCategory.AddProperty(SourceFileFbxHandle);
	
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	SourceFileFbxRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	SourceFileValueWidget = ValueWidget;
	FDetailWidgetRow &DetailWidgetRow = SourceFileFbxRow.CustomWidget();
	DetailWidgetRow.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueContent().MaxWidth)
		.MaxDesiredWidth(Row.ValueContent().MaxWidth)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				ValueWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("SourceFbxFile_Browse", "Browse..."))
				.OnClicked(this, &FFbxSceneImportDataDetails::OnBrowseClicked)
			]
		];
}

FReply FFbxSceneImportDataDetails::OnBrowseClicked()
{
	if (!SourceFileValueWidget.IsValid() || !SourceFileFbxHandle.IsValid())
	{
		return FReply::Handled();
	}
	//Popup a selectfile dialog and store the result into the asset
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if (DesktopPlatform != NULL)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(SourceFileValueWidget.ToSharedRef());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;
		
		FString DefaultFile = TEXT("");
		FString DefaultPath = TEXT("");
		if (SourceFileFbxHandle->GetValue(DefaultFile) == FPropertyAccess::Success)
		{
			DefaultPath = FPaths::GetPath(DefaultFile);
			if (!FPaths::FileExists(DefaultFile))
			{
				DefaultFile = TEXT("");
			}
		}
		else
		{
			DefaultFile = TEXT("");
			DefaultPath = FPaths::GetPath(FPaths::GetProjectFilePath());
		}
		bOpened = DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			TEXT("Select FBX|OBJ file.."),
			DefaultPath,
			DefaultFile,
			TEXT("FBX file (*.fbx)|*.fbx|OBJ file (*.obj)|*.obj"),
			EFileDialogFlags::None,
			OpenFilenames);
	}

	if (bOpened == true)
	{
		if (OpenFilenames.Num() > 0)
		{
			SourceFileFbxHandle->SetValue(OpenFilenames[0]);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
