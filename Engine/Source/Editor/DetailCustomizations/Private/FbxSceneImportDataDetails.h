// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class UFbxSceneImportData;

class FFbxSceneImportDataDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	/** End IDetailCustomization interface */
	
	
	TWeakObjectPtr<UFbxSceneImportData> ImportData;		// The UI data object being customised

private:

	FReply OnBrowseClicked();

	/** Use MakeInstance to create an instance of this class */
	FFbxSceneImportDataDetails();

	//The path we want to change
	TSharedPtr<IPropertyHandle> SourceFileFbxHandle;
	//The value widget for the source path
	TSharedPtr<SWidget> SourceFileValueWidget;
};
