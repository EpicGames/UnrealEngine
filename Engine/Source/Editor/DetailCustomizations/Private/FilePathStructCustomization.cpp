// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilePathStructCustomization.h"

#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SFilePathPicker.h"


#define LOCTEXT_NAMESPACE "FilePathStructCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FFilePathStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	/* do nothing */
}


void FFilePathStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	PathStringProperty = StructPropertyHandle->GetChildHandle("FilePath");

	FString FileTypeFilter;

	// construct file type filter
	const FString& MetaData = StructPropertyHandle->GetMetaData(TEXT("FilePathFilter"));


	bLongPackageName = StructPropertyHandle->HasMetaData(TEXT("LongPackageName"));
	bRelativeToGameDir = StructPropertyHandle->HasMetaData(TEXT("RelativeToGameDir"));

	if (MetaData.IsEmpty())
	{
		FileTypeFilter = TEXT("All files (*.*)|*.*");
	}
	else
	{
		if (MetaData.Contains(TEXT("|"))) // If MetaData follows the Description|ExtensionList format, use it as is
		{
			FileTypeFilter = MetaData;
		}
		else
		{
			FileTypeFilter = FString::Printf(TEXT("%s files (*.%s)|*.%s"), *MetaData, *MetaData, *MetaData);
		}
	}

	// create path picker widget
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SFilePathPicker)
				.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
				.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
				.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
				.FilePath(this, &FFilePathStructCustomization::HandleFilePathPickerFilePath)
				.FileTypeFilter(FileTypeFilter)
				.OnPathPicked(this, &FFilePathStructCustomization::HandleFilePathPickerPathPicked)
		];
}


/* FFilePathStructCustomization callbacks
 *****************************************************************************/

FString FFilePathStructCustomization::HandleFilePathPickerFilePath( ) const
{
	FString FilePath;
	PathStringProperty->GetValue(FilePath);

	return FilePath;
}


void FFilePathStructCustomization::HandleFilePathPickerPathPicked( const FString& PickedPath )
{
	FString FinalPath = PickedPath;
	if (bLongPackageName)
	{
		FString LongPackageName;
		FString StringFailureReason;

		// Don't try to convert empty string
		if (PickedPath.Len() > 0)
		{
			if (FPackageName::TryConvertFilenameToLongPackageName(PickedPath, LongPackageName, &StringFailureReason) == false)
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(StringFailureReason));
			}
		}
		FinalPath = LongPackageName;
	}
	else if (bRelativeToGameDir && !PickedPath.IsEmpty())
	{
		//A filepath under the project directory will be made relative to the project directory
		//Otherwise, the absolute path will be returned unless it doesn't exist, the current path will
		//be kept. This can happen if it's already relative to project dir (tabbing when selected)

		const FString ProjectDir = FPaths::ProjectDir();
		const FString AbsoluteProjectDir = FPaths::ConvertRelativePathToFull(ProjectDir);
		const FString AbsolutePickedPath = FPaths::ConvertRelativePathToFull(PickedPath);

		//Verify if absolute path to file exists. If it was already relative to content directory
		//the absolute will be to binaries and will possibly be garbage
		if (FPaths::FileExists(AbsolutePickedPath))
		{
			//If file is part of the project dir, chop the project dir part
			//Otherwise, use the absolute path
			if (AbsolutePickedPath.StartsWith(AbsoluteProjectDir))
			{
				FinalPath = AbsolutePickedPath.RightChop(AbsoluteProjectDir.Len());
			}
			else
			{
				FinalPath = AbsolutePickedPath;
			}
		}
		else
		{
			//If absolute file doesn't exist, it might already be relative to project dir
			//If not, then it might be a manual entry, so keep it untouched either way
			FinalPath = PickedPath;
		}
	}

	PathStringProperty->SetValue(FinalPath);
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(PickedPath));
}


#undef LOCTEXT_NAMESPACE
