// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

/**
 * Implements a details view customization for the FFilePath structure.
 */
class FFilePathStructCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance( )
	{
		return MakeShareable(new FFilePathStructCustomization());
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:

	/** Callback for getting the selected path in the picker widget. */
	FString HandleFilePathPickerFilePath( ) const;

	/** Callback for picking a file in the file path picker. */
	void HandleFilePathPickerPathPicked( const FString& PickedPath );

private:
	/** Pointer to the string that will be seet when changing the path */
	TSharedPtr<IPropertyHandle> PathStringProperty;
	bool bLongPackageName;
	
	/** Whether the output file path tries to be relative to the project directory */
	bool bRelativeToGameDir;
};
