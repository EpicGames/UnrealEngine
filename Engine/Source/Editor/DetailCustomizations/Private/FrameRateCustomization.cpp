// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameRateCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FrameRate.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SFrameRatePicker.h"

class IDetailChildrenBuilder;


#define LOCTEXT_NAMESPACE "FrameRateCustomization"


TSharedRef<IPropertyTypeCustomization> FFrameRateCustomization::MakeInstance()
{
	return MakeShareable(new FFrameRateCustomization);
}


void FFrameRateCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructPropertyHandle = InPropertyHandle;

	TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]

	.ValueContent()
		[
			SNew(SFrameRatePicker)
			.Font(CustomizationUtils.GetRegularFont())
		.HasMultipleValues(this, &FFrameRateCustomization::HasMultipleValues)
		.Value(this, &FFrameRateCustomization::GetFirstFrameRate)
		.OnValueChanged(this, &FFrameRateCustomization::SetFrameRate)
		].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
}


void FFrameRateCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}


FFrameRate FFrameRateCustomization::GetFirstFrameRate() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	for (const void* RawPtr : RawData)
	{
		if (RawPtr)
		{
			return *reinterpret_cast<const FFrameRate*>(RawPtr);
		}
	}

	return FFrameRate();
}

void FFrameRateCustomization::SetFrameRate(FFrameRate NewFrameRate)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
	{
		static FFrameRate DefaultRate(24,1);

		TArray<void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);
		FFrameRate* PreviousFrameRate = RawData.Num() == 1 ? reinterpret_cast<FFrameRate*>(RawData[0]) : &DefaultRate;

		FString TextValue;
		StructProperty->Struct->ExportText(TextValue, &NewFrameRate, PreviousFrameRate, nullptr, EPropertyPortFlags::PPF_None, nullptr);
		ensure(StructPropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
	}
}

bool FFrameRateCustomization::HasMultipleValues() const
{
	TArray<const void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);

	TOptional<FFrameRate> CompareAgainst;
	for (const void* RawPtr : RawData)
	{
		if (RawPtr == nullptr)
		{
			if (CompareAgainst.IsSet())
			{
				return false;
			}
		}
		else
		{
			FFrameRate ThisRate = *reinterpret_cast<const FFrameRate*>(RawPtr);

			if (!CompareAgainst.IsSet())
			{
				CompareAgainst = ThisRate;
			}
			else if (ThisRate != CompareAgainst.GetValue())
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
