// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
struct FFrameRate;

class FFrameRateCustomization : public IPropertyTypeCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:

	// IDetailCustomization interface

	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;

private:


	FFrameRate GetFirstFrameRate() const;
	void SetFrameRate(FFrameRate NewFrameRate);
	bool HasMultipleValues() const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
