// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeneralProjectSettingsDetails.h"

#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/App.h"
#include "Misc/Attribute.h"
#include "Misc/Paths.h"
#include "SExternalImageReference.h"
#include "SlotBase.h"
#include "SourceControlOperations.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FGeneralProjectSettingsDetails"

/////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FGeneralProjectSettingsDetails::MakeInstance()
{
	return MakeShareable(new FGeneralProjectSettingsDetails);
}

void FGeneralProjectSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& ImageCategory = DetailBuilder.EditCategory(TEXT("About"));

	const FText ProjectThumbnailDesc(LOCTEXT("ProjectThumbnailLabel", "Project Thumbnail"));

	const FString ProjectThumbnail_TargetImagePath = FPaths::GetPath(FPaths::GetProjectFilePath()) / FString::Printf(TEXT("%s.png"), FApp::GetProjectName());
	FString ProjectThumbnail_AutomaticImagePath = FPaths::ProjectSavedDir() / TEXT("AutoScreenshot.png");
	if (!FPaths::FileExists(ProjectThumbnail_AutomaticImagePath))
	{
		ProjectThumbnail_AutomaticImagePath = FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate") / TEXT("GameProjectDialog") / TEXT("default_game_thumbnail.png");
	}

	ImageCategory.AddCustomRow(ProjectThumbnailDesc)
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(ProjectThumbnailDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, ProjectThumbnail_AutomaticImagePath, ProjectThumbnail_TargetImagePath)
			.FileDescription(ProjectThumbnailDesc)
			.RequiredSize(FIntPoint(192, 192))
		]
	];

}

#undef LOCTEXT_NAMESPACE
