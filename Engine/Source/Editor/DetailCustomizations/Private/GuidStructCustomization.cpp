// Copyright Epic Games, Inc. All Rights Reserved.

#include "GuidStructCustomization.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FGuidStructCustomization"


/* IPropertyTypeCustomization interface
 *****************************************************************************/

void FGuidStructCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	PropertyHandle = StructPropertyHandle;
	InputValid = true;

	TSharedPtr<SWidget> QuickSetSlotContent;

	// create quick-set menu if needed
	if (PropertyHandle->IsEditConst())
	{
		QuickSetSlotContent = SNullWidget::NullWidget;
	}
	else
	{
		QuickSetSlotContent = 
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.OnGetMenuContent_Lambda([this]()
			{
				FMenuBuilder QuickSetMenuBuilder(true, nullptr);
				{
					FUIAction GenerateAction(FExecuteAction::CreateSP(this, &FGuidStructCustomization::HandleGuidActionClicked, EPropertyEditorGuidActions::Generate));
					QuickSetMenuBuilder.AddMenuEntry(LOCTEXT("GenerateAction", "Generate"), LOCTEXT("GenerateActionHint", "Generate a new random globally unique identifier (GUID)."), FSlateIcon(), GenerateAction);

					FUIAction InvalidateAction(FExecuteAction::CreateSP(this, &FGuidStructCustomization::HandleGuidActionClicked, EPropertyEditorGuidActions::Invalidate));
					QuickSetMenuBuilder.AddMenuEntry(LOCTEXT("InvalidateAction", "Invalidate"), LOCTEXT("InvalidateActionHint", "Set an invalid globally unique identifier (GUID)."), FSlateIcon(), InvalidateAction);
				}
				return QuickSetMenuBuilder.MakeWidget();
			});
	}

	// create struct header
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(325.0f)
		.MaxDesiredWidth(325.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					// text box
					SAssignNew(TextBox, SEditableTextBox)
						.ClearKeyboardFocusOnCommit(false)
						.IsEnabled_Lambda([PropertyHandleWeak = PropertyHandle->AsWeak()]()->bool
						{
							if (TSharedPtr<IPropertyHandle> PropertyHandleLocal = PropertyHandleWeak.Pin())
							{
								return !PropertyHandleLocal->IsEditConst();
							}
							return false;
						})
						.ForegroundColor(this, &FGuidStructCustomization::HandleTextBoxForegroundColor)
						.OnTextChanged(this, &FGuidStructCustomization::HandleTextBoxTextChanged)
						.OnTextCommitted(this, &FGuidStructCustomization::HandleTextBoxTextCommited)
						.SelectAllTextOnCommit(true)
						.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
						.Text(this, &FGuidStructCustomization::HandleTextBoxText)
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					// quick set menu
					QuickSetSlotContent.ToSharedRef()
				]
		];
}


void FGuidStructCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	// do nothing
}


/* FGuidStructCustomization implementation
 *****************************************************************************/

void FGuidStructCustomization::SetGuidValue( const FGuid& Guid )
{
	WriteGuidToProperty(PropertyHandle, Guid);
}


/* FGuidStructCustomization callbacks
 *****************************************************************************/

void FGuidStructCustomization::HandleGuidActionClicked( EPropertyEditorGuidActions::Type Action )
{
	// Clear focus so text field can be updated
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);

	if (Action == EPropertyEditorGuidActions::Generate)
	{
		SetGuidValue(FGuid::NewGuid());
		InputValid = true;
	}
	else if (Action == EPropertyEditorGuidActions::Invalidate)
	{
		SetGuidValue(FGuid());
		InputValid = true;
	}
}


FSlateColor FGuidStructCustomization::HandleTextBoxForegroundColor() const
{
	if (InputValid)
	{
		static const FName DefaultForeground("Colors.Foreground");
		return FAppStyle::Get().GetSlateColor(DefaultForeground);
	}

	static const FName Red("Colors.AccentRed");

	return FAppStyle::Get().GetSlateColor(Red);
}


FText FGuidStructCustomization::HandleTextBoxText( ) const
{
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	
	if (RawData.Num() != 1)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	if (RawData[0] == nullptr)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(((FGuid*)RawData[0])->ToString(EGuidFormats::DigitsWithHyphensInBraces));
}


void FGuidStructCustomization::HandleTextBoxTextChanged( const FText& NewText )
{
	FGuid Guid;
	InputValid = FGuid::Parse(NewText.ToString(), Guid);
}


void FGuidStructCustomization::HandleTextBoxTextCommited( const FText& NewText, ETextCommit::Type CommitInfo )
{
	FGuid ParsedGuid;
								
	if (FGuid::Parse(NewText.ToString(), ParsedGuid))
	{
		SetGuidValue(ParsedGuid);
	}
}

void WriteGuidToProperty(TSharedPtr<IPropertyHandle> GuidPropertyHandle, const FGuid& Guid)
{
	FScopedTransaction Transaction(FText::Format(LOCTEXT("EditGuidPropertyTransaction", "Edit {0}"), GuidPropertyHandle->GetPropertyDisplayName()));
	
	for (uint32 ChildIndex = 0; ChildIndex < 4; ++ChildIndex)
	{
		// Do not want a transaction on each individual set call as our scope transaction will handle it all
		// Need to mark first 3 as interactive so that post edit doesn't reinstance anything until we're done
		EPropertyValueSetFlags::Type GuidComponentFlags = EPropertyValueSetFlags::NotTransactable;
		TSharedRef<IPropertyHandle> ChildHandle = GuidPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		ChildHandle->SetValue((int32)Guid[ChildIndex], ChildIndex != 3 ? GuidComponentFlags | EPropertyValueSetFlags::InteractiveChange : GuidComponentFlags);
	}
}

#undef LOCTEXT_NAMESPACE
