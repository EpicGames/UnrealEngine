// Copyright Epic Games, Inc. All Rights Reserved.

#include "HardwareTargetingSettingsDetails.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HardwareTargetingModule.h"
#include "HardwareTargetingSettings.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Paths.h"
#include "PropertyHandle.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealEdMisc.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FHardwareTargetingSettingsDetails"

TSharedRef<IDetailCustomization> FHardwareTargetingSettingsDetails::MakeInstance()
{
	return MakeShareable(new FHardwareTargetingSettingsDetails);
}

class SRequiredDefaultConfig : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SRequiredDefaultConfig) {}

	SLATE_END_ARGS()

	~SRequiredDefaultConfig()
	{
		GetMutableDefault<UHardwareTargetingSettings>()->OnSettingChanged().RemoveAll(this);
	}

private:

	static FReply Apply()
	{
		IHardwareTargetingModule& Module = IHardwareTargetingModule::Get();
		Module.ApplyHardwareTargetingSettings();
		return FReply::Handled();
	}

public:

	TMap<TWeakObjectPtr<UObject>, TSharedPtr<SRichTextBlock>> SettingRegions;

	void Construct(const FArguments& InArgs)
	{
		LastStatusUpdate = 0;
		GetMutableDefault<UHardwareTargetingSettings>()->OnSettingChanged().AddRaw(this, &SRequiredDefaultConfig::Update);

		auto ApplyNow = []{
			Apply();
			FUnrealEdMisc::Get().RestartEditor(false);
			return FReply::Handled();
		};

		ChildSlot
		[

			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RestartMessage", "The following changes will be applied when this project is reopened."))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("RestartEditor", "Restart Editor"))
					.IsEnabled(this, &SRequiredDefaultConfig::CanApply)
					.OnClicked_Static(ApplyNow)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("ApplyLater", "Apply Later"))
					.IsEnabled(this, &SRequiredDefaultConfig::CanApply)
					.OnClicked_Static(&SRequiredDefaultConfig::Apply)
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(CheckoutNotices, SVerticalBox)
			]
		];
	}

	static EVisibility GetAnyPendingChangesVisibility()
	{
		return GetMutableDefault<UHardwareTargetingSettings>()->HasPendingChanges() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void Initialize(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& PendingChangesCategory)
	{
		auto IsVisible = []()->EVisibility{
			auto* Settings = GetMutableDefault<UHardwareTargetingSettings>();
			return Settings->HasPendingChanges() ? EVisibility::Visible : EVisibility::Collapsed;
		};

		FText CategoryHeaderTooltip = LOCTEXT("CategoryHeaderTooltip", "List of properties modified in this project setting category");

		IHardwareTargetingModule& Module = IHardwareTargetingModule::Get();
		for (const FModifiedDefaultConfig& Settings : Module.GetPendingSettingsChanges())
		{
			TSharedRef<SRichTextBlock> EditPropertiesBlock =
				SNew(SRichTextBlock)
				.AutoWrapText(false)
				.Justification(ETextJustify::Left)
				.TextStyle(FAppStyle::Get(), "HardwareTargets.Normal")
				.DecoratorStyleSet(&FAppStyle::Get());

			SettingRegions.Add(Settings.SettingsObject, EditPropertiesBlock);

			FDetailWidgetRow& CategoryRow = PendingChangesCategory.AddCustomRow(Settings.CategoryHeading)
				.Visibility(TAttribute<EVisibility>::Create(&SRequiredDefaultConfig::GetAnyPendingChangesVisibility))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(Settings.CategoryHeading)
					.ToolTipText(CategoryHeaderTooltip)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				.MaxDesiredWidth(300.0f)
				[
					EditPropertiesBlock
				];
		}

		Update();
	}

	bool CanApply() const
	{
		for (const auto& Watcher : FileWatcherWidgets)
		{
			if (!Watcher->IsUnlocked())
			{
				return false;
			}
		}

		return true;
	}

	void Update()
	{
		FileWatcherWidgets.Reset();
		CheckoutNotices->ClearChildren();

		IHardwareTargetingModule& Module = IHardwareTargetingModule::Get();
		int32 SlotIndex = 0;

		// Run thru the settings and push changes to the existing settings, as well as build a list of inis that will need to be edited
		TSet<FString> SeenConfigFiles;
		for (const FModifiedDefaultConfig& Settings : Module.GetPendingSettingsChanges())
		{
			if (!Settings.SettingsObject.IsValid())
			{
				continue;
			}

			SettingRegions.FindChecked(Settings.SettingsObject)->SetText(Settings.Description);

			// @todo userconfig: Should we check for GlobalUserConfig here? It's not ever going to be checked in...
			if (!Settings.SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config | CLASS_DefaultConfig /*| CLASS_GlobalUserConfig | CLASS_ProjectUserConfig*/))
			{
				continue;
			}
			

			FString ConfigFile = FPaths::ConvertRelativePathToFull(Settings.SettingsObject->GetDefaultConfigFilename());

			if (!SeenConfigFiles.Contains(ConfigFile))
			{
				SeenConfigFiles.Add(ConfigFile);

				TSharedRef<SSettingsEditorCheckoutNotice> FileWatcherWidget =
					SNew(SSettingsEditorCheckoutNotice)
					.ConfigFilePath(ConfigFile);
				FileWatcherWidgets.Add(FileWatcherWidget);

				CheckoutNotices->AddSlot()
				.Padding(FMargin(0.f, 0.f, 12.f, 5.f))
				[
					FileWatcherWidget
				];
				SlotIndex++;
			}
		}
	}

	TArray<TSharedPtr<SSettingsEditorCheckoutNotice>> FileWatcherWidgets;

	TSharedPtr<SVerticalBox> CheckoutNotices;

	double LastStatusUpdate;
};

void FHardwareTargetingSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{	
	IDetailCategoryBuilder& HardwareTargetingCategory = DetailBuilder.EditCategory(TEXT("Target Hardware"));
	IDetailCategoryBuilder& PendingChangesCategory = DetailBuilder.EditCategory(TEXT("Pending Changes"));

	auto AnyPendingChangesVisible = []()->EVisibility{
		auto* Settings = GetMutableDefault<UHardwareTargetingSettings>();
		return Settings->HasPendingChanges() ? EVisibility::Visible : EVisibility::Collapsed;
	};
	auto NoPendingChangesVisible = []()->EVisibility{
		auto* Settings = GetMutableDefault<UHardwareTargetingSettings>();
		return Settings->HasPendingChanges() ? EVisibility::Collapsed : EVisibility::Visible;
	};

	TSharedRef<SRequiredDefaultConfig> ConfigWidget = SNew(SRequiredDefaultConfig);
	IHardwareTargetingModule& HardwareTargeting = IHardwareTargetingModule::Get();
	PendingChangesCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SBox)
			.Visibility_Static(AnyPendingChangesVisible)
			[
				ConfigWidget
			]
		]
		+SVerticalBox::Slot()
		[
			SNew(SBox)
			.Visibility_Static(NoPendingChangesVisible)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("NoPendingChangesMessage", "There are no pending settings changes."))
			]
		]
	];

	ConfigWidget->Initialize(DetailBuilder, PendingChangesCategory);

	TSharedPtr<SWidget> HardwareClassCombo, GraphicsPresetCombo;

	// Setup the hardware class combo
	{
		auto PropertyName = GET_MEMBER_NAME_CHECKED(UHardwareTargetingSettings, TargetedHardwareClass);
		DetailBuilder.HideProperty(PropertyName);

		TSharedRef<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyName);
		auto SetPropertyValue = [](EHardwareClass NewValue, TSharedRef<IPropertyHandle> InProperty){
			InProperty->SetValue(uint8(NewValue));
		};
		auto GetPropertyValue = [](TSharedRef<IPropertyHandle> InProperty){
			uint8 Value = 0;
			InProperty->GetValue(Value);
			return EHardwareClass(Value);
		};
		
		HardwareClassCombo = HardwareTargeting.MakeHardwareClassTargetCombo(
			FOnHardwareClassChanged::CreateStatic(SetPropertyValue, Property),
			TAttribute<EHardwareClass>::Create(TAttribute<EHardwareClass>::FGetter::CreateStatic(GetPropertyValue, Property))
		);
	}

	// Setup the graphics preset combo
	{
		auto PropertyName = GET_MEMBER_NAME_CHECKED(UHardwareTargetingSettings, DefaultGraphicsPerformance);
		DetailBuilder.HideProperty(PropertyName);

		TSharedRef<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyName);
		auto SetPropertyValue = [](EGraphicsPreset NewValue, TSharedRef<IPropertyHandle> InProperty){
			InProperty->SetValue(uint8(NewValue));
		};
		auto GetPropertyValue = [](TSharedRef<IPropertyHandle> InProperty){
			uint8 Value = 0;
			InProperty->GetValue(Value);
			return EGraphicsPreset(Value);
		};
		GraphicsPresetCombo = HardwareTargeting.MakeGraphicsPresetTargetCombo(
			FOnGraphicsPresetChanged::CreateStatic(SetPropertyValue, Property),
			TAttribute<EGraphicsPreset>::Create(TAttribute<EGraphicsPreset>::FGetter::CreateStatic(GetPropertyValue, Property))
		);
	}

	HardwareTargetingCategory.AddCustomRow(LOCTEXT("HardwareTargetingOption", "Targeted Hardware"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("OptimizeProjectFor", "Optimize project settings for"))
		.Font(DetailBuilder.GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(0.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(10.f, 0.f))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			HardwareClassCombo.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.f, 0.f, 10.f, 0.f))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			GraphicsPresetCombo.ToSharedRef()
		]
	];
}

#undef LOCTEXT_NAMESPACE
