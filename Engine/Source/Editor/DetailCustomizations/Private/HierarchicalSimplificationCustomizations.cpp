// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalSimplificationCustomizations.h"

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "GameFramework/WorldSettings.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IGeometryProcessingInterfacesModule.h"
#include "IMeshReductionInterfaces.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "HierarchicalSimplificationCustomizations"

TSharedRef<IPropertyTypeCustomization> FHierarchicalSimplificationCustomizations::MakeInstance() 
{
	return MakeShareable( new FHierarchicalSimplificationCustomizations );
}

void FHierarchicalSimplificationCustomizations::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FFormatOrderedArguments Args;
	Args.Add(StructPropertyHandle->GetPropertyDisplayName());
	FText Name = FText::Format(LOCTEXT("HLODLevelName", "HLOD Level {0}"), Args);
	
	HeaderRow.
	NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget(Name)
	]
	.ValueContent()
	[
		StructPropertyHandle->CreatePropertyValueWidget(false)
	];
}

void FHierarchicalSimplificationCustomizations::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren( NumChildren );	
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;	
	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle( ChildIndex ).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}
	
	// Create two sub-settings groups for clean overview
	IDetailGroup& ClusterGroup = ChildBuilder.AddGroup(NAME_None, FText::FromString("Cluster generation settings"));
	IDetailGroup& MergeGroup = ChildBuilder.AddGroup(NAME_None, FText::FromString("Mesh generation settings"));

	// Retrieve special case properties
	SimplificationMethodPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, SimplificationMethod));
	TSharedPtr< IPropertyHandle > ProxyMeshSettingPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, ProxySetting));
	TSharedPtr< IPropertyHandle > MergeMeshSettingPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, MergeSetting));
	TSharedPtr< IPropertyHandle > ApproximateMeshSettingsPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, ApproximateSettings));
	TSharedPtr< IPropertyHandle > TransitionScreenSizePropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, TransitionScreenSize));
	TSharedPtr< IPropertyHandle > OverrideDrawDistancePropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, OverrideDrawDistance));
	TSharedPtr< IPropertyHandle > ReusePreviousLevelClustersPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FHierarchicalSimplification, bReusePreviousLevelClusters));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// Handle special property cases (done inside the loop to maintain order according to the struct
		if (Iter.Value() == SimplificationMethodPropertyHandle)
		{
			IDetailPropertyRow& SimplifyMeshRow = MergeGroup.AddPropertyRow(SimplificationMethodPropertyHandle.ToSharedRef());
			AddResetToDefaultOverrides(SimplifyMeshRow);

			static const UEnum* HLODSimplificationMethodEnum = StaticEnum<EHierarchicalSimplificationMethod>();
	
			// Determine whether or not there is a mesh merging interface available (SimplygonMeshReduction/SimplygonSwarm)
			IMeshReductionModule* ReductionModule = FModuleManager::Get().LoadModulePtr<IMeshReductionModule>("MeshReductionInterface");
			if (ReductionModule == nullptr || ReductionModule->GetMeshMergingInterface() == nullptr)
			{
				static FText RestrictReason = LOCTEXT("SimplifySimplificationMethodUnavailable", "Simplify method is not available, MeshReductionInterface module is missing");
				TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShared<FPropertyRestriction>(RestrictReason);
				EnumRestriction->AddHiddenValue(HLODSimplificationMethodEnum->GetNameStringByValue((int64)EHierarchicalSimplificationMethod::Simplify));
				SimplificationMethodPropertyHandle->AddRestriction(EnumRestriction.ToSharedRef());
			}			

			IGeometryProcessingInterfacesModule* GeometryProcessingInterfacesModule = FModuleManager::Get().LoadModulePtr<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
			if (GeometryProcessingInterfacesModule == nullptr || GeometryProcessingInterfacesModule->GetApproximateActorsImplementation() == nullptr)
			{
				static FText RestrictReason = LOCTEXT("ApproximateSimplificationMethodUnavailable", "Approximate method is not available, GeometryProcessingInterfaces module is missing");
				TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShared<FPropertyRestriction>(RestrictReason);
				EnumRestriction->AddHiddenValue(HLODSimplificationMethodEnum->GetNameStringByValue((int64)EHierarchicalSimplificationMethod::Approximate));
				SimplificationMethodPropertyHandle->AddRestriction(EnumRestriction.ToSharedRef());
			}
		}
		else if (Iter.Value() == ProxyMeshSettingPropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = MergeGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);

			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FHierarchicalSimplificationCustomizations::IsProxyMeshSettingVisible));
		}
		else if (Iter.Value() == MergeMeshSettingPropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = MergeGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);

			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FHierarchicalSimplificationCustomizations::IsMergeMeshSettingVisible));
		}
		else if (Iter.Value() == ApproximateMeshSettingsPropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = MergeGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);

			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FHierarchicalSimplificationCustomizations::IsApproximateMeshSettingVisible));
		}		
		else  if (Iter.Value() == TransitionScreenSizePropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = MergeGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);
		}
		else if (Iter.Value() == OverrideDrawDistancePropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = MergeGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);
		}
		else if (Iter.Value() == ReusePreviousLevelClustersPropertyHandle)
		{
			IDetailPropertyRow& SettingsRow = ClusterGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);

			uint32 Index = StructPropertyHandle->GetIndexInArray();
			// Hide the property for HLOD level 0
			SettingsRow.Visibility(TAttribute<EVisibility>::Create([Index]()
			{
				return (Index == INDEX_NONE || Index > 0) ? EVisibility::Visible : EVisibility::Collapsed;
			}));
		}
		else
		{
			IDetailPropertyRow& SettingsRow = ClusterGroup.AddPropertyRow(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);
		}
	}
}

EHierarchicalSimplificationMethod FHierarchicalSimplificationCustomizations::GetSelectedSimplificationMethod() const
{
	uint8 SimplificationMethod = 0;
	if (SimplificationMethodPropertyHandle)
	{
		SimplificationMethodPropertyHandle->GetValue(SimplificationMethod);
	}
	return (EHierarchicalSimplificationMethod)SimplificationMethod;
}

EVisibility FHierarchicalSimplificationCustomizations::IsProxyMeshSettingVisible() const
{
	return GetSelectedSimplificationMethod() == EHierarchicalSimplificationMethod::Simplify ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FHierarchicalSimplificationCustomizations::IsMergeMeshSettingVisible() const
{
	return GetSelectedSimplificationMethod() == EHierarchicalSimplificationMethod::Merge ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FHierarchicalSimplificationCustomizations::IsApproximateMeshSettingVisible() const
{
	return GetSelectedSimplificationMethod() == EHierarchicalSimplificationMethod::Approximate ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
