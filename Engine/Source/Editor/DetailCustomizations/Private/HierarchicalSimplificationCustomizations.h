// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HLOD/HLODSetup.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "OverrideResetToDefault.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IPropertyHandle;

class FHierarchicalSimplificationCustomizations : public IPropertyTypeCustomization, public TOverrideResetToDefaultWithStaticUStruct<FHierarchicalSimplification>
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:
	EHierarchicalSimplificationMethod GetSelectedSimplificationMethod() const;

	EVisibility IsProxyMeshSettingVisible() const;
	EVisibility IsMergeMeshSettingVisible() const;
	EVisibility IsApproximateMeshSettingVisible() const;

	TSharedPtr< IPropertyHandle > SimplificationMethodPropertyHandle;
};
