// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImportantToggleSettingCustomization.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/ImportantToggleSettingInterface.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformProcess.h"
#include "IDetailPropertyRow.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ImportantToggleSettingCustomization"

class SImportantToggleButton : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SImportantToggleButton)
		: _Text()
	{}

		SLATE_STYLE_ARGUMENT(FCheckBoxStyle, CheckBoxStyle)
		SLATE_ARGUMENT(FText, Text)
		SLATE_ARGUMENT(FText, ToolTipText)
		SLATE_ATTRIBUTE(bool, IsSet)
		SLATE_EVENT(FSimpleDelegate, OnToggled)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnToggled = InArgs._OnToggled;
		IsSetAttribute = InArgs._IsSet;

		FSlateFontInfo LargeDetailsFont = IDetailLayoutBuilder::GetDetailFontBold();
		LargeDetailsFont.Size += 4;

		ChildSlot
		[
			SNew(SCheckBox)
			.Style(InArgs._CheckBoxStyle)
			.IsChecked(this, &SImportantToggleButton::GetCheckedState)
			.OnCheckStateChanged(this, &SImportantToggleButton::OnClick)
			.ToolTipText(InArgs._ToolTipText)
			.Padding(FMargin(16.0f, 12.0f))
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(true)
			[
				SNew(STextBlock)
				.Text(InArgs._Text)
				.Font(LargeDetailsFont)
			]
		];
	}

private:
	void OnClick(ECheckBoxState State)
	{
		OnToggled.ExecuteIfBound();
	}

	ECheckBoxState GetCheckedState() const
	{
		return IsSetAttribute.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

private:
	TAttribute<bool> IsSetAttribute;
	FSimpleDelegate OnToggled;
};

//====================================================================================
// FImportantToggleSettingCustomization
//====================================================================================

TSharedRef<IDetailCustomization> FImportantToggleSettingCustomization::MakeInstance()
{
	return MakeShareable(new FImportantToggleSettingCustomization);
}

void FImportantToggleSettingCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() == 1)
	{
		ToggleSettingObject = Objects[0];
		IImportantToggleSettingInterface* ToggleSettingInterface = Cast<IImportantToggleSettingInterface>(ToggleSettingObject.Get());

		if (ToggleSettingInterface != nullptr)
		{
			FName CategoryName;
			FName PropertyName;
			ToggleSettingInterface->GetToggleCategoryAndPropertyNames(CategoryName, PropertyName);

			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName);
			TogglePropertyHandle = DetailBuilder.GetProperty(PropertyName);

			FSlateFontInfo StateDescriptionFont = IDetailLayoutBuilder::GetDetailFont();
			StateDescriptionFont.Size += 4;

			// Customize collision section
			Category.InitiallyCollapsed(false)
			.AddProperty(TogglePropertyHandle)
			.ShouldAutoExpand(true)
			.CustomWidget()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 12.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImportantToggleButton)
						.CheckBoxStyle(FAppStyle::Get(), "Property.ToggleButton.Start")
						.Text(ToggleSettingInterface->GetFalseStateLabel())
						.ToolTipText(ToggleSettingInterface->GetFalseStateTooltip())
						.IsSet(this, &FImportantToggleSettingCustomization::IsToggleValue, false)
						.OnToggled(this, &FImportantToggleSettingCustomization::OnToggledTo, false)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImportantToggleButton)
						.CheckBoxStyle(FAppStyle::Get(), "Property.ToggleButton.End")
						.Text(ToggleSettingInterface->GetTrueStateLabel())
						.ToolTipText(ToggleSettingInterface->GetTrueStateTooltip())
						.IsSet(this, &FImportantToggleSettingCustomization::IsToggleValue, true)
						.OnToggled(this, &FImportantToggleSettingCustomization::OnToggledTo, true)
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.Padding(0.0f, 12.0f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(SHyperlink)
							.Text(ToggleSettingInterface->GetAdditionalInfoUrlLabel())
							.OnNavigate(this, &FImportantToggleSettingCustomization::OnNavigateHyperlink, ToggleSettingInterface->GetAdditionalInfoUrl())
						]
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 12.0f)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(this, &FImportantToggleSettingCustomization::GetDescriptionText)
					.Font(StateDescriptionFont)
				]
			];			
		}
	}


}

bool FImportantToggleSettingCustomization::IsToggleValue(bool bValue) const
{
	bool bPropertyValue = false;
	TogglePropertyHandle->GetValue(bPropertyValue);
	return bPropertyValue == bValue;
}

void FImportantToggleSettingCustomization::OnToggledTo(bool bSetTo)
{
	TogglePropertyHandle->SetValue(bSetTo);
}

void FImportantToggleSettingCustomization::OnNavigateHyperlink(FString Url)
{
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
}

FText FImportantToggleSettingCustomization::GetDescriptionText() const
{
	IImportantToggleSettingInterface* ToogleSettingInterface = Cast<IImportantToggleSettingInterface>(ToggleSettingObject.Get());

	if (ToogleSettingInterface != nullptr)
	{
		bool bPropertyValue = false;
		TogglePropertyHandle->GetValue(bPropertyValue);

		if (bPropertyValue)
		{
			return ToogleSettingInterface->GetTrueStateDescription();
		}
		else
		{
			return ToogleSettingInterface->GetFalseStateDescription();
		}
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
