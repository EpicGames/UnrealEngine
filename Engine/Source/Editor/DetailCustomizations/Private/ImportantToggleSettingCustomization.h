// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UObject;

//class FImportantToggleSettingCustomization : public IPropertyTypeCustomization
//{
//public:
//	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
//
//	/** IPropertyTypeCustomization instance */
//	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
//	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
//
//protected:
//	TSharedPtr<IPropertyHandle> ToggleProperty;
//};

class FImportantToggleSettingCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

private:
	//ECheckBoxState IsBoolPropertyTrue(TSharedRef<IPropertyHandle> PropertyHandle) const;
	//void OnCheckStateChanged(ECheckBoxState CheckBoxState, TSharedRef<IPropertyHandle> PropertyHandle);

	bool IsToggleValue(bool bValue) const;
	void OnToggledTo(bool bSetTo);
	void OnNavigateHyperlink(FString Url);
	FText GetDescriptionText() const;

private:
	TSharedPtr<IPropertyHandle> TogglePropertyHandle;
	TWeakObjectPtr<UObject> ToggleSettingObject;
};
