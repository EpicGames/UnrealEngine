// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputSettingsDetails.h"

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerInput.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "InputSettingsDetails"

namespace UE
{
	static void HandleInputDeprecationStatusChanged(UInputSettings* InputSettings, ECheckBoxState NewState, const FName MappingName)
	{
		if (!InputSettings)
		{
			return;
		}
		
		const FScopedTransaction Transaction(LOCTEXT("DeprecateActionMappingGroup_Transaction", "Deprecate action mapping group"));
		
		InputSettings->Modify();

		const bool bShouldBeDeprecated = NewState == ECheckBoxState::Checked;
		InputSettings->SetActionMappingDeprecated(MappingName, bShouldBeDeprecated);

		// Make sure to save out the default config file changes
		InputSettings->SaveKeyMappings();
		InputSettings->TryUpdateDefaultConfigFile();
	}

	static ECheckBoxState GetInputDeprecationCheckBoxState(UInputSettings* InputSettings, const FName MappingName)
	{
		if (InputSettings)
		{
			const bool bIsDeprecated = InputSettings->IsActionMappingDeprecated(MappingName);
			return bIsDeprecated ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Undetermined;
	}
}

////////////////////////////////
// FActionMappingsNodeBuilder //
////////////////////////////////

FName InputSettingsDetails::FMappingSet::GetActionMappingName() const
{
	// Get the name of the mapping to deprecate
	FName MappingName = NAME_None;
	if (Mappings.Num() > 0)
	{
		Mappings[0]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, ActionName))->GetValue(MappingName);
	}
	
	return MappingName;
}

FName InputSettingsDetails::FMappingSet::GetAxisMappingName() const
{
	FName MappingName = NAME_None;
	if (Mappings.Num() > 0)
	{
		Mappings[0]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, AxisName))->GetValue(MappingName);
	}
	return MappingName;
}


FActionMappingsNodeBuilder::FActionMappingsNodeBuilder( IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle )
	: DetailLayoutBuilder( InDetailLayoutBuilder )
	, ActionMappingsPropertyHandle( InPropertyHandle )
{
	// Delegate for when the children in the array change
	FSimpleDelegate RebuildChildrenDelegate = FSimpleDelegate::CreateRaw( this, &FActionMappingsNodeBuilder::RebuildChildren );
	ActionMappingsPropertyHandle->SetOnPropertyValueChanged( RebuildChildrenDelegate );
	ActionMappingsPropertyHandle->AsArray()->SetOnNumElementsChanged( RebuildChildrenDelegate );
}

void FActionMappingsNodeBuilder::Tick( float DeltaTime )
{
	if (GroupsRequireRebuild())
	{
		RebuildChildren();
	}
	HandleDelayedGroupExpansion();
}

void FActionMappingsNodeBuilder::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton( FSimpleDelegate::CreateSP( this, &FActionMappingsNodeBuilder::AddActionMappingButton_OnClick), 
		LOCTEXT("AddActionMappingToolTip", "Adds Action Mapping") );

	TSharedRef<SWidget> ClearButton = PropertyCustomizationHelpers::MakeEmptyButton( FSimpleDelegate::CreateSP( this, &FActionMappingsNodeBuilder::ClearActionMappingButton_OnClick), 
		LOCTEXT("ClearActionMappingToolTip", "Removes all Action Mappings") );

	NodeRow
	.FilterString(ActionMappingsPropertyHandle->GetPropertyDisplayName())
	[
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			ActionMappingsPropertyHandle->CreatePropertyNameWidget()
		]
		+SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AddButton
		]
		+SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ClearButton
		]
	];
}

void FActionMappingsNodeBuilder::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	RebuildGroupedMappings();

	for (int32 Index = 0; Index < GroupedMappings.Num(); ++Index)
	{
		InputSettingsDetails::FMappingSet& MappingSet = GroupedMappings[Index];

		FString GroupNameString(TEXT("ActionMappings."));
		MappingSet.SharedName.AppendString(GroupNameString);
		FName GroupName(*GroupNameString);
		IDetailGroup& ActionMappingGroup = ChildrenBuilder.AddGroup(GroupName, FText::FromName(MappingSet.SharedName));
		MappingSet.DetailGroup = &ActionMappingGroup;

		TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilder::AddActionMappingToGroupButton_OnClick, MappingSet),
			LOCTEXT("AddActionMappingToGroupToolTip", "Adds Action Mapping to Group"));

		TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FActionMappingsNodeBuilder::RemoveActionMappingGroupButton_OnClick, MappingSet),
			LOCTEXT("RemoveActionMappingGroupToolTip", "Removes Action Mapping Group"));

		ActionMappingGroup.HeaderRow()
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(InputSettingsDetails::InputConstants::TextBoxWidth)
				[
					SNew(SEditableTextBox)
					.Padding(2.0f)
					.Text(FText::FromName(MappingSet.SharedName))
					.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FActionMappingsNodeBuilder::OnActionMappingNameCommitted, MappingSet))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				AddButton
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				RemoveButton
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeprecationLabel_Checkbox", "Is Deprecated"))
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FActionMappingsNodeBuilder::GetActionMappingCheckBoxState, MappingSet.GetActionMappingName())
				.OnCheckStateChanged(this, &FActionMappingsNodeBuilder::OnDeprecationCheckboxChanged, MappingSet.GetActionMappingName())
				.ToolTipText(LOCTEXT("DeprecateActionGroup", "If true, then this action mapping will be considered deprecated."))
			]
		];
	
		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			ActionMappingGroup.AddPropertyRow(MappingSet.Mappings[MappingIndex]).ShowPropertyButtons(false);
		}
	}
}

void FActionMappingsNodeBuilder::AddActionMappingButton_OnClick()
{
	static const FName BaseActionMappingName(*LOCTEXT("NewActionMappingName", "NewActionMapping").ToString());
	const FScopedTransaction Transaction(LOCTEXT("AddActionMapping_Transaction", "Add Action Mapping"));

	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputSettings* InputSettings = CastChecked<UInputSettings>(OuterObjects[0]);
		InputSettings->Modify();
		ActionMappingsPropertyHandle->NotifyPreChange();

		FName NewActionMappingName = InputSettings->GetUniqueActionName(BaseActionMappingName);

		DelayedGroupExpansionStates.Emplace(NewActionMappingName, true);
		FInputActionKeyMapping NewMapping(NewActionMappingName);
		InputSettings->AddActionMapping(NewMapping);

		ActionMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FActionMappingsNodeBuilder::ClearActionMappingButton_OnClick()
{
	ActionMappingsPropertyHandle->AsArray()->EmptyArray();
}

void FActionMappingsNodeBuilder::OnActionMappingNameCommitted(const FText& InName, ETextCommit::Type CommitInfo, const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("RenameActionMapping_Transaction", "Rename Action Mapping"));

	FName NewName = FName(*InName.ToString());
	FName CurrentName = NewName;

	if (MappingSet.Mappings.Num() > 0)
	{
		MappingSet.Mappings[0]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, ActionName))->GetValue(CurrentName);
	}

	if (NewName != CurrentName)
	{
		for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
		{
			MappingSet.Mappings[Index]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, ActionName))->SetValue(NewName);
		}

		if (MappingSet.DetailGroup)
		{
			DelayedGroupExpansionStates.Emplace(NewName, MappingSet.DetailGroup->GetExpansionState());

			// Don't want to save expansion state of old name
			MappingSet.DetailGroup->ToggleExpansion(false);
		}
	}
}

void FActionMappingsNodeBuilder::AddActionMappingToGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("AddActionMappingToGroup_Transaction", "Add Action Mapping To Group"));

	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputSettings* InputSettings = CastChecked<UInputSettings>(OuterObjects[0]);
		InputSettings->Modify();
		ActionMappingsPropertyHandle->NotifyPreChange();

		DelayedGroupExpansionStates.Emplace(MappingSet.SharedName, true);
		FInputActionKeyMapping NewMapping(MappingSet.SharedName);
		InputSettings->AddActionMapping(NewMapping);

		ActionMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FActionMappingsNodeBuilder::RemoveActionMappingGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveActionMappingGroup_Transaction", "Remove Action Mapping Group"));

	// Since this axis mapping has been removed, also remove it from the deprecation array
	UE::HandleInputDeprecationStatusChanged(GetInputSettings(), ECheckBoxState::Unchecked, MappingSet.GetActionMappingName());
	
	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();

	TArray<int32> SortedIndices;
	for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
	{
		SortedIndices.AddUnique(MappingSet.Mappings[Index]->GetIndexInArray());
	}
	SortedIndices.Sort();

	for (int32 Index = SortedIndices.Num() - 1; Index >= 0; --Index)
	{
		ActionMappingsArrayHandle->DeleteItem(SortedIndices[Index]);
	}
}

void FActionMappingsNodeBuilder::OnDeprecationCheckboxChanged(ECheckBoxState NewState, const FName MappingName)
{
	UE::HandleInputDeprecationStatusChanged(GetInputSettings(), NewState, MappingName);
}

ECheckBoxState FActionMappingsNodeBuilder::GetActionMappingCheckBoxState(const FName MappingName) const
{
	return UE::GetInputDeprecationCheckBoxState(GetInputSettings(), MappingName);
}

UInputSettings* FActionMappingsNodeBuilder::GetInputSettings() const
{
	TArray<UObject*> OuterObjects;
	ActionMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		return CastChecked<UInputSettings>(OuterObjects[0]);
	}
	return nullptr;
}

bool FActionMappingsNodeBuilder::GroupsRequireRebuild() const
{
	for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
	{
		const InputSettingsDetails::FMappingSet& MappingSet = GroupedMappings[GroupIndex];
		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			FName ActionName;
			MappingSet.Mappings[MappingIndex]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, ActionName))->GetValue(ActionName);
			if (MappingSet.SharedName != ActionName)
			{
				return true;
			}
		}
	}
	return false;
}

void FActionMappingsNodeBuilder::RebuildGroupedMappings()
{
	GroupedMappings.Empty();

	TSharedPtr<IPropertyHandleArray> ActionMappingsArrayHandle = ActionMappingsPropertyHandle->AsArray();

	uint32 NumMappings;
	ActionMappingsArrayHandle->GetNumElements(NumMappings);
	for (uint32 Index = 0; Index < NumMappings; ++Index)
	{
		TSharedRef<IPropertyHandle> ActionMapping = ActionMappingsArrayHandle->GetElement(Index);
		FName ActionName;
		FPropertyAccess::Result Result = ActionMapping->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, ActionName))->GetValue(ActionName);

		if (Result == FPropertyAccess::Success)
		{
			int32 FoundIndex = INDEX_NONE;
			for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
			{
				if (GroupedMappings[GroupIndex].SharedName == ActionName)
				{
					FoundIndex = GroupIndex;
					break;
				}
			}
			if (FoundIndex == INDEX_NONE)
			{
				FoundIndex = GroupedMappings.Num();
				GroupedMappings.AddZeroed();
				GroupedMappings[FoundIndex].SharedName = ActionName;
			}
			GroupedMappings[FoundIndex].Mappings.Add(ActionMapping);
		}
	}
}

void FActionMappingsNodeBuilder::HandleDelayedGroupExpansion()
{
	if (DelayedGroupExpansionStates.Num() > 0)
	{
		for (auto GroupState : DelayedGroupExpansionStates)
		{
			for (auto& MappingSet : GroupedMappings)
			{
				if (MappingSet.SharedName == GroupState.Key)
				{
					MappingSet.DetailGroup->ToggleExpansion(GroupState.Value);
					break;
				}
			}
		}
		DelayedGroupExpansionStates.Empty();
	}
}

//////////////////////////////
// FAxisMappingsNodeBuilder //
//////////////////////////////

FAxisMappingsNodeBuilder::FAxisMappingsNodeBuilder( IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle )
	: DetailLayoutBuilder( InDetailLayoutBuilder )
	, AxisMappingsPropertyHandle( InPropertyHandle )
{
	// Delegate for when the children in the array change
	FSimpleDelegate RebuildChildrenDelegate = FSimpleDelegate::CreateRaw( this, &FAxisMappingsNodeBuilder::RebuildChildren );
	AxisMappingsPropertyHandle->SetOnPropertyValueChanged( RebuildChildrenDelegate );
	AxisMappingsPropertyHandle->AsArray()->SetOnNumElementsChanged( RebuildChildrenDelegate );
}

void FAxisMappingsNodeBuilder::Tick( float DeltaTime )
{
	if (GroupsRequireRebuild())
	{
		RebuildChildren();
	}
	HandleDelayedGroupExpansion();
}

void FAxisMappingsNodeBuilder::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton( FSimpleDelegate::CreateSP( this, &FAxisMappingsNodeBuilder::AddAxisMappingButton_OnClick), 
		LOCTEXT("AddAxisMappingToolTip", "Adds Axis Mapping") );

	TSharedRef<SWidget> ClearButton = PropertyCustomizationHelpers::MakeEmptyButton( FSimpleDelegate::CreateSP( this, &FAxisMappingsNodeBuilder::ClearAxisMappingButton_OnClick), 
		LOCTEXT("ClearAxisMappingToolTip", "Removes all Axis Mappings") );

	NodeRow
	[
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			AxisMappingsPropertyHandle->CreatePropertyNameWidget()
		]
		+SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AddButton
		]
		+SHorizontalBox::Slot()
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ClearButton
		]
	];
}

void FAxisMappingsNodeBuilder::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	RebuildGroupedMappings();

	for (int32 Index = 0; Index < GroupedMappings.Num(); ++Index)
	{
		InputSettingsDetails::FMappingSet& MappingSet = GroupedMappings[Index];

		FString GroupNameString(TEXT("AxisMappings."));
		MappingSet.SharedName.AppendString(GroupNameString);
		FName GroupName(*GroupNameString);
		IDetailGroup& AxisMappingGroup = ChildrenBuilder.AddGroup(GroupName, FText::FromName(MappingSet.SharedName));
		MappingSet.DetailGroup = &AxisMappingGroup;

		TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &FAxisMappingsNodeBuilder::AddAxisMappingToGroupButton_OnClick, MappingSet),
			LOCTEXT("AddAxisMappingToGroupToolTip", "Adds Axis Mapping to Group"));

		TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FAxisMappingsNodeBuilder::RemoveAxisMappingGroupButton_OnClick, MappingSet),
			LOCTEXT("RemoveAxisMappingGroupToolTip", "Removes Axis Mapping Group"));

		AxisMappingGroup.HeaderRow()
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SBox )
				.WidthOverride(InputSettingsDetails::InputConstants::TextBoxWidth)
				[
					SNew(SEditableTextBox)
					.Padding(2.0f)
					.Text(FText::FromName(MappingSet.SharedName))
					.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FAxisMappingsNodeBuilder::OnAxisMappingNameCommitted, MappingSet))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				AddButton
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				RemoveButton
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeprecationLabel_Checkbox", "Is Deprecated"))
			]
			+SHorizontalBox::Slot()
			.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FAxisMappingsNodeBuilder::GetAxisMappingCheckBoxState, MappingSet.GetAxisMappingName())
				.OnCheckStateChanged(this, &FAxisMappingsNodeBuilder::OnDeprecationCheckboxChanged, MappingSet.GetAxisMappingName())
				.ToolTipText(LOCTEXT("DeprecateAxisGroup", "If true, then this axis mapping will be considered deprecated."))
			]
		];

		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			AxisMappingGroup.AddPropertyRow(MappingSet.Mappings[MappingIndex]).ShowPropertyButtons(false);
		}
	}
}

void FAxisMappingsNodeBuilder::AddAxisMappingButton_OnClick()
{
	static const FName BaseAxisMappingName(*LOCTEXT("NewAxisMappingName", "NewAxisMapping").ToString());
	static int32 NewMappingCount = 0;
	const FScopedTransaction Transaction(LOCTEXT("AddAxisMapping_Transaction", "Add Axis Mapping"));

	TArray<UObject*> OuterObjects;
	AxisMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputSettings* InputSettings = CastChecked<UInputSettings>(OuterObjects[0]);
		InputSettings->Modify();
		AxisMappingsPropertyHandle->NotifyPreChange();

		FName NewAxisMappingName = InputSettings->GetUniqueAxisName(BaseAxisMappingName);

		DelayedGroupExpansionStates.Emplace(NewAxisMappingName, true);
		FInputAxisKeyMapping NewMapping(NewAxisMappingName);
		InputSettings->AddAxisMapping(NewMapping);

		AxisMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FAxisMappingsNodeBuilder::ClearAxisMappingButton_OnClick()
{
	AxisMappingsPropertyHandle->AsArray()->EmptyArray();
}

void FAxisMappingsNodeBuilder::OnAxisMappingNameCommitted(const FText& InName, ETextCommit::Type CommitInfo, const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("RenameAxisMapping_Transaction", "Rename Axis Mapping"));

	FName NewName = FName(*InName.ToString());
	FName CurrentName = NewName;

	if (MappingSet.Mappings.Num() > 0)
	{
		MappingSet.Mappings[0]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, AxisName))->GetValue(CurrentName);
	}

	if (NewName != CurrentName)
	{
		for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
		{
			MappingSet.Mappings[Index]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, AxisName))->SetValue(NewName);
		}

		if (MappingSet.DetailGroup)
		{
			DelayedGroupExpansionStates.Emplace(NewName, MappingSet.DetailGroup->GetExpansionState());

			// Don't want to save expansion state of old name
			MappingSet.DetailGroup->ToggleExpansion(false);
		}
	}
}

void FAxisMappingsNodeBuilder::AddAxisMappingToGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("AddAxisMappingToGroup_Transaction", "Add Axis Mapping To Group"));

	TArray<UObject*> OuterObjects;
	AxisMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UInputSettings* InputSettings = CastChecked<UInputSettings>(OuterObjects[0]);
		InputSettings->Modify();
		AxisMappingsPropertyHandle->NotifyPreChange();

		DelayedGroupExpansionStates.Emplace(MappingSet.SharedName, true);
		FInputAxisKeyMapping NewMapping(MappingSet.SharedName);
		InputSettings->AddAxisMapping(NewMapping);

		AxisMappingsPropertyHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
	}
}

void FAxisMappingsNodeBuilder::RemoveAxisMappingGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet)
{
	const FScopedTransaction Transaction(LOCTEXT("RemoveAxisMappingGroup_Transaction", "Remove Axis Mapping Group"));

	// Since this axis mapping has been removed, also remove it from the deprecation array
	UE::HandleInputDeprecationStatusChanged(GetInputSettings(), ECheckBoxState::Unchecked, MappingSet.GetAxisMappingName());
	
	TSharedPtr<IPropertyHandleArray> AxisMappingsArrayHandle = AxisMappingsPropertyHandle->AsArray();

	TArray<int32> SortedIndices;
	for (int32 Index = 0; Index < MappingSet.Mappings.Num(); ++Index)
	{
		SortedIndices.AddUnique(MappingSet.Mappings[Index]->GetIndexInArray());
	}
	SortedIndices.Sort();

	for (int32 Index = SortedIndices.Num() - 1; Index >= 0; --Index)
	{
		AxisMappingsArrayHandle->DeleteItem(SortedIndices[Index]);
	}
}

void FAxisMappingsNodeBuilder::OnDeprecationCheckboxChanged(ECheckBoxState NewState, const FName MappingName)
{
	UE::HandleInputDeprecationStatusChanged(GetInputSettings(), NewState, MappingName);
}

ECheckBoxState FAxisMappingsNodeBuilder::GetAxisMappingCheckBoxState(const FName MappingName) const
{
	return UE::GetInputDeprecationCheckBoxState(GetInputSettings(), MappingName);
}

UInputSettings* FAxisMappingsNodeBuilder::GetInputSettings() const
{
	TArray<UObject*> OuterObjects;
	AxisMappingsPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		return CastChecked<UInputSettings>(OuterObjects[0]);
	}
	return nullptr;
}

bool FAxisMappingsNodeBuilder::GroupsRequireRebuild() const
{
	for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
	{
		const InputSettingsDetails::FMappingSet& MappingSet = GroupedMappings[GroupIndex];
		for (int32 MappingIndex = 0; MappingIndex < MappingSet.Mappings.Num(); ++MappingIndex)
		{
			FName AxisName;
			MappingSet.Mappings[MappingIndex]->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, AxisName))->GetValue(AxisName);
			if (MappingSet.SharedName != AxisName)
			{
				return true;
			}
		}
	}
	return false;
}

void FAxisMappingsNodeBuilder::RebuildGroupedMappings()
{
	GroupedMappings.Empty();

	TSharedPtr<IPropertyHandleArray> AxisMappingsArrayHandle = AxisMappingsPropertyHandle->AsArray();

	uint32 NumMappings;
	AxisMappingsArrayHandle->GetNumElements(NumMappings);
	for (uint32 Index = 0; Index < NumMappings; ++Index)
	{
		TSharedRef<IPropertyHandle> AxisMapping = AxisMappingsArrayHandle->GetElement(Index);
		FName AxisName;
		FPropertyAccess::Result Result = AxisMapping->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, AxisName))->GetValue(AxisName);

		if (Result == FPropertyAccess::Success)
		{
			int32 FoundIndex = INDEX_NONE;
			for (int32 GroupIndex = 0; GroupIndex < GroupedMappings.Num(); ++GroupIndex)
			{
				if (GroupedMappings[GroupIndex].SharedName == AxisName)
				{
					FoundIndex = GroupIndex;
					break;
				}
			}
			if (FoundIndex == INDEX_NONE)
			{
				FoundIndex = GroupedMappings.Num();
				GroupedMappings.AddZeroed();
				GroupedMappings[FoundIndex].SharedName = AxisName;
			}
			GroupedMappings[FoundIndex].Mappings.Add(AxisMapping);
		}
	}
}

void FAxisMappingsNodeBuilder::HandleDelayedGroupExpansion()
{
	if (DelayedGroupExpansionStates.Num() > 0)
	{
		for (auto GroupState : DelayedGroupExpansionStates)
		{
			for (auto& MappingSet : GroupedMappings)
			{
				if (MappingSet.SharedName == GroupState.Key)
				{
					MappingSet.DetailGroup->ToggleExpansion(GroupState.Value);
					break;
				}
			}
		}
		DelayedGroupExpansionStates.Empty();
	}
}

/////////////////////////
// FInputSettingsDetails //
/////////////////////////

TSharedRef<IDetailCustomization> FInputSettingsDetails::MakeInstance()
{
	return MakeShareable(new FInputSettingsDetails);
}

void FInputSettingsDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	static const FName BindingsCategory = TEXT("Bindings");
	static const FName ActionMappings = UInputSettings::GetActionMappingsPropertyName();
	static const FName AxisMappings = UInputSettings::GetAxisMappingsPropertyName();

	IDetailCategoryBuilder& MappingsDetailCategoryBuilder = DetailBuilder.EditCategory(BindingsCategory);

	// If the new Enhanced Input module is loaded, then add a warning telling people to use
	// that instead of these legacy Action/Axis bindings
	static const FName ForegroundColorStyle("Colors.Foreground");
	static const FName WarningColorStyle("Colors.AccentYellow");
	static const FSlateBrush* WarningBrush = FAppStyle::Get().GetBrush("Icons.AlertCircle");
	
	MappingsDetailCategoryBuilder.AddCustomRow(LOCTEXT("Mappings_Title", "Action Axis Mappings"))
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0.0f, 10.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(10.0f, 0.0f)
				.VAlign(VAlign_Bottom)
				[
					SNew(SImage)
					.Image(WarningBrush)
					.Visibility(this, &FInputSettingsDetails::GetLegacyWarningVisibility)
					.ColorAndOpacity(FAppStyle::Get().GetSlateColor(ForegroundColorStyle))		
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Bottom)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.AutoWrapText(true)					
					.Visibility(this, &FInputSettingsDetails::GetLegacyWarningVisibility)
					.Text(LOCTEXT("Mappings_DeprecationWarning", "Axis and Action mappings are now deprecated, please use Enhanced Input Actions and Input Mapping Contexts instead."))
					.ColorAndOpacity(FAppStyle::Get().GetSlateColor(WarningColorStyle))
				]	
			]
			+ SVerticalBox::Slot()
			.AutoHeight()			
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AutoWrapText(true)
				.Text(LOCTEXT("Mappings_Description", "Action and Axis Mappings provide a mechanism to conveniently map keys and axes to input behaviors by inserting a layer of indirection between the input behavior and the keys that invoke it. Action Mappings are for key presses and releases, while Axis Mappings allow for inputs that have a continuous range."))
			]		
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			IDocumentation::Get()->CreateAnchor(FString("Gameplay/Input"))
		]
	];

	// Custom Action Mappings
	const TSharedPtr<IPropertyHandle> ActionMappingsPropertyHandle = DetailBuilder.GetProperty(ActionMappings, UInputSettings::StaticClass());
	ActionMappingsPropertyHandle->MarkHiddenByCustomization();

	const TSharedRef<FActionMappingsNodeBuilder> ActionMappingsBuilder = MakeShareable( new FActionMappingsNodeBuilder( &DetailBuilder, ActionMappingsPropertyHandle ) );
	MappingsDetailCategoryBuilder.AddCustomBuilder(ActionMappingsBuilder);

	// Custom Axis Mappings
	const TSharedPtr<IPropertyHandle> AxisMappingsPropertyHandle = DetailBuilder.GetProperty(AxisMappings, UInputSettings::StaticClass());
	AxisMappingsPropertyHandle->MarkHiddenByCustomization();

	const TSharedRef<FAxisMappingsNodeBuilder> AxisMappingsBuilder = MakeShareable( new FAxisMappingsNodeBuilder( &DetailBuilder, AxisMappingsPropertyHandle ) );
	MappingsDetailCategoryBuilder.AddCustomBuilder(AxisMappingsBuilder);
}

EVisibility FInputSettingsDetails::GetLegacyWarningVisibility() const
{
	return FModuleManager::Get().IsModuleLoaded("EnhancedInput") ? EVisibility::Visible : EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
