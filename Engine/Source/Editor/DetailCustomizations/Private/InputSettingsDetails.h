// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateTypes.h"

class FDetailWidgetRow;
class FText;
class IDetailChildrenBuilder;
class IDetailGroup;
class IDetailLayoutBuilder;
class IPropertyHandle;

namespace InputSettingsDetails
{
	namespace InputConstants
	{
		const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
		const float TextBoxWidth = 250.0f;
		const float ScaleBoxWidth = 50.0f;
	}

	struct FMappingSet
	{
		FName SharedName;
		IDetailGroup* DetailGroup;
		TArray<TSharedRef<IPropertyHandle>> Mappings;
		FName GetActionMappingName() const;
		FName GetAxisMappingName() const;
	};
}

class FActionMappingsNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FActionMappingsNodeBuilder>
{
public:
	FActionMappingsNodeBuilder( IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle );

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRebuildChildren  ) override { OnRebuildChildren = InOnRebuildChildren; } 
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick( float DeltaTime ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual bool InitiallyCollapsed() const override { return true; };
	virtual FName GetName() const override { return FName(TEXT("ActionMappings")); }

private:
	void AddActionMappingButton_OnClick();
	void ClearActionMappingButton_OnClick();
	void OnActionMappingNameCommitted(const FText& InName, ETextCommit::Type CommitInfo, const InputSettingsDetails::FMappingSet MappingSet);
	void AddActionMappingToGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet);
	void RemoveActionMappingGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet);

	void OnDeprecationCheckboxChanged(ECheckBoxState NewState, const FName MappingName);
	ECheckBoxState GetActionMappingCheckBoxState(const FName MappingName) const;
	class UInputSettings* GetInputSettings() const;
	

	bool GroupsRequireRebuild() const;
	void RebuildGroupedMappings();
	void RebuildChildren()
	{
		OnRebuildChildren.ExecuteIfBound();
	}
	/** Makes sure that groups have their expansion set after any rebuilding */
	void HandleDelayedGroupExpansion();

private:
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Associated detail layout builder */
	IDetailLayoutBuilder* DetailLayoutBuilder;

	/** Property handle to associated action mappings */
	TSharedPtr<IPropertyHandle> ActionMappingsPropertyHandle;

	TArray<InputSettingsDetails::FMappingSet> GroupedMappings;

	TArray<TPair<FName, bool>> DelayedGroupExpansionStates;
};

class FAxisMappingsNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FAxisMappingsNodeBuilder>
{
public:
	FAxisMappingsNodeBuilder( IDetailLayoutBuilder* InDetailLayoutBuilder, const TSharedPtr<IPropertyHandle>& InPropertyHandle );

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRebuildChildren  ) override { OnRebuildChildren = InOnRebuildChildren; } 
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick( float DeltaTime ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual bool InitiallyCollapsed() const override { return true; };
	virtual FName GetName() const override { return FName(TEXT("AxisMappings")); }

private:
	void AddAxisMappingButton_OnClick();
	void ClearAxisMappingButton_OnClick();
	void OnAxisMappingNameCommitted(const FText& InName, ETextCommit::Type CommitInfo, const InputSettingsDetails::FMappingSet MappingSet);
	void AddAxisMappingToGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet);
	void RemoveAxisMappingGroupButton_OnClick(const InputSettingsDetails::FMappingSet MappingSet);

	void OnDeprecationCheckboxChanged(ECheckBoxState NewState, const FName MappingName);
	ECheckBoxState GetAxisMappingCheckBoxState(const FName MappingName) const;
	class UInputSettings* GetInputSettings() const;

	bool GroupsRequireRebuild() const;
	void RebuildGroupedMappings();
	void RebuildChildren()
	{
		OnRebuildChildren.ExecuteIfBound();
	}
	/** Makes sure that groups have their expansion set after any rebuilding */
	void HandleDelayedGroupExpansion();

private:
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Associated detail layout builder */
	IDetailLayoutBuilder* DetailLayoutBuilder;

	/** Property handle to associated axis mappings */
	TSharedPtr<IPropertyHandle> AxisMappingsPropertyHandle;

	TArray<InputSettingsDetails::FMappingSet> GroupedMappings;

	TArray<TPair<FName, bool>> DelayedGroupExpansionStates;
};

class FInputSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( class IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/**
	 * If true, then we should display some warning text about the Axis/Action mappings being legacy
	 * in favor of Enhanced Input
	 */
	EVisibility GetLegacyWarningVisibility() const; 
};
