// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputStructCustomization.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "GameFramework/PlayerInput.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "InputSettingsDetails.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "InputStructCustomization"

///////////////////////////////////
// FInputAxisConfigCustomization //
///////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FInputAxisConfigCustomization::MakeInstance() 
{
	return MakeShareable( new FInputAxisConfigCustomization );
}

void FInputAxisConfigCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	FString AxisKeyName;
	InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisConfigEntry, AxisKeyName))->GetValue(AxisKeyName);

	HeaderRow.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget(FText::FromString(AxisKeyName))
	];
}

void FInputAxisConfigCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> AxisProperties = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisConfigEntry, AxisProperties));

	uint32 NumChildren;
	AxisProperties->GetNumChildren( NumChildren );

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		StructBuilder.AddProperty( AxisProperties->GetChildHandle(ChildIndex).ToSharedRef() );
	}
}

//////////////////////////////////////
// FInputActionMappingCustomization //
//////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FInputActionMappingCustomization::MakeInstance() 
{
	return MakeShareable( new FInputActionMappingCustomization );
}

void FInputActionMappingCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	ActionMappingHandle = InStructPropertyHandle;
}

void FInputActionMappingCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> KeyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, Key));
	TSharedPtr<IPropertyHandle> ShiftHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, bShift));
	TSharedPtr<IPropertyHandle> CtrlHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, bCtrl));
	TSharedPtr<IPropertyHandle> AltHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, bAlt));
	TSharedPtr<IPropertyHandle> CmdHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputActionKeyMapping, bCmd));

	const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = InStructPropertyHandle->GetParentHandle()->AsArray();

	TSharedRef<SWidget> RemoveButton = (ParentArrayHandle.IsValid() ? PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FInputActionMappingCustomization::RemoveActionMappingButton_OnClick),
		LOCTEXT("RemoveActionMappingToolTip", "Removes Action Mapping")) : SNullWidget::NullWidget);

	StructBuilder.AddCustomRow( LOCTEXT("KeySearchStr", "Key") )
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.AutoWidth()
		[
			SNew( SBox )
			.WidthOverride(InputSettingsDetails::InputConstants::TextBoxWidth)
			[
				StructBuilder.GenerateStructValueWidget( KeyHandle.ToSharedRef() )
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ShiftHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ShiftHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CtrlHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CtrlHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AltHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AltHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CmdHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CmdHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			RemoveButton
		]
	];
}

void FInputActionMappingCustomization::RemoveActionMappingButton_OnClick()
{
	if( ActionMappingHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = ActionMappingHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		ParentArrayHandle->DeleteItem( ActionMappingHandle->GetIndexInArray() );
	}
}

//////////////////////////////////////
// FInputAxisMappingCustomization //
//////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FInputAxisMappingCustomization::MakeInstance() 
{
	return MakeShareable( new FInputAxisMappingCustomization );
}

void FInputAxisMappingCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	AxisMappingHandle = InStructPropertyHandle;
}

void FInputAxisMappingCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> KeyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, Key));
	TSharedPtr<IPropertyHandle> ScaleHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputAxisKeyMapping, Scale));

	const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = InStructPropertyHandle->GetParentHandle()->AsArray();

	TSharedRef<SWidget> RemoveButton = (ParentArrayHandle.IsValid() ? PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FInputAxisMappingCustomization::RemoveAxisMappingButton_OnClick),
		LOCTEXT("RemoveAxisMappingToolTip", "Removes Axis Mapping")) : SNullWidget::NullWidget);

	StructBuilder.AddCustomRow( LOCTEXT("KeySearchStr", "Key") )
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.AutoWidth()
		[
			SNew( SBox )
			.WidthOverride(InputSettingsDetails::InputConstants::TextBoxWidth)
			[
				StructBuilder.GenerateStructValueWidget( KeyHandle.ToSharedRef() )
			]
		]
		+SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ScaleHandle->CreatePropertyNameWidget()
		]
		+SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(InputSettingsDetails::InputConstants::ScaleBoxWidth)
			[
				ScaleHandle->CreatePropertyValueWidget()
			]
		]
		+SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			RemoveButton
		]
	];
}

void FInputAxisMappingCustomization::RemoveAxisMappingButton_OnClick()
{
	if( AxisMappingHandle->IsValidHandle() )
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = AxisMappingHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		ParentArrayHandle->DeleteItem( AxisMappingHandle->GetIndexInArray() );
	}
}

#undef LOCTEXT_NAMESPACE
