// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStaticMeshComponentDetails.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class UObject;

#define LOCTEXT_NAMESPACE "InstancedStaticMeshComponentDetails"

TAutoConsoleVariable<int32> CVarMaxNumInstancesDetails(
	TEXT("r.Editor.MaxNumInstancesDetails"),
	512,
	TEXT("Maximum number of instances shown in the details panel. Above this value, instances are hidden by default. \n")
	TEXT("< 0 : No maximum\n"),
	ECVF_RenderThreadSafe);

TSharedRef<IDetailCustomization> FInstancedStaticMeshComponentDetails::MakeInstance()
{
	return MakeShareable(new FInstancedStaticMeshComponentDetails);
}

void FInstancedStaticMeshComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TSharedPtr<IPropertyHandle> PerInstanceSMDataProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMData));
	TSharedPtr<IPropertyHandle> PerInstanceSMCustomDataProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UInstancedStaticMeshComponent, PerInstanceSMCustomData));


	// Doesn't support multi-selection : 
	if (ObjectsBeingCustomized.Num() != 1)
	{
		PerInstanceSMDataProperty->MarkHiddenByCustomization();
		PerInstanceSMCustomDataProperty->MarkHiddenByCustomization();
	}
	else
	{
		ComponentBeingCustomized = Cast<UInstancedStaticMeshComponent>(ObjectsBeingCustomized[0]);
		if (ensure(ComponentBeingCustomized.IsValid()))
		{
			const FName InstancesCategoryName("Instances");
			IDetailCategoryBuilder& InstancesCategory = DetailBuilder.EditCategory(InstancesCategoryName);

			NumInstances = ComponentBeingCustomized->PerInstanceSMData.Num();
			bForceShowAllInstances = ComponentBeingCustomized->bForceShowAllInstancesDetails;

			int32 MaxNumInstances = CVarMaxNumInstancesDetails.GetValueOnGameThread();
			bool bTooManyInstances = (MaxNumInstances >= 0) && (NumInstances > MaxNumInstances);
			// If there are too many instances to display, hide the property as it will slow down the UI : 
			if (bTooManyInstances && !bForceShowAllInstances)
			{
				PerInstanceSMDataProperty->MarkHiddenByCustomization();
				PerInstanceSMCustomDataProperty->MarkHiddenByCustomization();
			}

			// If there are too many instances to display, display an additional button to show/hide instances : 
			if (bTooManyInstances)
			{
				const FText ShowHideAllInstancesText = LOCTEXT("ShowHideAllInstancesText", "Show/Hide All Instances");
				const FText SlatePerformanceWarningText = LOCTEXT("SlatePerformanceWarningText", "Slate Performance Warning");
				const FText TooltipText = LOCTEXT("ShowHideAllInstancesTooltip", "Displays per-instance data in the details. Can be very slow when there are lots of instances.");
				FDetailWidgetRow& CustomRow = InstancesCategory.AddCustomRow(ShowHideAllInstancesText)
					.NameContent()
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.AutoWidth()
							.Padding(0.0f, 0.0f, 4.0f, 0.0f)
							[
								SNew(SImage)
									.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
									.ToolTipText(TooltipText)
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.AutoWidth()
							[
								SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFontItalic())
									.Text(SlatePerformanceWarningText)
									.ToolTipText(TooltipText)
							]
					]
					.ValueContent()
					.MaxDesiredWidth(120.f)
					[
						SNew(SButton)
							.OnClicked(this, &FInstancedStaticMeshComponentDetails::OnShowHideAllInstancesClicked)
							.ToolTipText(TooltipText)
							.IsEnabled(this, &FInstancedStaticMeshComponentDetails::IsShowHideAllInstancesEnabled)
							[
								SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFontItalic())
									.Text(this, &FInstancedStaticMeshComponentDetails::GetShowHideAllInstancesText)
							]
					];
			}
		}
	}
}

void FInstancedStaticMeshComponentDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

FReply FInstancedStaticMeshComponentDetails::OnShowHideAllInstancesClicked()
{
	bForceShowAllInstances = !bForceShowAllInstances;

	// pass the new value the component, as the following refresh will instanciate a new FInstancedStaticMeshComponentDetails : 
	if (ComponentBeingCustomized.IsValid())
	{
		ComponentBeingCustomized->bForceShowAllInstancesDetails = bForceShowAllInstances;
	}

	// Here we can only take the ptr as ForceRefreshDetails() checks that the reference is unique.
	if (IDetailLayoutBuilder* DetailBuilder = CachedDetailBuilder.Pin().Get())
	{
		DetailBuilder->ForceRefreshDetails();
	}
	return FReply::Handled();
}

bool FInstancedStaticMeshComponentDetails::IsShowHideAllInstancesEnabled() const
{
	return (NumInstances > 0);
}

FText FInstancedStaticMeshComponentDetails::GetShowHideAllInstancesText() const
{
	return bForceShowAllInstances ?
		FText::Format(LOCTEXT("HideAllInstances", "Hide All {0} Instances"), NumInstances)
		: FText::Format(LOCTEXT("ShowAllInstances", "Show All {0} Instances"), NumInstances);
}

#undef LOCTEXT_NAMESPACE