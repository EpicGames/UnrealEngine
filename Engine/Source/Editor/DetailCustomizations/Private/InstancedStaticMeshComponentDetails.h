// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;
class IDetailLayoutBuilder;
class UInstancedStaticMeshComponent;

class FInstancedStaticMeshComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

private:
	FReply OnShowHideAllInstancesClicked();
	bool IsShowHideAllInstancesEnabled() const;
	FText GetShowHideAllInstancesText() const;

private:
	/** Indicate whether we're currently forcing the display of the instance list or not */
	bool bForceShowAllInstances = false;
	/** Component being edited */
	TWeakObjectPtr<UInstancedStaticMeshComponent> ComponentBeingCustomized;
	/** Current number of instances to show in the display panel */
	int32 NumInstances = 0;
	/** The detail builder for this customization */
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
};

