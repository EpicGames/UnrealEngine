// Copyright Epic Games, Inc. All Rights Reserved.

#include "IntervalStructCustomization.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include <limits>

#define LOCTEXT_NAMESPACE "IntervalStructCustomization"


/* Helper traits for getting a metadata property based on the template parameter type
 *****************************************************************************/

namespace IntervalMetadata
{
	template <typename NumericType>
	struct FGetHelper
	{ };

	template <>
	struct FGetHelper<float>
	{
		static float GetMetaData(const FProperty* Property, const TCHAR* Key)
		{
			return Property->GetFloatMetaData(Key);
		}
	};

	template <>
	struct FGetHelper<int32>
	{
		static int32 GetMetaData(const FProperty* Property, const TCHAR* Key)
		{
			return Property->GetIntMetaData(Key);
		}
	};
}


/* FIntervalStructCustomization static interface
 *****************************************************************************/

template <typename NumericType>
TSharedRef<IPropertyTypeCustomization> FIntervalStructCustomization<NumericType>::MakeInstance()
{
	return MakeShareable(new FIntervalStructCustomization<NumericType>);
}


/* IPropertyTypeCustomization interface
 *****************************************************************************/

template <typename NumericType>
void FIntervalStructCustomization<NumericType>::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Get handles to the properties we're interested in
	MinValueHandle = StructPropertyHandle->GetChildHandle(TEXT("Min"));
	MaxValueHandle = StructPropertyHandle->GetChildHandle(TEXT("Max"));
	check(MinValueHandle.IsValid());
	check(MaxValueHandle.IsValid());

	// Get min/max metadata values if defined
	auto Property = StructPropertyHandle->GetProperty();
	check(Property != nullptr);

	const FString& MetaUIMinString = Property->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = Property->GetMetaData(TEXT("UIMax"));
	const FString& MetaClampMinString = Property->GetMetaData(TEXT("ClampMin"));
	const FString& MetaClampMaxString = Property->GetMetaData(TEXT("ClampMax"));
	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : MetaClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : MetaClampMaxString;

	NumericType ClampMin = std::numeric_limits<NumericType>::lowest();
	NumericType ClampMax = std::numeric_limits<NumericType>::max();
	TTypeFromString<NumericType>::FromString(ClampMin, *MetaClampMinString);
	TTypeFromString<NumericType>::FromString(ClampMax, *MetaClampMaxString);

	NumericType UIMin = std::numeric_limits<NumericType>::lowest();
	NumericType UIMax = std::numeric_limits<NumericType>::max();
	TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
	TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);

	const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
	const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

	MinAllowedValue = MetaClampMinString.Len() ? ClampMin : TOptional<NumericType>();
	MaxAllowedValue = MetaClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
	MinAllowedSliderValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
	MaxAllowedSliderValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

	bAllowInvertedInterval = Property->HasMetaData(TEXT("AllowInvertedInterval"));
	bClampToMinMaxLimits = Property->HasMetaData(TEXT("ClampToMinMaxLimits"));

	// Build the widgets
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(251.0f)
	.MaxDesiredWidth(251.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("MinLabel", "Min"))
		]
		+SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<NumericType>)
			.Value(this, &FIntervalStructCustomization<NumericType>::OnGetValue, EIntervalField::Min)
			.MinValue(MinAllowedValue)
			.MinSliderValue(MinAllowedSliderValue)
			.MaxValue(this, &FIntervalStructCustomization<NumericType>::OnGetMaxValue)
			.MaxSliderValue(this, &FIntervalStructCustomization<NumericType>::OnGetMaxSliderValue)
			.OnValueCommitted(this, &FIntervalStructCustomization<NumericType>::OnValueCommitted, EIntervalField::Min)
			.OnValueChanged(this, &FIntervalStructCustomization<NumericType>::OnValueChanged, EIntervalField::Min)
			.OnBeginSliderMovement(this, &FIntervalStructCustomization<NumericType>::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FIntervalStructCustomization<NumericType>::OnEndSliderMovement)
			.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.AllowSpin(true)
			.IsEnabled(this, &FIntervalStructCustomization::IsPropertyEnabled, EIntervalField::Min)
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 0.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("MaxLabel", "Max"))
		]
		+SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SNumericEntryBox<NumericType>)
			.Value(this, &FIntervalStructCustomization<NumericType>::OnGetValue, EIntervalField::Max)
			.MinValue(this, &FIntervalStructCustomization<NumericType>::OnGetMinValue)
			.MinSliderValue(this, &FIntervalStructCustomization<NumericType>::OnGetMinSliderValue)
			.MaxValue(MaxAllowedValue)
			.MaxSliderValue(MaxAllowedSliderValue)
			.OnValueCommitted(this, &FIntervalStructCustomization<NumericType>::OnValueCommitted, EIntervalField::Max)
			.OnValueChanged(this, &FIntervalStructCustomization<NumericType>::OnValueChanged, EIntervalField::Max)
			.OnBeginSliderMovement(this, &FIntervalStructCustomization<NumericType>::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FIntervalStructCustomization<NumericType>::OnEndSliderMovement)
			.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.AllowSpin(true)
			.IsEnabled(this, &FIntervalStructCustomization::IsPropertyEnabled, EIntervalField::Max)
			
		]
	];
}

template <typename NumericType>
void FIntervalStructCustomization<NumericType>::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Don't display children, as editing them directly can break the constraints
}


/* FIntervalStructCustomization callbacks
 *****************************************************************************/

template <typename NumericType>
static TOptional<NumericType> GetValue(IPropertyHandle* Handle)
{
	NumericType Value;
	if (Handle->GetValue(Value) == FPropertyAccess::Success)
	{
		return TOptional<NumericType>(Value);
	}

	return TOptional<NumericType>();
}

template <typename NumericType>
TOptional<NumericType> FIntervalStructCustomization<NumericType>::OnGetValue(EIntervalField Field) const
{
	return GetValue<NumericType>((Field == EIntervalField::Min) ? MinValueHandle.Get() : MaxValueHandle.Get());
}

template <typename NumericType>
TOptional<NumericType> FIntervalStructCustomization<NumericType>::OnGetMinValue() const
{
	if (bClampToMinMaxLimits)
	{
		return GetValue<NumericType>(MinValueHandle.Get());
	}

	return MinAllowedValue;
}

template <typename NumericType>
TOptional<NumericType> FIntervalStructCustomization<NumericType>::OnGetMinSliderValue() const
{
	if (bClampToMinMaxLimits)
	{
		return GetValue<NumericType>(MinValueHandle.Get());
	}

	return MinAllowedSliderValue;
}

template <typename NumericType>
TOptional<NumericType> FIntervalStructCustomization<NumericType>::OnGetMaxValue() const
{
	if (bClampToMinMaxLimits)
	{
		return GetValue<NumericType>(MaxValueHandle.Get());
	}

	return MaxAllowedValue;
}

template <typename NumericType>
TOptional<NumericType> FIntervalStructCustomization<NumericType>::OnGetMaxSliderValue() const
{
	if (bClampToMinMaxLimits)
	{
		return GetValue<NumericType>(MaxValueHandle.Get());
	}

	return MaxAllowedSliderValue;
}

template <typename NumericType>
void FIntervalStructCustomization<NumericType>::SetValue(NumericType NewValue, EIntervalField Field, EPropertyValueSetFlags::Type Flags)
{
	IPropertyHandle* Handle = (Field == EIntervalField::Min) ? MinValueHandle.Get() : MaxValueHandle.Get();
	IPropertyHandle* OtherHandle = (Field == EIntervalField::Min) ? MaxValueHandle.Get() : MinValueHandle.Get();

	// This is to guard out 2 cases
	// a). SSpinBox flushes out any attempted uncommited value during destruction, and Custom Widgets was destructed before IPropertyTypeCustomization(see FDetailPropertyRow)
	// b). It's possible that the event DetailsView resets its Object gets routed before the event SSpinBox loses user focus, which will commit value.   
	if (Handle->GetNumOuterObjects() == 0)
	{
		TArray<TSharedPtr<FStructOnScope>> OuterStructs;
		Handle->GetOuterStructs(OuterStructs);

		if (OuterStructs.Num() == 0)
		{
			return;
		}
	}

	const TOptional<NumericType> OtherValue = GetValue<NumericType>(OtherHandle);
	const bool bOutOfRange = OtherValue.IsSet() && ((Field == EIntervalField::Min && NewValue > OtherValue.GetValue()) || (Field == EIntervalField::Max && NewValue < OtherValue.GetValue()));

	// The order of execution of the Intertactive change vs commit is super important for the Undo history
	// So Interactive we must do, Handle, Other Handle
	// For Commit: we must do the reverse to properly close the scope of traction
	if (!bOutOfRange || bAllowInvertedInterval)
	{
		if (Flags == EPropertyValueSetFlags::InteractiveChange)
		{
			ensure(Handle->SetValue(NewValue, Flags) == FPropertyAccess::Success);

			if (OtherValue.IsSet())
			{
				ensure(OtherHandle->SetValue(OtherValue.GetValue(), Flags) == FPropertyAccess::Success);
			}
		}
		else
		{
			if (OtherValue.IsSet())
			{
				ensure(OtherHandle->SetValue(OtherValue.GetValue(), Flags) == FPropertyAccess::Success);
			}

			ensure(Handle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
		}
	}
	else if (!bClampToMinMaxLimits)
	{
		if (Flags == EPropertyValueSetFlags::InteractiveChange)
		{
			ensure(Handle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
			ensure(OtherHandle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
		}
		else
		{
			ensure(OtherHandle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
			ensure(Handle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
		}
	}
}

template <typename NumericType>
void FIntervalStructCustomization<NumericType>::OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, EIntervalField Field)
{
	if (!bIsUsingSlider || (bIsUsingSlider && ShouldAllowSpin()))
	{
		SetValue(NewValue, Field);
	}
}

template <typename NumericType>
void FIntervalStructCustomization<NumericType>::OnValueChanged(NumericType NewValue, EIntervalField Field)
{
	if (bIsUsingSlider && ShouldAllowSpin())
	{
		SetValue(NewValue, Field, EPropertyValueSetFlags::InteractiveChange);
	}
}


template <typename NumericType>
void FIntervalStructCustomization<NumericType>::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	if (ShouldAllowSpin())
	{
		GEditor->BeginTransaction(LOCTEXT("SetIntervalProperty", "Set Interval Property"));
	}
}


template <typename NumericType>
void FIntervalStructCustomization<NumericType>::OnEndSliderMovement(NumericType /*NewValue*/)
{
	bIsUsingSlider = false;

	if (ShouldAllowSpin())
	{
		GEditor->EndTransaction();
	}
}

template <typename NumericType>
bool FIntervalStructCustomization<NumericType>::ShouldAllowSpin() const
{
	return true;
}

template <typename NumericType>
bool FIntervalStructCustomization<NumericType>::IsPropertyEnabled(EIntervalField Field) const
{
	IPropertyHandle* Handle = (Field == EIntervalField::Min) ? MinValueHandle.Get() : MaxValueHandle.Get();
	if (Handle == nullptr)
	{
		return false;
	}
	return !Handle->IsEditConst();
}

/* Only explicitly instantiate the types which are supported
 *****************************************************************************/

template class FIntervalStructCustomization<float>;
template class FIntervalStructCustomization<double>;
template class FIntervalStructCustomization<int32>;


#undef LOCTEXT_NAMESPACE
