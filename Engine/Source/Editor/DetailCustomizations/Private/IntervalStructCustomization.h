// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;

enum class EIntervalField
{
	Min,
	Max
};

/**
 * Implements a details panel customization for numeric TInterval structures.
 */
template <typename NumericType>
class FIntervalStructCustomization : public IPropertyTypeCustomization
{
public:

	FIntervalStructCustomization()
		: bIsUsingSlider(false) {}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

public:

	/**
	* Creates a new instance.
	*
	* @return A new struct customization for Guids.
	*/
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:

	/**
	 * Gets the value for the provided interval field
	 *
	 * @param	Field	Which FInterval field to get
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetValue(EIntervalField Field) const;

	/**
	 * Gets the minimum allowed value
	 *
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetMinValue() const;

	/**
	 * Gets the minimum slider allowed value
	 *
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetMinSliderValue() const;

	/**
	 * Gets the maximum allowed value
	 *
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetMaxValue() const;

	/**
	 * Gets the maximum slider allowed value
	 *
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetMaxSliderValue() const;

	/**
	 * Called when the value is committed from the property editor
	 *
	 * @param	NewValue		The new value of the property
	 * @param	CommitType		How the value was committed (unused)
	 * @param	Field			Which TInterval field is being committed to
	 */
	void OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, EIntervalField Field);

	/**
	 * Called when the value is changed in the property editor
	 *
	 * @param	NewValue		The new value of the property
	 * @param	Field			Which TInterval field is being committed to
	 */
	void OnValueChanged(NumericType NewValue, EIntervalField Field);

	/**
	 * Called when a value starts to be changed by a slider
	 */
	void OnBeginSliderMovement();

	/**
	 * Called when a value stops being changed by a slider
	 *
	 * @param	NewValue		The new value of the property
	 */
	void OnEndSliderMovement(NumericType NewValue);

	/**
	 * Sets the interval field specified to a new value
	 *
	 * @param	NewValue		The new value of the property
	 * @param	Field			Which TInterval field is being set
	 * @param	Flags			Flags specifying how the property should be set
	 */
	void SetValue(NumericType NewValue, EIntervalField Field, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);

	/**
	 * Determines if the spin box is enabled on a numeric value widget
	 *
	 * @return Whether the spin box should be enabled.
	 */
	bool ShouldAllowSpin() const;

	/* If the property is disable we must disable the editing */
	bool IsPropertyEnabled(EIntervalField Field) const;

	// Cached shared pointers to properties that we are managing
	TSharedPtr<IPropertyHandle> MinValueHandle;
	TSharedPtr<IPropertyHandle> MaxValueHandle;

	// Min and max allowed values from the metadata
	TOptional<NumericType> MinAllowedValue;
	TOptional<NumericType> MaxAllowedValue;
	TOptional<NumericType> MinAllowedSliderValue;
	TOptional<NumericType> MaxAllowedSliderValue;
	
	// Flags whether the slider is being moved at the moment on any of our widgets
	bool bIsUsingSlider;

	// Specifies that the Min value may be set greater than the Max value
	bool bAllowInvertedInterval;

	// Specifies that the Min value imposes a minimum limit on the Max value, and vice versa
	bool bClampToMinMaxLimits;
};
