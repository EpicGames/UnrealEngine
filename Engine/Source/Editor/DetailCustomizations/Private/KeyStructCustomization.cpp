// Copyright Epic Games, Inc. All Rights Reserved.

#include "KeyStructCustomization.h"

#include "Containers/Array.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "InputSettingsDetails.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FKeyStructCustomization"

FKeyStructCustomization::FKeyStructCustomization() = default;

/* FKeyStructCustomization static interface
 *****************************************************************************/

void FKeyStructCustomization::SetEnableKeySelector(bool bKeySelectorEnabled)
{
	bEnableKeySelector = bKeySelectorEnabled;
	if (KeySelector)
	{
		KeySelector->SetEnabledFromKeyStructCustomization(bEnableKeySelector);
	}
}

void FKeyStructCustomization::SetKey(const FString& KeyName)
{
	PropertyHandle->SetValueFromFormattedString(KeyName);
}

TSharedRef<IPropertyTypeCustomization> FKeyStructCustomization::MakeInstance( )
{
	return MakeShareable(new FKeyStructCustomization);
}

/* IPropertyTypeCustomization interface
 *****************************************************************************/

void FKeyStructCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	PropertyHandle = StructPropertyHandle;

	// We only want to allow the FKey to be changed if the property is NOT const.
	const bool bCanChangeKey = !StructPropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_EditConst);
	
	// create struct header
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.0f)
	.MaxDesiredWidth(325.0f)
	[
		SNew(SKeySelector)
		.CurrentKey(this, &FKeyStructCustomization::GetCurrentKey)
		.OnKeyChanged(this, &FKeyStructCustomization::OnKeyChanged)
		.Font(StructCustomizationUtils.GetRegularFont())
		.AllowClear(!StructPropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_NoClear))
		.FilterBlueprintBindable(false)
		.AllowKeyChange(bCanChangeKey)
	];
}

void FKeyStructCustomization::CustomizeHeaderOnlyWithButton(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TSharedRef<SWidget> Button)
{
	PropertyHandle = StructPropertyHandle;

	KeySelector = SNew(SKeySelector)
		.CurrentKey(this, &FKeyStructCustomization::GetCurrentKey)
		.OnKeyChanged(this, &FKeyStructCustomization::OnKeyChanged)
		.Font(StructCustomizationUtils.GetRegularFont())
		.AllowClear(!StructPropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_NoClear))
		.FilterBlueprintBindable(false)
		.IsEnabled_Lambda([this]() -> bool
	    {
		    return bEnableKeySelector;
	    });
	
	KeySelector->SetEnabledFromKeyStructCustomization(bEnableKeySelector);
	KeySelector->SetDisabledKeySelectorToolTip(DisabledKeySelectorToolTip);
	
	// create struct header
	HeaderRow.NameContent()
	.MinDesiredWidth(125.0f)
	.MaxDesiredWidth(325.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		//.AutoWidth()
		[
			KeySelector.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.Padding(InputSettingsDetails::InputConstants::PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			Button
		]
	];
}

TOptional<FKey> FKeyStructCustomization::GetCurrentKey() const
{
	if (!bEnableKeySelector)
	{
		PropertyHandle->SetValueFromFormattedString(DefaultKeyName);
	}
	
	TArray<void*> StructPtrs;
	PropertyHandle->AccessRawData(StructPtrs);

	if (StructPtrs.Num() > 0)
	{
		FKey* SelectedKey = (FKey*)StructPtrs[0];

		if (SelectedKey)
		{
			for(int32 StructPtrIndex = 1; StructPtrIndex < StructPtrs.Num(); ++StructPtrIndex)
			{
				if (*(FKey*)StructPtrs[StructPtrIndex] != *SelectedKey)
				{
					return TOptional<FKey>();
				}
			}

			return *SelectedKey;
		}
	}

	return FKey();
}

void FKeyStructCustomization::OnKeyChanged(TSharedPtr<FKey> SelectedKey)
{
	PropertyHandle->SetValueFromFormattedString(SelectedKey->ToString());
}

#undef LOCTEXT_NAMESPACE
