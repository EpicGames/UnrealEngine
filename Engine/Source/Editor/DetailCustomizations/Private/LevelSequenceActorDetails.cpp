// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceActorDetails.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "LevelSequenceActorDetails"


TSharedRef<IDetailCustomization> FLevelSequenceActorDetails::MakeInstance()
{
	return MakeShareable( new FLevelSequenceActorDetails );
}

void AddAllSubObjectProperties(TArray<UObject*>& SubObjects, IDetailCategoryBuilder& Category, TAttribute<EVisibility> Visibility = TAttribute<EVisibility>(EVisibility::Visible))
{
	SubObjects.Remove(nullptr);
	if (!SubObjects.Num())
	{
		return;
	}

	for (const FProperty* TestProperty : TFieldRange<FProperty>(SubObjects[0]->GetClass()))
	{
		if (TestProperty->HasAnyPropertyFlags(CPF_Edit))
		{
			const bool bAdvancedDisplay = TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay);
			const EPropertyLocation::Type PropertyLocation = bAdvancedDisplay ? EPropertyLocation::Advanced : EPropertyLocation::Common;

			IDetailPropertyRow* NewRow = Category.AddExternalObjectProperty(SubObjects, TestProperty->GetFName(), PropertyLocation);
			if (NewRow)
			{
				NewRow->Visibility(Visibility);
			}
		}
	}
}

void FLevelSequenceActorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout.GetSelectedObjects();

	for( int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex )
	{
		const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[ObjectIndex];
		if ( CurrentObject.IsValid() )
		{
			ALevelSequenceActor* CurrentLevelSequenceActor = Cast<ALevelSequenceActor>(CurrentObject.Get());
			if (CurrentLevelSequenceActor != NULL)
			{
				LevelSequenceActor = CurrentLevelSequenceActor;
				break;
			}
		}
	}

	TArray<ALevelSequenceActor*> LevelSequenceActors;
	{
		TArray<TWeakObjectPtr<>> ObjectPtrs;
		DetailLayout.GetObjectsBeingCustomized(ObjectPtrs);

		for (TWeakObjectPtr<> WeakObj : ObjectPtrs)
		{
			if (auto* Actor = Cast<ALevelSequenceActor>(WeakObj.Get()))
			{
				LevelSequenceActors.Add(Actor);
			}
		}
	}
	

	IDetailCategoryBuilder& GeneralCategory = DetailLayout.EditCategory( "General", NSLOCTEXT("GeneralDetails", "General", "General"), ECategoryPriority::Important );

	GeneralCategory.AddCustomRow( NSLOCTEXT("LevelSequenceActorDetails", "OpenLevelSequence", "Open Level Sequence") )
	.RowTag("OpenLevelSequence")
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(0.f, 5.f, 10.f, 5.f)
		[
			SNew(SButton)
			.ContentPadding(3.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.IsEnabled( this, &FLevelSequenceActorDetails::CanOpenLevelSequenceForActor )
			.OnClicked( this, &FLevelSequenceActorDetails::OnOpenLevelSequenceForActor )
			.Text( NSLOCTEXT("LevelSequenceActorDetails", "OpenLevelSequence", "Open Level Sequence") )
		]
	];

	TArray<UObject*> SubObjects;

	IDetailCategoryBuilder& BurnInOptionsCategory = DetailLayout.EditCategory( "BurnInOptions", LOCTEXT("BurnInOptions", "Burn In Options") ).InitiallyCollapsed(false);
	{
		SubObjects.Reset();
		Algo::Transform(LevelSequenceActors, SubObjects, &ALevelSequenceActor::BurnInOptions);

		AddAllSubObjectProperties(SubObjects, BurnInOptionsCategory);
	}

	IDetailCategoryBuilder& BindingOverridesCategory = DetailLayout.EditCategory( "BindingOverrides", LOCTEXT("BindingOverrides", "Binding Overrides") ).InitiallyCollapsed(false);
	{
		SubObjects.Reset();
		Algo::Transform(LevelSequenceActors, SubObjects, &ALevelSequenceActor::BindingOverrides);
		AddAllSubObjectProperties(SubObjects, BindingOverridesCategory);
	}

	IDetailCategoryBuilder& InstanceDataCategory = DetailLayout.EditCategory( "InstanceData", LOCTEXT("InstanceData", "Instance Data") ).InitiallyCollapsed(false);
	{
		TSharedRef<IPropertyHandle> UseInstanceData = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ALevelSequenceActor, bOverrideInstanceData));

		DetailLayout.HideProperty(UseInstanceData);
		InstanceDataCategory.AddProperty(UseInstanceData);

		SubObjects.Reset();
		Algo::Transform(LevelSequenceActors, SubObjects, &ALevelSequenceActor::DefaultInstanceData);

		auto IsVisible = [UseInstanceData]() -> EVisibility
		{
			bool bValue = false;
			return UseInstanceData->GetValue(bValue) == FPropertyAccess::Success && bValue ? EVisibility::Visible : EVisibility::Collapsed;
		};

		TAttribute<EVisibility> Visibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda(IsVisible));
		AddAllSubObjectProperties(SubObjects, InstanceDataCategory, Visibility);
	}
}


bool FLevelSequenceActorDetails::CanOpenLevelSequenceForActor() const
{
	if( LevelSequenceActor.IsValid() )
	{
		return LevelSequenceActor.Get()->GetSequence() != nullptr;
	}
	return false;
}

FReply FLevelSequenceActorDetails::OnOpenLevelSequenceForActor()
{
	if( LevelSequenceActor.IsValid() )
	{
		UObject* LoadedObject = LevelSequenceActor.Get()->GetSequence();
		if (LoadedObject != nullptr)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LoadedObject);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
