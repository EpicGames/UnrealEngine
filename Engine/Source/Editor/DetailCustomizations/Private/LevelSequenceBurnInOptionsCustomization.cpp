// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceBurnInOptionsCustomization.h"

#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Layout/Visibility.h"
#include "LevelSequenceActor.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"

class UObject;

void AddPropertiesRecursive(TSharedRef<IPropertyHandle> Property, IDetailChildrenBuilder& ChildBuilder)
{
	uint32 NumChildren = 0;
	Property->GetNumChildren(NumChildren);

	for (uint32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedRef<IPropertyHandle> Child = Property->GetChildHandle(Index).ToSharedRef();
		if (Child->GetProperty())
		{
			ChildBuilder.AddProperty(Child);
		}
		else
		{
			AddPropertiesRecursive(Child, ChildBuilder);
		}
	}
}

void FLevelSequenceBurnInOptionsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FLevelSequenceBurnInOptionsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyUtilities> Utils = StructCustomizationUtils.GetPropertyUtilities();

	FSimpleDelegate Refresh = FSimpleDelegate::CreateRaw(Utils.Get(), &IPropertyUtilities::RequestRefresh);

	{
		TSharedRef<IPropertyHandle> Property = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(ULevelSequenceBurnInOptions, bUseBurnIn)).ToSharedRef();
		Property->SetOnPropertyValueChanged(Refresh);
		Property->SetOnChildPropertyValueChanged(Refresh);
	}

	{
		TSharedRef<IPropertyHandle> Property = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(ULevelSequenceBurnInOptions, BurnInClass)).ToSharedRef();
		Property->SetOnPropertyValueChanged(Refresh);
		Property->SetOnChildPropertyValueChanged(Refresh);
	}

	AddPropertiesRecursive(StructPropertyHandle, ChildBuilder);
}

void FLevelSequenceBurnInInitSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda(
		[PropertyHandle]
		{
			UObject* ObjectValue = nullptr;
			PropertyHandle->GetValue(ObjectValue);
			return ObjectValue ? EVisibility::Visible : EVisibility::Collapsed;
		}
	)));

	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FLevelSequenceBurnInInitSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	AddPropertiesRecursive(StructPropertyHandle, ChildBuilder);
}
