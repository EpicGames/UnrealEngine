// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightComponentDetails.h"

#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/LocalLightComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Scene.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "DetailWidgetRow.h"
#include "PropertyEditorArchetypePolicy.h"

#define LOCTEXT_NAMESPACE "LightComponentDetails"

TSharedRef<IDetailCustomization> FLightComponentDetails::MakeInstance()
{
	return MakeShareable( new FLightComponentDetails );
}

void FLightComponentDetails::AddLocalLightIntensityWithUnit(IDetailLayoutBuilder& DetailBuilder, ULocalLightComponent* Component)
{
	float ConversionFactor = 1.f;
	uint8 Value = 0; // Unitless
	if (IntensityUnitsProperty->GetValue(Value) == FPropertyAccess::Success)
	{
		ConversionFactor = ULocalLightComponent::GetUnitsConversionFactor((ELightUnits)0, (ELightUnits)Value);
	}
	IntensityUnitsProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FLightComponentDetails::OnIntensityUnitsPreChange, Component));
	IntensityUnitsProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLightComponentDetails::OnIntensityUnitsChanged, Component));

	// Inverse squared falloff point lights (the default) are in units of lumens, instead of just being a brightness scale
	if (Component->IntensityUnits == ELightUnits::EV)
	{
		// +/-32 stops give a large enough range
		LightIntensityProperty->SetInstanceMetaData("UIMin",TEXT("-32.0f"));
		LightIntensityProperty->SetInstanceMetaData("UIMax",TEXT("32.0f"));
	}
	else
	{
		LightIntensityProperty->SetInstanceMetaData("UIMin",TEXT("0.0f"));
		LightIntensityProperty->SetInstanceMetaData("UIMax",  *FString::SanitizeFloat(100000.0f * ConversionFactor));
		LightIntensityProperty->SetInstanceMetaData("SliderExponent", TEXT("2.0f"));
	}
	if (Component->IntensityUnits == ELightUnits::Lumens)
	{
		LightIntensityProperty->SetInstanceMetaData("Units", TEXT("lm"));
		LightIntensityProperty->SetToolTipText(LOCTEXT("LightIntensityInLumensToolTipText", "Luminous power or flux in lumens"));
	}
	else if (Component->IntensityUnits == ELightUnits::Candelas)
	{
		LightIntensityProperty->SetInstanceMetaData("Units", TEXT("cd"));
		LightIntensityProperty->SetToolTipText(LOCTEXT("LightIntensityInCandelasToolTipText", "Luminous intensity in candelas"));
	}
	else if (Component->IntensityUnits == ELightUnits::EV)
	{
		LightIntensityProperty->SetInstanceMetaData("Units", TEXT("ev"));
		LightIntensityProperty->SetToolTipText(LOCTEXT("LightIntensityInEVToolTipText", "Luminous intensity in EV100"));
	}
	else if (Component->IntensityUnits == ELightUnits::Nits)
	{
		LightIntensityProperty->SetInstanceMetaData("Units", TEXT("nt"));
		LightIntensityProperty->SetToolTipText(LOCTEXT("LightIntensityInNitsToolTipText", "Luminance in nits"));
	}

	// Make these come first
	IDetailCategoryBuilder& LightCategory = DetailBuilder.EditCategory( "Light", FText::GetEmpty(), ECategoryPriority::TypeSpecific );

	// Target version have intensity value and unity sitting side by side. Currently there is a 
	// bug causing the reset button to not reset the intensity value, which is why this version 
	// is not enabled by default for now
#if 0
	// Add light intensity and unit on the same line
	FDetailWidgetRow& IntensityAndUnitRow = DetailBuilder.AddCustomRowToCategory(LightIntensityProperty, LightIntensityProperty->GetPropertyDisplayName())
	.NameContent()
	[
		LightIntensityProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 2.f, 10.f, 2.f)
		[
			LightIntensityProperty->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			IntensityUnitsProperty->CreatePropertyValueWidget()
		]
	];

	if (!IESBrightnessEnabledProperty->IsValidHandle())
	{
		TWeakObjectPtr<ULightComponent> BaseComponent = Component;
		IntensityAndUnitRow
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(this, &FLightComponentDetails::IsIntensityResetToDefaultVisible, BaseComponent), FResetToDefaultHandler::CreateSP(this, &FLightComponentDetails::ResetIntensityToDefault, BaseComponent)));
	}
	else
	{
		TWeakObjectPtr<ULightComponent> BaseComponent = Component;
		IntensityAndUnitRow
			.IsEnabled(TAttribute<bool>(this, &FLightComponentDetails::IsLightBrightnessEnabled))
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(this, &FLightComponentDetails::IsIntensityResetToDefaultVisible, BaseComponent), FResetToDefaultHandler::CreateSP(this, &FLightComponentDetails::ResetIntensityToDefault, BaseComponent)));
	}
#else
	// Add light intensity and light unit onto two different lines
	if( !IESBrightnessEnabledProperty->IsValidHandle() )
	{
		LightCategory.AddProperty( LightIntensityProperty );
	}
	else
	{
		TWeakObjectPtr<ULightComponent> BaseComponent = Component;
		LightCategory.AddProperty( LightIntensityProperty )
			.IsEnabled( TAttribute<bool>( this, &FLightComponentDetails::IsLightBrightnessEnabled ) )
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(this, &FLightComponentDetails::IsIntensityResetToDefaultVisible, BaseComponent), FResetToDefaultHandler::CreateSP(this, &FLightComponentDetails::ResetIntensityToDefault, BaseComponent)));
	}
	LightCategory.AddProperty(IntensityUnitsProperty).OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(this, &FLightComponentDetails::IsIntensityUnitsResetToDefaultVisible, Component), FResetToDefaultHandler::CreateSP(this, &FLightComponentDetails::ResetIntensityUnitsToDefault, Component)));
#endif
}

void FLightComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	TWeakObjectPtr<ULightComponent> Component(Cast<ULightComponent>(Objects[0].Get()));

	// Mobility property is on the scene component base class not the light component and that is why we have to use USceneComponent::StaticClass
	TSharedRef<IPropertyHandle> MobilityHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, Mobility), USceneComponent::StaticClass());
	// Set a mobility tooltip specific to lights
	MobilityHandle->SetToolTipText(LOCTEXT("LightMobilityTooltip", "Mobility for lights controls what the light is allowed to do at runtime and therefore what rendering methods are used.\n* A movable light uses fully dynamic lighting and anything can change in game, however it has a large performance cost, typically proportional to the light's influence size.\n* A stationary light will only have its shadowing and bounced lighting from static geometry baked by Lightmass, all other lighting will be dynamic.  It can change color and intensity in game. \n* A static light is fully baked into lightmaps and therefore has no performance cost, but also can't change in game."));

	IDetailCategoryBuilder& LightCategory = DetailBuilder.EditCategory( "Light", FText::GetEmpty(), ECategoryPriority::TypeSpecific );

	// The bVisible checkbox in the rendering category is frequently used on lights
	// Editing the rendering category and giving it TypeSpecific priority will place it just under the Light category
	DetailBuilder.EditCategory("Rendering", FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	LightIntensityProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, Intensity), ULightComponentBase::StaticClass());
	IntensityUnitsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULocalLightComponent, IntensityUnits), ULocalLightComponent::StaticClass());
	IESBrightnessTextureProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, IESTexture));
	IESBrightnessEnabledProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, bUseIESBrightness));
	IESBrightnessScaleProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, IESBrightnessScale));

	if( !IESBrightnessEnabledProperty->IsValidHandle() )
	{
		// Brightness and color should be listed first
		if (ULocalLightComponent* LocalComponent = Cast<ULocalLightComponent>(Objects[0].Get()))
		{
			AddLocalLightIntensityWithUnit(DetailBuilder, LocalComponent);
		}
		else
		{
			LightCategory.AddProperty( LightIntensityProperty );
		}
		LightCategory.AddProperty( DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULightComponent, LightColor), ULightComponentBase::StaticClass() ) );
	}
	else
	{
		if (ULocalLightComponent* LocalComponent = Cast<ULocalLightComponent>(Objects[0].Get()))
		{
			AddLocalLightIntensityWithUnit(DetailBuilder, LocalComponent);
		}
		else
		{
			LightCategory.AddProperty( LightIntensityProperty )
				.IsEnabled( TAttribute<bool>( this, &FLightComponentDetails::IsLightBrightnessEnabled ) )
				.OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateSP(this, &FLightComponentDetails::IsIntensityResetToDefaultVisible, Component), FResetToDefaultHandler::CreateSP(this, &FLightComponentDetails::ResetIntensityToDefault, Component)));
		}

		LightCategory.AddProperty( DetailBuilder.GetProperty("LightColor", ULightComponentBase::StaticClass() ) );

		IDetailCategoryBuilder& LightProfilesCategory = DetailBuilder.EditCategory( "Light Profiles", LOCTEXT("LightProfiles", "Light Profiles"), ECategoryPriority::Default );
		LightProfilesCategory.AddProperty( IESBrightnessTextureProperty );

		LightProfilesCategory.AddProperty( IESBrightnessEnabledProperty )
			.IsEnabled( TAttribute<bool>( this, &FLightComponentDetails::IsUseIESBrightnessEnabled ) );

		LightProfilesCategory.AddProperty( IESBrightnessScaleProperty)
			.IsEnabled( TAttribute<bool>( this, &FLightComponentDetails::IsIESBrightnessScaleEnabled ) );
	}
}

bool FLightComponentDetails::IsLightBrightnessEnabled() const
{
	return !IsIESBrightnessScaleEnabled();
}

bool FLightComponentDetails::IsUseIESBrightnessEnabled() const
{
	UObject* IESTexture;
	IESBrightnessTextureProperty->GetValue(IESTexture);
	return (IESTexture != NULL);
}

bool FLightComponentDetails::IsIESBrightnessScaleEnabled() const
{
	bool Enabled;
	IESBrightnessEnabledProperty->GetValue(Enabled);
	return IsUseIESBrightnessEnabled() && Enabled;
}

void FLightComponentDetails::SetComponentIntensity(ULightComponent* Component, float InIntensity)
{
	check(Component);

	FProperty* IntensityProperty = FindFieldChecked<FProperty>(ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponent, Intensity));
	FPropertyChangedEvent PropertyChangedEvent(IntensityProperty);

	const float PreviousIntensity = Component->Intensity;
	Component->SetLightBrightness(InIntensity);
	Component->PostEditChangeProperty(PropertyChangedEvent);
	Component->MarkRenderStateDirty();

	// Propagate changes to instances.
	TArray<UObject*> Instances;
	Component->GetArchetypeInstances(Instances);
	for (UObject* Instance : Instances)
	{
		ULocalLightComponent* InstanceComponent = Cast<ULocalLightComponent>(Instance);
		if (InstanceComponent && InstanceComponent->Intensity == PreviousIntensity)
		{
			InstanceComponent->Intensity = Component->Intensity;
			InstanceComponent->PostEditChangeProperty(PropertyChangedEvent);
			InstanceComponent->MarkRenderStateDirty();
		}
	}
}

void FLightComponentDetails::ResetIntensityToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TWeakObjectPtr<ULightComponent> Component)
{
	ULightComponent* ArchetypeComponent = Component.IsValid() ? Cast<ULocalLightComponent>(PropertyEditorPolicy::GetArchetype(Component.Get())) : nullptr;
	if (ArchetypeComponent)
	{
		SetComponentIntensity(Component.Get(), ArchetypeComponent->ComputeLightBrightness());
	}
	else
	{
		// Fall back to default handler. 
		PropertyHandle->ResetToDefault();
	}
}

bool FLightComponentDetails::IsIntensityResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, TWeakObjectPtr<ULightComponent> Component) const
{
	ULightComponent* ArchetypeComponent = Component.IsValid() ? Cast<ULocalLightComponent>(PropertyEditorPolicy::GetArchetype(Component.Get())) : nullptr;
	if (ArchetypeComponent)
	{
		return !FMath::IsNearlyEqual(Component->ComputeLightBrightness(), ArchetypeComponent->ComputeLightBrightness());
	}
	else
	{
		// Fall back to default handler
		return PropertyHandle->DiffersFromDefault();
	}
}

void FLightComponentDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FLightComponentDetails::OnIntensityUnitsPreChange(ULocalLightComponent* Component)
{
	if (Component)
	{
		LastLightBrigtness = Component->ComputeLightBrightness();
	}
}

void FLightComponentDetails::OnIntensityUnitsChanged(ULocalLightComponent* Component)
{
	// Convert the brightness using the new units.
	if (Component)
	{
		FLightComponentDetails::SetComponentIntensity(Component, LastLightBrigtness);
	}

	// Here we can only take the ptr as ForceRefreshDetails() checks that the reference is unique.
	IDetailLayoutBuilder* DetailBuilder = CachedDetailBuilder.Pin().Get();
	if (DetailBuilder)
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

namespace
{
	void SetComponentIntensityUnits(ULocalLightComponent* Component, ELightUnits InUnits)
	{
		check(Component);

		FProperty* IntensityUnitsProperty = FindFieldChecked<FProperty>(ULocalLightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULocalLightComponent, IntensityUnits));
		FPropertyChangedEvent PropertyChangedEvent(IntensityUnitsProperty);

		const ELightUnits PreviousUnits = Component->IntensityUnits;
		Component->IntensityUnits = InUnits;
		Component->PostEditChangeProperty(PropertyChangedEvent);
		Component->MarkRenderStateDirty();

		// Propagate changes to instances.
		TArray<UObject*> Instances;
		Component->GetArchetypeInstances(Instances);
		for (UObject* Instance : Instances)
		{
			ULocalLightComponent* InstanceComponent = Cast<ULocalLightComponent>(Instance);
			if (InstanceComponent && InstanceComponent->IntensityUnits == PreviousUnits)
			{
				InstanceComponent->IntensityUnits = Component->IntensityUnits;
				InstanceComponent->PostEditChangeProperty(PropertyChangedEvent);
				InstanceComponent->MarkRenderStateDirty();
			}
		}
	}
}

void FLightComponentDetails::ResetIntensityUnitsToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, ULocalLightComponent* Component)
{
	// Actors (and blueprints) spawned from the actor factory inherit the intensity units from the project settings.
	if (Component && PropertyEditorPolicy::GetArchetype(Component) && !PropertyEditorPolicy::GetArchetype(Component)->IsInBlueprint())
	{
		static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
		const ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnGameThread();

		if (DefaultUnits != Component->IntensityUnits)
		{
			SetComponentIntensityUnits(Component, DefaultUnits);
		}
	}
	else
	{
		// Fall back to default handler. 
		PropertyHandle->ResetToDefault();
	}
}

bool FLightComponentDetails::IsIntensityUnitsResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, ULocalLightComponent* Component) const
{
	// Actors (and blueprints) spawned from the actor factory inherit the project settings.
	if (Component && PropertyEditorPolicy::GetArchetype(Component) && !PropertyEditorPolicy::GetArchetype(Component)->IsInBlueprint())
	{
		static const auto CVarDefaultLightUnits = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.LightUnits"));
		const ELightUnits DefaultUnits = (ELightUnits)CVarDefaultLightUnits->GetValueOnGameThread();
		return DefaultUnits != Component->IntensityUnits;
	}
	else
	{
		// Fall back to default handler
		return PropertyHandle->DiffersFromDefault();
	}
}

#undef LOCTEXT_NAMESPACE
