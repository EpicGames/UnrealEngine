// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class ULightComponent;
class ULocalLightComponent;

class FLightComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

	static void SetComponentIntensity(ULightComponent* Component, float InIntensity);

private:

	/** Helper functions which tell whether the various custom controls are enabled or not */
	bool IsLightBrightnessEnabled() const;
	bool IsUseIESBrightnessEnabled() const;
	bool IsIESBrightnessScaleEnabled() const;

	void ResetIntensityToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, TWeakObjectPtr<ULightComponent> Component);
	bool IsIntensityResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, TWeakObjectPtr<ULightComponent> Component) const;

private:
	TSharedPtr<IPropertyHandle> IESBrightnessTextureProperty;
	TSharedPtr<IPropertyHandle> IESBrightnessEnabledProperty;
	TSharedPtr<IPropertyHandle> IESBrightnessScaleProperty;
	TSharedPtr<IPropertyHandle> LightIntensityProperty;
	TSharedPtr<IPropertyHandle> IntensityUnitsProperty;

	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;
	void ResetIntensityUnitsToDefault(TSharedPtr<IPropertyHandle> PropertyHandle, ULocalLightComponent* Component);
	bool IsIntensityUnitsResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle, ULocalLightComponent* Component) const;

	/** Called when the intensity units are changed */
	void OnIntensityUnitsPreChange(ULocalLightComponent* Component);
	void OnIntensityUnitsChanged(ULocalLightComponent* Component);

	/* Add local light intensity with unit */
	void AddLocalLightIntensityWithUnit(IDetailLayoutBuilder& DetailBuilder, ULocalLightComponent* Component);

	float LastLightBrigtness = 0;
};
