// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightingChannelsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FLightingChannelsCustomization"

TSharedRef<IPropertyTypeCustomization> FLightingChannelsCustomization::MakeInstance()
{
	return MakeShareable( new FLightingChannelsCustomization );
}

void FLightingChannelsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	LightingChannelsHandle = StructPropertyHandle;

	TSharedRef<SHorizontalBox> ButtonOptionsPanel =
		SNew(SHorizontalBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
	;
	
	HeaderRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(this, &FLightingChannelsCustomization::GetStructPropertyNameText)
		.ToolTipText(this, &FLightingChannelsCustomization::GetStructPropertyTooltipText)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Visibility(EVisibility::SelfHitTestInvisible)
	]
	.ValueContent()
	[
		ButtonOptionsPanel
	];

	uint32 ChildCount;
	LightingChannelsHandle->GetNumChildren(ChildCount);

	for (uint32 ChildIndex = 0; ChildIndex < ChildCount; ChildIndex++)
	{
		const FText NumericText = FText::AsNumber(ChildIndex);
		const FText SlotTooltipText = FText::Format(LOCTEXT("LightingChannelToggleFormat", "Toggle Lighting Channel {0}"), NumericText);

		const bool bIsLastChild = ChildIndex == ChildCount - 1;
		
		ButtonOptionsPanel->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.f, 0.f, bIsLastChild ? 0.f : 8.f, 0.f)
		[
			SNew(SBox)
			.WidthOverride(20.f)
			.HAlign(HAlign_Fill)
			.IsEnabled(this, &FLightingChannelsCustomization::IsLightingChannelButtonEditable, ChildIndex)
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.ChannelToggleButton"))
				.ToolTipText(SlotTooltipText)
				.OnCheckStateChanged(this, &FLightingChannelsCustomization::OnButtonCheckedStateChanged, ChildIndex)
				.IsChecked(this, &FLightingChannelsCustomization::GetButtonCheckedState, ChildIndex)
				.HAlign(HAlign_Center)
				.Padding(FMargin(0.f, 2.f))
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("SmallText"))
					.Visibility(EVisibility::HitTestInvisible)
					.Text(NumericText)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
	}
}

void FLightingChannelsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Display channels as a normal foldout struct
	uint32 ChildCount;
	PropertyHandle->GetNumChildren(ChildCount);

	for (uint32 ChildIndex = 0; ChildIndex < ChildCount; ChildIndex++)
	{
		if (TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex))
		{
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

FText FLightingChannelsCustomization::GetStructPropertyNameText() const
{
	if (LightingChannelsHandle.IsValid())
	{
		return LightingChannelsHandle->GetPropertyDisplayName();
	}

	return FText::GetEmpty();
}

FText FLightingChannelsCustomization::GetStructPropertyTooltipText() const
{
	if (LightingChannelsHandle.IsValid())
	{
		return LightingChannelsHandle->GetToolTipText();
	}

	return FText::GetEmpty();
}

bool FLightingChannelsCustomization::IsLightingChannelButtonEditable(uint32 ChildIndex) const
{
	if (LightingChannelsHandle.IsValid())
	{
		if (TSharedPtr<IPropertyHandle> ChildHandle = LightingChannelsHandle->GetChildHandle(ChildIndex))
		{
			return ChildHandle->IsEditable() && !ChildHandle->IsEditConst();
		}
	}
	return false;
}

void FLightingChannelsCustomization::OnButtonCheckedStateChanged(ECheckBoxState NewState, uint32 ChildIndex) const
{
	if (LightingChannelsHandle.IsValid())
	{
		uint32 OutNumChildren = 0;
		LightingChannelsHandle->GetNumChildren(OutNumChildren);

		if (ChildIndex < OutNumChildren)
		{
			LightingChannelsHandle->GetChildHandle(ChildIndex)->SetValue(NewState == ECheckBoxState::Checked);
			LightingChannelsHandle->GetChildHandle(ChildIndex)->NotifyFinishedChangingProperties();
			LightingChannelsHandle->NotifyFinishedChangingProperties();
		}
	}
}

ECheckBoxState FLightingChannelsCustomization::GetButtonCheckedState(uint32 ChildIndex) const
{
	if (LightingChannelsHandle.IsValid())
	{
		uint32 OutNumChildren = 0;
		LightingChannelsHandle->GetNumChildren(OutNumChildren);

		if (ChildIndex < OutNumChildren)
		{
			bool Value;
			LightingChannelsHandle->GetChildHandle(ChildIndex)->GetValue(Value);
			return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Undetermined;
}

#undef LOCTEXT_NAMESPACE
