// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

#define UE_API DETAILCUSTOMIZATIONS_API

/**
 * Customizes Lighting Channels as a horizontal row of buttons.
 */
class FLightingChannelsCustomization : public IPropertyTypeCustomization
{
public:
	
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FLightingChannelsCustomization(){}

	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	UE_API FText GetStructPropertyNameText() const;
	UE_API FText GetStructPropertyTooltipText() const;

	UE_API bool IsLightingChannelButtonEditable(uint32 ChildIndex) const;
	UE_API void OnButtonCheckedStateChanged(ECheckBoxState NewState, uint32 ChildIndex) const;
	UE_API ECheckBoxState GetButtonCheckedState(uint32 ChildIndex) const;

	TSharedPtr<IPropertyHandle> LightingChannelsHandle;
	FCheckBoxStyle Style;
};

#undef UE_API
