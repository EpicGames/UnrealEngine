// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/LinearColorRampCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "SColorGradientEditor.h"
#include "Curves/LinearColorRamp.h"
#include "DetailWidgetRow.h"

FLinearColorRampCustomization::FLinearColorRampCustomization(const FLinearColorRampCustomizationArgs InArgs)
	: Args(InArgs)
{
}

TSharedRef<IPropertyTypeCustomization> FLinearColorRampCustomization::MakeInstance()
{
	return MakeShared<FLinearColorRampCustomization>();
}

TSharedRef<IPropertyTypeCustomization> FLinearColorRampCustomization::MakeInstance(const FLinearColorRampCustomizationArgs InArgs)
{
	return MakeShared<FLinearColorRampCustomization>(InArgs);
}

void FLinearColorRampCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (const FStructProperty* const StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		if (StructProperty->Struct == FLinearColorRamp::StaticStruct())
		{
			void* Data = nullptr;

			if (PropertyHandle->GetValueData(Data) == FPropertyAccess::Success)
			{
				FLinearColorRamp* ColorRamp = reinterpret_cast<FLinearColorRamp*>(Data);

				TSharedPtr<SColorGradientEditor> GradientEditor;

				HeaderRow
					[
						SAssignNew(GradientEditor, SColorGradientEditor)
							.ViewMinInput(0.0f)
							.ViewMaxInput(1.0f)
							.ClampStopsToViewRange(true)
							.WithAlphaChannel(Args.bWithAlphaChannel)
					];

				GradientEditor->SetCurveOwner(ColorRamp);
			}
		}
	}
}


