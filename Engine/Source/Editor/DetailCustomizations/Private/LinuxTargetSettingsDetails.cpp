// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxTargetSettingsDetails.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorDirectories.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "SExternalImageReference.h"
#include "RHIDefinitions.h"
#include "RHIShaderFormatDefinitions.inl"
#include "ShaderFormatsPropertyDetails.h"

#if WITH_ENGINE
#include "AudioDevice.h"
#endif 

#define LOCTEXT_NAMESPACE "LinuxTargetSettingsDetails"

namespace LinuxTargetSettingsDetailsConstants
{
	/** The filename for the game splash screen */
	const FString GameSplashFileName(TEXT("Splash/Splash.bmp"));

	/** The filename for the editor splash screen */
	const FString EditorSplashFileName(TEXT("Splash/EdSplash.bmp"));

	/** ToolTip used when an option is not available to binary users. */
	const FText DisabledTip = LOCTEXT("GitHubSourceRequiredToolTip", "This requires GitHub source.");
}

static FText GetFriendlyNameFromLinuxShaderFormat(const FName InShaderFormat)
{
	FText FriendlyRHIName;

	if (InShaderFormat == NAME_GLSL_150_ES31)
	{
		FriendlyRHIName = LOCTEXT("OpenGL3ES31", "OpenGL 3 (Mobile, Experimental)");
	}
	else if (InShaderFormat == NAME_VULKAN_ES3_1_ANDROID || InShaderFormat == NAME_VULKAN_ES3_1)
	{
		FriendlyRHIName = LOCTEXT("Vulkan ES31", "Vulkan Mobile (ES3.1)");
	}
	else if (InShaderFormat == NAME_VULKAN_SM5)
	{
		FriendlyRHIName = LOCTEXT("VulkanSM5", "Vulkan Desktop (SM5)");
	}
	else if (InShaderFormat == NAME_VULKAN_SM6)
	{
		FriendlyRHIName = LOCTEXT("VulkanSM6", "Vulkan Desktop (SM6)");
	}
	else if (InShaderFormat == TEXT("GLSL_430"))
	{
		// Explicitly remove these formats as they are obsolete/not quite supported; users can still target them by adding them as +TargetedRHIs in the TargetPlatform ini.
		FriendlyRHIName = FText::GetEmpty();
	}
	else
	{
		FriendlyRHIName = LOCTEXT("UnknownRHI", "UnknownRHI");
	}

	return FriendlyRHIName;
}


TSharedRef<IDetailCustomization> FLinuxTargetSettingsDetails::MakeInstance()
{
	return MakeShareable(new FLinuxTargetSettingsDetails);
}

namespace ELinuxImageScope
{
	enum Type
	{
		Engine,
		GameOverride
	};
}

/* Helper function used to generate filenames for splash screens */
static FString GetLinuxSplashFilename(ELinuxImageScope::Type Scope, bool bIsEditorSplash)
{
	FString Filename;

	if (Scope == ELinuxImageScope::Engine)
	{
		Filename = FPaths::EngineContentDir();
	}
	else
	{
		Filename = FPaths::ProjectContentDir();
	}

	if(bIsEditorSplash)
	{
		Filename /= LinuxTargetSettingsDetailsConstants::EditorSplashFileName;
	}
	else
	{
		Filename /= LinuxTargetSettingsDetailsConstants::GameSplashFileName;
	}

	Filename = FPaths::ConvertRelativePathToFull(Filename);

	return Filename;
}

/* Helper function used to generate filenames for icons */
static FString GetLinuxIconFilename(ELinuxImageScope::Type Scope)
{
	if (Scope == ELinuxImageScope::Engine)
	{
		FString Filename = FPaths::EngineDir() / FString(TEXT("Source/Runtime/Launch/Resources/Linux/UnrealEngine.png"));
		return FPaths::ConvertRelativePathToFull(Filename);
	}
	else
	{
		FString Filename = FPaths::ProjectDir() / TEXT("Build/Linux/Application.png");
		if(!FPaths::FileExists(Filename))
		{
			FString LegacyFilename = FPaths::GameSourceDir() / FString(FApp::GetProjectName()) / FString(TEXT("Resources/Linux")) / FString(FApp::GetProjectName()) + TEXT(".icns");
			if(FPaths::FileExists(LegacyFilename))
			{
				Filename = LegacyFilename;
			}
		}
		return FPaths::ConvertRelativePathToFull(Filename);
	}
}

void FLinuxTargetSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Setup the supported/targeted RHI property view
	ITargetPlatformSettings* TargetPlatformSettings = FModuleManager::GetModuleChecked<ITargetPlatformSettingsModule>("LinuxTargetPlatformSettings").GetTargetPlatformSettings()[0];
	TargetShaderFormatsDetails = MakeShareable(new FShaderFormatsPropertyDetails(&DetailBuilder));
	TargetShaderFormatsDetails->CreateTargetShaderFormatsPropertyView(TargetPlatformSettings, &GetFriendlyNameFromLinuxShaderFormat);

	// Next add the splash image customization
	const FText EditorSplashDesc(LOCTEXT("EditorSplashLabel", "Editor Splash"));
	IDetailCategoryBuilder& SplashCategoryBuilder = DetailBuilder.EditCategory(TEXT("Splash"));
	FDetailWidgetRow& EditorSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(EditorSplashDesc);

	const FString EditorSplash_TargetImagePath = GetLinuxSplashFilename(ELinuxImageScope::GameOverride, true);
	const FString EditorSplash_DefaultImagePath = GetLinuxSplashFilename(ELinuxImageScope::Engine, true);

	TArray<FString> ImageExtensions;
	ImageExtensions.Add(TEXT("png"));
	ImageExtensions.Add(TEXT("jpg"));
	ImageExtensions.Add(TEXT("bmp"));

	EditorSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(EditorSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, EditorSplash_DefaultImagePath, EditorSplash_TargetImagePath)
			.FileDescription(EditorSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FLinuxTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FLinuxTargetSettingsDetails::HandlePostExternalIconCopy))
			.DeleteTargetWhenDefaultChosen(true)
			.FileExtensions(ImageExtensions)
			.DeletePreviousTargetWhenExtensionChanges(true)
		]
	];

	const FText GameSplashDesc(LOCTEXT("GameSplashLabel", "Game Splash"));
	FDetailWidgetRow& GameSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(GameSplashDesc);
	const FString GameSplash_TargetImagePath = GetLinuxSplashFilename(ELinuxImageScope::GameOverride, false);
	const FString GameSplash_DefaultImagePath = GetLinuxSplashFilename(ELinuxImageScope::Engine, false);

	GameSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(GameSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GameSplash_DefaultImagePath, GameSplash_TargetImagePath)
			.FileDescription(GameSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FLinuxTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FLinuxTargetSettingsDetails::HandlePostExternalIconCopy))
			.DeleteTargetWhenDefaultChosen(true)
			.FileExtensions(ImageExtensions)
			.DeletePreviousTargetWhenExtensionChanges(true)
		]
	];

	IDetailCategoryBuilder& IconsCategoryBuilder = DetailBuilder.EditCategory(TEXT("Icon"));
	FDetailWidgetRow& GameIconWidgetRow = IconsCategoryBuilder.AddCustomRow(LOCTEXT("GameIconLabel", "Game Icon"));
	GameIconWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("GameIconLabel", "Game Icon"))
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GetLinuxIconFilename(ELinuxImageScope::Engine), GetLinuxIconFilename(ELinuxImageScope::GameOverride))
			.FileDescription(GameSplashDesc)
			.OnPreExternalImageCopy(FOnPreExternalImageCopy::CreateSP(this, &FLinuxTargetSettingsDetails::HandlePreExternalIconCopy))
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FLinuxTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FLinuxTargetSettingsDetails::HandlePostExternalIconCopy))
		]
	];

	AudioPluginWidgetManager.BuildAudioCategory(DetailBuilder, FString(TEXT("Linux")));
}


bool FLinuxTargetSettingsDetails::HandlePreExternalIconCopy(const FString& InChosenImage)
{
	return true;
}


FString FLinuxTargetSettingsDetails::GetPickerPath()
{
	return FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
}


bool FLinuxTargetSettingsDetails::HandlePostExternalIconCopy(const FString& InChosenImage)
{
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InChosenImage));
	return true;
}

void FLinuxTargetSettingsDetails::HandleAudioDeviceSelected(FString AudioDeviceName, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(AudioDeviceName);
}

FSlateColor FLinuxTargetSettingsDetails::HandleAudioDeviceBoxForegroundColor(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	FString Value;

	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		if (Value.IsEmpty() || IsValidAudioDeviceName(Value))
		{
			static const FName InvertedForegroundName("InvertedForeground");

			// Return a valid slate color for a valid audio device
			return FAppStyle::GetSlateColor(InvertedForegroundName);
		}
	}

	// Return Red, which means its an invalid audio device
	return FLinearColor::Red;
}

FText FLinuxTargetSettingsDetails::HandleAudioDeviceTextBoxText(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	FString Value;

	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		FString LinuxAudioDeviceName;
		GConfig->GetString(TEXT("/Script/LinuxTargetPlatform.LinuxTargetSettings"), TEXT("AudioDevice"), LinuxAudioDeviceName, GEngineIni);
		return FText::FromString(LinuxAudioDeviceName);
	}

	return FText::GetEmpty();
}

void FLinuxTargetSettingsDetails::HandleAudioDeviceTextBoxTextChanged(const FText& InText, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(InText.ToString());
}

void FLinuxTargetSettingsDetails::HandleAudioDeviceTextBoxTextComitted(const FText& InText, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FString Value;

	// Clear the property if its not valid
	if ((PropertyHandle->GetValue(Value) != FPropertyAccess::Success) || !IsValidAudioDeviceName(Value))
	{
		PropertyHandle->SetValue(FString());
	}
}

bool FLinuxTargetSettingsDetails::IsValidAudioDeviceName(const FString& InDeviceName) const
{
	bool bIsValid = false;

#if WITH_ENGINE
	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		TArray<FString> DeviceNames;
		AudioDevice->GetAudioDeviceList(DeviceNames);

		for (FString& DeviceName : DeviceNames)
		{
			if (InDeviceName == DeviceName)
			{
				bIsValid = true;
				break;
			}
		}
	}
#endif

	return bIsValid;
}

TSharedRef<SWidget> FLinuxTargetSettingsDetails::MakeAudioDeviceMenu(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, nullptr);

#if WITH_ENGINE
	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		TArray<FString> AudioDeviceNames;
		AudioDevice->GetAudioDeviceList(AudioDeviceNames);

		// Construct the custom menu widget from the list of device names
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("AudioDevicesSectionHeader", "Audio Devices"));
		{
			for (int32 i = 0; i < AudioDeviceNames.Num(); i++)
			{
				FUIAction Action(FExecuteAction::CreateRaw(this, &FLinuxTargetSettingsDetails::HandleAudioDeviceSelected, AudioDeviceNames[i], PropertyHandle));
				MenuBuilder.AddMenuEntry(
					FText::FromString(AudioDeviceNames[i]),
					FText::FromString(TEXT("")),
					FSlateIcon(),
					Action
					);
			}
		}
		MenuBuilder.EndSection();
	}
#endif

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
