// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "TargetPlatformAudioCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FShaderFormatsPropertyDetails;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

enum class ECheckBoxState : uint8;

/**
 * Manages the Transform section of a details view                    
 */
class FLinuxTargetSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	/** Delegate handler for before an icon is copied */
	bool HandlePreExternalIconCopy(const FString& InChosenImage);

	/** Delegate handler to get the path to start picking from */
	FString GetPickerPath();

	/** Delegate handler to set the path to start picking from */
	bool HandlePostExternalIconCopy(const FString& InChosenImage);

	/** Handles when a new audio device is selected from list of available audio devices. */
	void HandleAudioDeviceSelected(FString AudioDeviceName, TSharedPtr<IPropertyHandle> PropertyHandle);

	/** Handles changing the foreground color of the audio device box. */
	FSlateColor HandleAudioDeviceBoxForegroundColor(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	/** Handles getting text of the audio device list text block. */
	FText HandleAudioDeviceTextBoxText(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	/** Handles text changes in the audio device list text block. */
	void HandleAudioDeviceTextBoxTextChanged(const FText& InText, TSharedPtr<IPropertyHandle> PropertyHandle);

	/** Handles committing changes in the audio list text block. */
	void HandleAudioDeviceTextBoxTextComitted(const FText& InText, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> PropertyHandle);

protected:

	/** Checks if the device name is valid. */
	bool IsValidAudioDeviceName(const FString& InDeviceName) const;

	/** Creates a widget for the audio device picker. */
	TSharedRef<SWidget> MakeAudioDeviceMenu(const TSharedPtr<IPropertyHandle>& PropertyHandle);

private:
	/** Reference to the target shader formats property view */
	TSharedPtr<FShaderFormatsPropertyDetails> TargetShaderFormatsDetails;

	/** Manager for Audio Plugin widget. */
	FAudioPluginWidgetManager AudioPluginWidgetManager;
};
