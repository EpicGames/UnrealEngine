// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalLightComponentDetails.h"

#include "Components/LightComponentBase.h"
#include "Components/LocalLightComponent.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Scene.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LightComponentDetails.h"
#include "Misc/AssertionMacros.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "LocalLightComponentDetails"

TSharedRef<IDetailCustomization> FLocalLightComponentDetails::MakeInstance()
{
	return MakeShareable( new FLocalLightComponentDetails );
}

void FLocalLightComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	ULocalLightComponent* Component = Cast<ULocalLightComponent>(Objects[0].Get());

	IDetailCategoryBuilder& LightCategory = DetailBuilder.EditCategory("Light", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	LightCategory.AddProperty( DetailBuilder.GetProperty( GET_MEMBER_NAME_CHECKED(ULocalLightComponent, AttenuationRadius), ULocalLightComponent::StaticClass() ) );

}

#undef LOCTEXT_NAMESPACE
