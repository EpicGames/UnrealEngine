// Copyright Epic Games, Inc. All Rights Reserved.

#include "MarginCustomization.h"

#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"

TSharedRef<IPropertyTypeCustomization> FMarginStructCustomization::MakeInstance() 
{
	return MakeShareable( new FMarginStructCustomization() );
}

void FMarginStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	StructPropertyHandle = InStructPropertyHandle;

	const FString& UVSpaceString( StructPropertyHandle->GetProperty()->GetMetaData( TEXT( "UVSpace" ) ) );
	bIsMarginUsingUVSpace = UVSpaceString.Len() > 0 && UVSpaceString == TEXT( "true" );

	NumericInterface = MakeShareable(new TDefaultNumericTypeInterface<float>);

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren( NumChildren );

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		ChildPropertyHandles.Add( StructPropertyHandle->GetChildHandle( ChildIndex ).ToSharedRef() );
	}

	TSharedPtr<SHorizontalBox> HorizontalBox;

	HeaderRow.IsEnabled(TAttribute<bool>(InStructPropertyHandle, &IPropertyHandle::IsEditable));
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth( 250.0f )
	.MaxDesiredWidth( 250.0f )
	[
		SAssignNew( HorizontalBox, SHorizontalBox )
	];

	HorizontalBox->AddSlot()
	[
		MakePropertyWidget()
	];
}

void FMarginStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	const uint32 NumChildren = ChildPropertyHandles.Num();

	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ChildHandle = ChildPropertyHandles[ ChildIndex ];
		IDetailPropertyRow& PropertyRow = StructBuilder.AddProperty( ChildHandle );
		PropertyRow
		.IsEnabled(TAttribute<bool>(ChildHandle, &IPropertyHandle::IsEditable))
		.CustomWidget()
		.NameContent()
		[
			ChildHandle->CreatePropertyNameWidget( ChildHandle->GetPropertyDisplayName() )
		]
		.ValueContent()
		[
			MakeChildPropertyWidget( ChildIndex )
		];
	}
}

TSharedRef<SEditableTextBox> FMarginStructCustomization::MakePropertyWidget()
{
	return
		SAssignNew( MarginEditableTextBox,SEditableTextBox )
		.Text( this, &FMarginStructCustomization::GetMarginText )
		.ToolTipText( NSLOCTEXT( "UnrealEd", "MarginPropertyToolTip", "Margin values" ) )
		.OnTextCommitted( this, &FMarginStructCustomization::OnMarginTextCommitted )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
		.SelectAllTextWhenFocused( true )
		.SelectAllTextOnCommit( true )
		.ClearKeyboardFocusOnCommit( false )
	;
}

TSharedRef<SWidget> FMarginStructCustomization::MakeChildPropertyWidget( int32 PropertyIndex) const
{
	return
		SNew(SNumericEntryBox<float>)
		.Value(this, &FMarginStructCustomization::OnGetValue, PropertyIndex)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.OnValueCommitted(const_cast<FMarginStructCustomization*>(this), &FMarginStructCustomization::OnValueCommitted, PropertyIndex)
		.OnValueChanged(const_cast<FMarginStructCustomization*>(this), &FMarginStructCustomization::OnValueChanged, PropertyIndex)
		.OnBeginSliderMovement(const_cast<FMarginStructCustomization*>(this), &FMarginStructCustomization::OnBeginSliderMovement)
		.OnEndSliderMovement(const_cast<FMarginStructCustomization*>(this), &FMarginStructCustomization::OnEndSliderMovement)
		.LabelVAlign(VAlign_Center)
		.AllowSpin(bIsMarginUsingUVSpace ? true : false)
		.MinValue(bIsMarginUsingUVSpace ? 0.0f : TNumericLimits<float>::Lowest())
		.MaxValue(bIsMarginUsingUVSpace ? 1.0f : TNumericLimits<float>::Max())
		.MinSliderValue(bIsMarginUsingUVSpace ? 0.0f : TNumericLimits<float>::Lowest())
		.MaxSliderValue(bIsMarginUsingUVSpace ? 1.0f : TNumericLimits<float>::Max())
		.TypeInterface(NumericInterface);
}

FText FMarginStructCustomization::GetMarginText() const
{
	return FText::FromString( GetMarginTextFromProperties() );
}

void FMarginStructCustomization::OnMarginTextCommitted( const FText& InText, ETextCommit::Type InCommitType )
{
	FString InString( InText.ToString() );

	bool bError = false;

	if( InCommitType != ETextCommit::OnCleared )
	{
		TArray<float> PropertyValues;

		while( InString.Len() > 0 && !bError )
		{
			FString LeftString;
			const bool bSuccess = InString.Split( TEXT( "," ), &LeftString, &InString );

			if( !InString.IsEmpty() && ((bSuccess && !LeftString.IsEmpty()) || !bSuccess) )
			{
				if( !bSuccess )
				{
					LeftString = InString;
					InString.Empty();
				}

				LeftString.TrimStartAndEndInline();

				float Value = 0.f;
				TOptional<float> NumericValue = NumericInterface->FromString(LeftString, Value);
				if (NumericValue.IsSet())
				{
					Value = NumericValue.GetValue();
					PropertyValues.Add( bIsMarginUsingUVSpace ? FMath::Clamp( Value, 0.0f, 1.0f ) : FMath::Max( Value, 0.0f ) );
				}
				else
				{
					bError = true;
				}
			}
			else
			{
				bError = true;
			}
		}

		
		if( !bError )
		{

			FMargin NewMargin;
			// Update the property values
			if( PropertyValues.Num() == 1 )
			{
				// Uniform margin
				NewMargin = FMargin( PropertyValues[0] );
			}
			else if( PropertyValues.Num() == 2 )
			{
				// Uniform on the two axes
				NewMargin = FMargin( PropertyValues[0], PropertyValues[1] );
			}
			else if( PropertyValues.Num() == 4 )
			{
				NewMargin.Left = PropertyValues[0];
				NewMargin.Top = PropertyValues[1];
				NewMargin.Right = PropertyValues[2];
				NewMargin.Bottom = PropertyValues[3];
			}
			else
			{
				bError = true;
			}


			if (!bError)
			{
				if (bIsMarginUsingUVSpace)
				{
					if (NewMargin.Left + NewMargin.Right > 1.0f)
					{
						NewMargin.Left = 1.0f - NewMargin.Right;
					}

					if (NewMargin.Top + NewMargin.Bottom > 1.0f)
					{
						NewMargin.Top = 1.0f - NewMargin.Bottom;
					}
				}

				FScopedTransaction Transaction(FText::Format(NSLOCTEXT("FMarginStructCustomization", "SetMarginProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName()));
				
				ChildPropertyHandles[0]->SetValue(NewMargin.Left);
				ChildPropertyHandles[1]->SetValue(NewMargin.Top);
				ChildPropertyHandles[2]->SetValue(NewMargin.Right);
				ChildPropertyHandles[3]->SetValue(NewMargin.Bottom);

				MarginEditableTextBox->SetError( FString() );
			}
		}

		if( bError )
		{
			MarginEditableTextBox->SetError( NSLOCTEXT( "UnrealEd", "InvalidMarginText", "Valid Margin formats are:\nUniform Margin; eg. 0.5\nHorizontal / Vertical Margins; eg. 2, 3\nLeft / Top / Right / Bottom Margins; eg. 0.2, 1, 1.5, 3" ) );
		}
	}
}

FString FMarginStructCustomization::GetMarginTextFromProperties() const
{
	FString MarginText;
	float PropertyValues[ 4 ];
	bool bMultipleValues = false;

	for( int32 PropertyIndex = 0; PropertyIndex < 4 && !bMultipleValues; ++PropertyIndex )
	{
		const FPropertyAccess::Result Result = ChildPropertyHandles[ PropertyIndex ]->GetValue( PropertyValues[ PropertyIndex ] );
		if( Result == FPropertyAccess::MultipleValues )
		{
			bMultipleValues = true;
		}
	}

	if( bMultipleValues )
	{
		MarginText = NSLOCTEXT( "PropertyEditor", "MultipleValues", "Multiple Values" ).ToString();
	}
	else
	{
		if( PropertyValues[ 0 ] == PropertyValues[ 1 ] && PropertyValues[ 1 ] == PropertyValues[ 2 ] && PropertyValues[ 2 ] == PropertyValues[ 3 ] )
		{
			// Uniform margin
			MarginText = FString::SanitizeFloat( PropertyValues[ 0 ] );
		}
		else if( PropertyValues[ 0 ] == PropertyValues[ 2 ] && PropertyValues[ 1 ] == PropertyValues[ 3 ] )
		{
			// Horizontal, Vertical margins
			MarginText = FString::SanitizeFloat( PropertyValues[ 0 ] ) + FString( ", " ) + FString::SanitizeFloat( PropertyValues[ 1 ] );
		}
		else
		{
			// Left, Top, Right, Bottom margins
			MarginText = FString::SanitizeFloat( PropertyValues[ 0 ] ) + FString( ", " ) + FString::SanitizeFloat( PropertyValues[ 1 ] ) + FString( ", " ) +
				FString::SanitizeFloat( PropertyValues[ 2 ] ) + FString( ", " ) + FString::SanitizeFloat( PropertyValues[ 3 ] );
		}
	}

	return MarginText;
}


TOptional<float> FMarginStructCustomization::OnGetValue( int32 PropertyIndex ) const
{
	TSharedRef<IPropertyHandle> ChildHandle = ChildPropertyHandles[ PropertyIndex ];
	float FloatVal = 0.0f;
	if( ChildHandle->GetValue( FloatVal ) == FPropertyAccess::Success )
	{
		return TOptional<float>( FloatVal );
	}

	return TOptional<float>();
}

void FMarginStructCustomization::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	GEditor->BeginTransaction( FText::Format( NSLOCTEXT("FMarginStructCustomization", "SetMarginProperty", "Edit {0}"), StructPropertyHandle->GetPropertyDisplayName() ) );
}

void FMarginStructCustomization::OnEndSliderMovement( float NewValue )
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}

void FMarginStructCustomization::OnValueCommitted( float NewValue, ETextCommit::Type CommitType, int32 PropertyIndex )
{
	TSharedRef<IPropertyHandle> ChildHandle = ChildPropertyHandles[ PropertyIndex ];
	ChildHandle->SetValue( NewValue );
}	

void FMarginStructCustomization::OnValueChanged( float NewValue, int32 PropertyIndex )
{
	if( bIsUsingSlider )
	{
		TSharedRef<IPropertyHandle> ChildHandle = ChildPropertyHandles[ PropertyIndex ];
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange;
		ChildHandle->SetValue( NewValue, Flags );
	}
}

