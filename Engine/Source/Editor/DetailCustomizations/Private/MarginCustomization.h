// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class SEditableTextBox;
class SWidget;
template <typename NumericType> struct INumericTypeInterface;

class FMarginStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FMarginStructCustomization()
		: bIsUsingSlider( false )
	{}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

private:

	/**
	 * Make the margin property widget
	 */
	TSharedRef<SEditableTextBox> MakePropertyWidget();

	/**
	 * Make the widget for the specified property
	 *
	 * @param PropertyIndex	The index of the child property to the make the widget for
	 */
	TSharedRef<SWidget> MakeChildPropertyWidget( int32 PropertyIndex) const;

	/**
	 Get the margin property value as text
	 */
	FText GetMarginText() const;

	/**
	 * Delegate to commit margin text
	 */
	void OnMarginTextCommitted( const FText& InText, ETextCommit::Type InCommitType );

	/**
	 * Update the margin text from the margin property values
	 */
	FString GetMarginTextFromProperties() const;

	/**
	 * Gets the value as a float for the provided property handle
	 *
	 * @param PropertyIndex	The index of the child property to get the value from
	 * @return The value or unset if it could not be accessed
	 */
	virtual TOptional<float> OnGetValue( int32 PropertyIndex ) const;

	/**
	 * Called when the value is committed from the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param CommitType	How the value was committed (unused)
	 * @param PropertyIndex	The index of the child property that the new value is for
	 */
	virtual void OnValueCommitted( float NewValue, ETextCommit::Type CommitType, int32 PropertyIndex );
	
	/**
	 * Called when the value is changed in the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param PropertyIndex	The index of the child property that the new value is for
	 */
	virtual void OnValueChanged( float NewValue, int32 PropertyIndex );
	
	/**
	 * Called when a value starts to be changed by a slider
	 */
	void OnBeginSliderMovement();

	/**
	 * Called when a value stops being changed by a slider
	 */
	void OnEndSliderMovement( float NewValue );
	
	/** Handle to the margin being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Child property handles */
	TArray< TSharedRef<IPropertyHandle> > ChildPropertyHandles;

	/** True if this margin value working in uv space */
	bool bIsMarginUsingUVSpace;

	/** True if a value is being changed by dragging a slider */
	bool bIsUsingSlider;

	/** Margin text editable text box */
	TSharedPtr<SEditableTextBox> MarginEditableTextBox;

	/** Used to evaluate margin strings as mathematical expressions **/
	TSharedPtr< INumericTypeInterface<float> > NumericInterface;
};

