// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialAttributePropertyDetails.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Text.h"
#include "MaterialShared.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionGetMaterialAttributes.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Templates/Tuple.h"
#include "RenderUtils.h"

class SToolTip;

TSharedRef<IDetailCustomization> FMaterialAttributePropertyDetails::MakeInstance()
{
	return MakeShareable(new FMaterialAttributePropertyDetails);
}

void FMaterialAttributePropertyDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Populate combo boxes with material property list
	FMaterialAttributeDefinitionMap::GetAttributeNameToIDList(AttributeNameToIDList);

	static const FString ExcludedFrontMaterialName = FString("FrontMaterial");

	AttributeDisplayNameList.Empty(AttributeNameToIDList.Num());
	for (const TPair<FString, FGuid>& NameGUIDPair : AttributeNameToIDList)
	{
		// We remove unwanted elements from the list of material attributes that can be used in the material layer workflow.
		// This filtering is ran only when constructing the UI element.
		if(NameGUIDPair.Key != ExcludedFrontMaterialName)
		{
			AttributeDisplayNameList.Add(MakeShareable(new FString(NameGUIDPair.Key)));
		}
	}

	// Fetch root property we're dealing with
	TSharedPtr<IPropertyHandle> PropertyGetArray = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionGetMaterialAttributes, AttributeGetTypes));
	TSharedPtr<IPropertyHandle> PropertySetArray = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionSetMaterialAttributes, AttributeSetTypes));
	TSharedPtr<IPropertyHandle> PropertyArray;

	if (PropertyGetArray->IsValidHandle())
	{
		PropertyArray = PropertyGetArray;
	}
	else if (PropertySetArray->IsValidHandle())
	{
		PropertyArray = PropertySetArray;
	}
	
	check(PropertyArray->IsValidHandle());

	// Add builder for children to handle array changes
	TSharedRef<FDetailArrayBuilder> ArrayChildBuilder = MakeShareable(new FDetailArrayBuilder(PropertyArray.ToSharedRef()));
	ArrayChildBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FMaterialAttributePropertyDetails::OnBuildChild));

	IDetailCategoryBuilder& AttributesCategory = DetailLayout.EditCategory("MaterialAttributes", FText::GetEmpty(), ECategoryPriority::Important);
	AttributesCategory.AddCustomBuilder(ArrayChildBuilder);
}

void FMaterialAttributePropertyDetails::OnBuildChild(TSharedRef<IPropertyHandle> ChildHandle, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	// Add an overridden combo box
	IDetailPropertyRow& PropertyArrayRow = ChildrenBuilder.AddProperty(ChildHandle);

	FPropertyComboBoxArgs ComboArgs(
		ChildHandle,
		FOnGetPropertyComboBoxStrings::CreateLambda([this](TArray<TSharedPtr<FString>>& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems) -> void  
		{									
			OutComboBoxStrings = AttributeDisplayNameList;
		}), 
		FOnGetPropertyComboBoxValue::CreateLambda( [this, ChildHandle]() -> FString
		{
			FString AttributeName;
			if (ChildHandle->IsValidHandle())
			{
				// Convert attribute ID string to display name
				FString IDString; FGuid IDValue;
				ChildHandle->GetValueAsFormattedString(IDString);
				FGuid::ParseExact(IDString, EGuidFormats::Digits, IDValue);

				AttributeName = FMaterialAttributeDefinitionMap::GetAttributeName(IDValue);
			} 
			return AttributeName;
		}),
		FOnPropertyComboBoxValueSelected::CreateLambda( [this, ChildHandle] (const FString& Selection)
		{
			if (ChildHandle->IsValidHandle())
			{
				// Convert display name to attribute ID
				for (const auto& NameIDPair : AttributeNameToIDList)
				{
					if (NameIDPair.Key == Selection)
					{
						ChildHandle->SetValueFromFormattedString(NameIDPair.Value.ToString(EGuidFormats::Digits));
						break;
					}
				}
			}
		})
	);
	ComboArgs.ShowSearchForItemCount = 1;


	PropertyArrayRow.CustomWidget()
	.NameContent()
	[
		ChildHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyCustomizationHelpers::MakePropertyComboBox(ComboArgs)
	];
}

