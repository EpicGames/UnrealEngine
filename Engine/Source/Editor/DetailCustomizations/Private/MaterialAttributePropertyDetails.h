// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class FString;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
struct FGuid;

/**
*	Customization for material attribute get/set nodes to handle GUID-Name conversions.
*/
class FMaterialAttributePropertyDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	void OnBuildChild(TSharedRef<IPropertyHandle> ChildHandle, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

private:
	TArray<TPair<FString, FGuid>>	AttributeNameToIDList;
	TArray<TSharedPtr<FString>>		AttributeDisplayNameList;
};
