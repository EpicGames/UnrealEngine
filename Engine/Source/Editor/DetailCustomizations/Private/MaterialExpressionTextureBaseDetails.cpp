// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionTextureBaseDetails.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "Engine/EngineTypes.h"
#include "Engine/Texture.h"
#include "Internationalization/Internationalization.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

class UObject;

#define LOCTEXT_NAMESPACE "MaterialExpressionTextureBaseDetails"

TSharedRef<IDetailCustomization> FMaterialExpressionTextureBaseDetails::MakeInstance()
{
	return MakeShareable(new FMaterialExpressionTextureBaseDetails);
}

FMaterialExpressionTextureBaseDetails::~FMaterialExpressionTextureBaseDetails()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(DelegateHandle);
}

void FMaterialExpressionTextureBaseDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() > 0)
	{
		Expression = CastChecked<UMaterialExpressionTextureBase>(Objects[0]);
	}

	EnumRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("VirtualTextureSamplerMatch", "Sampler type must match VirtualTexture usage")));
	TSharedPtr<IPropertyHandle> SamplerTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureBase, SamplerType));
	SamplerTypeHandle->AddRestriction(EnumRestriction.ToSharedRef());

	// IPropertyHandle::SetOnPropertyValueChanged will catch property changes initiated by the editor
	TSharedPtr<IPropertyHandle> TextureHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureBase, Texture));
	TextureHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMaterialExpressionTextureBaseDetails::OnTextureChanged));

	// FCoreUObjectDelegates::OnObjectPropertyChanged will catch property changes initiated by C++
	// This is required to ensure UI is properly updated when the VT conversion tool runs and updates textures/materials
	DelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FMaterialExpressionTextureBaseDetails::OnPropertyChanged);

	OnTextureChanged();
}

void FMaterialExpressionTextureBaseDetails::OnTextureChanged()
{
	bool bAllowVirtualTexture = true;
	bool bAllowNonVirtualTexture = true;
	if (Expression.IsValid() && Expression->Texture)
	{
		bAllowVirtualTexture = Expression->Texture->VirtualTextureStreaming;
		bAllowNonVirtualTexture = !bAllowVirtualTexture;
	}

	EnumRestriction->RemoveAll();

	const UEnum* MaterialSamplerTypeEnum = StaticEnum<EMaterialSamplerType>();
	for (int SamplerTypeIndex = 0; SamplerTypeIndex < SAMPLERTYPE_MAX; ++SamplerTypeIndex)
	{
		const bool bIsVirtualTexture = IsVirtualSamplerType((EMaterialSamplerType)SamplerTypeIndex);
		if ((bIsVirtualTexture && !bAllowVirtualTexture) || (!bIsVirtualTexture && !bAllowNonVirtualTexture))
		{
			EnumRestriction->AddHiddenValue(MaterialSamplerTypeEnum->GetNameStringByValue(SamplerTypeIndex));
		}
	}
}

void FMaterialExpressionTextureBaseDetails::OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ObjectBeingModified == Expression && PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMaterialExpressionTextureBase, Texture))
	{
		// Update enum list if our texture reference is changed
		OnTextureChanged();
	}
	else if(Expression.IsValid() && Expression->Texture == ObjectBeingModified && PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTexture, VirtualTextureStreaming))
	{
		// Update enum list of currently assigned texture's VT streaming status is changed
		OnTextureChanged();
	}
}

#undef LOCTEXT_NAMESPACE
