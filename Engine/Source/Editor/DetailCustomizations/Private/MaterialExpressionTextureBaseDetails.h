// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FPropertyRestriction;
class IDetailLayoutBuilder;
class UMaterialExpressionTextureBase;
class UObject;
struct FPropertyChangedEvent;

class FMaterialExpressionTextureBaseDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FMaterialExpressionTextureBaseDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

	TSharedPtr<FPropertyRestriction> EnumRestriction;
	TWeakObjectPtr<UMaterialExpressionTextureBase> Expression;
	FDelegateHandle DelegateHandle;

	void OnTextureChanged();
	void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
};

