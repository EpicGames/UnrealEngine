// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialInstanceDynamicDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

TSharedRef<IDetailCustomization> FMaterialInstanceDynamicDetails::MakeInstance()
{
	return MakeShareable(new FMaterialInstanceDynamicDetails);
}

void FMaterialInstanceDynamicDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	DetailLayout.HideCategory(TEXT("PhysicalMaterialMask"));
	DetailLayout.HideCategory(TEXT("Material"));
	DetailLayout.HideCategory(TEXT("Lightmass"));
	DetailLayout.HideCategory(TEXT("Previewing"));
	DetailLayout.HideCategory(TEXT("ImportSettings"));

	IDetailCategoryBuilder& MaterialInstanceCategory = DetailLayout.EditCategory(TEXT("MaterialInstance"));
	// Add/hide other properties
	TSharedRef<IPropertyHandle> PhysMatHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, PhysMaterial), UMaterialInstance::StaticClass());
	PhysMatHandle->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> OverrideSubSurfaceHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, bOverrideSubsurfaceProfile), UMaterialInstance::StaticClass());
	OverrideSubSurfaceHandle->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> OverrideSpecularProfileHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, bOverrideSpecularProfile), UMaterialInstance::StaticClass());
	OverrideSpecularProfileHandle->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> OverrideToonProfileHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, bOverrideToonProfile), UMaterialInstance::StaticClass());
	OverrideToonProfileHandle->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> BasePropertiesHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, BasePropertyOverrides), UMaterialInstance::StaticClass());
	BasePropertiesHandle->MarkHiddenByCustomization();

	// Customize Parent property so we can check for recursively set parents
 	TSharedRef<IPropertyHandle> ParentPropertyHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMaterialInstance, Parent), UMaterialInstance::StaticClass());
	ParentPropertyHandle->MarkHiddenByCustomization();
	IDetailPropertyRow& ParentPropertyRow = MaterialInstanceCategory.AddProperty(ParentPropertyHandle);

	ParentPropertyHandle->MarkResetToDefaultCustomized();

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;

	ParentPropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	const bool bShowChildren = true;
	ParentPropertyRow.CustomWidget(bShowChildren)
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
	.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(ParentPropertyHandle)
			.AllowedClass(UMaterialInterface::StaticClass())
			.ThumbnailPool(DetailLayout.GetThumbnailPool())
			.AllowClear(false)
			.DisplayUseSelected(false)
			.EnableContentPicker(false)
		];

	ValueWidget.Reset();

}
