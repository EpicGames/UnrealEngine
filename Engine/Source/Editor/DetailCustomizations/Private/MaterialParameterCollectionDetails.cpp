// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialParameterCollectionDetails.h"

#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "Materials/MaterialParameterCollection.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MaterialParameterCollectionDetails"

TSharedRef<IDetailCustomization> FMaterialParameterCollectionDetails::MakeInstance()
{
	return MakeShareable(new FMaterialParameterCollectionDetails);
}

FMaterialParameterCollectionDetails::~FMaterialParameterCollectionDetails()
{
	UnbindDelegates();
}

class FMaterialParameterCollectionDetails::FBaseOverridesSelectorConst
{
public:
	FBaseOverridesSelectorConst(const UMaterialParameterCollection* OverrideCollection) :
		Collection(const_cast<UMaterialParameterCollection*>(OverrideCollection))
	{
	}

	template<class FCollectionParameterType, class... FArgs>
	decltype(auto) Contains(const FCollectionParameterType& CollectionParameter, FArgs&&... Args) const
	{
		return SelectBaseOverridesMap(CollectionParameter).Contains(CollectionParameter.Id, Forward<FArgs>(Args)...);
	}

	template<class FCollectionParameterType, class... FArgs>
	decltype(auto) Find(const FCollectionParameterType& CollectionParameter, FArgs&&... Args) const
	{
		return SelectBaseOverridesMap(CollectionParameter).Find(CollectionParameter.Id, Forward<FArgs>(Args)...);
	}

protected:
	TMap<FGuid, float>& SelectBaseOverridesMap(const FCollectionScalarParameter&) const
	{
		return Collection->ScalarParameterBaseOverrides;
	}

	TMap<FGuid, FLinearColor>& SelectBaseOverridesMap(const FCollectionVectorParameter&) const
	{
		return Collection->VectorParameterBaseOverrides;
	}

private:
	UMaterialParameterCollection* Collection;
};

class FMaterialParameterCollectionDetails::FBaseOverridesSelector : public FBaseOverridesSelectorConst
{
public:
	FBaseOverridesSelector(UMaterialParameterCollection* OverrideCollection) :
		FBaseOverridesSelectorConst(OverrideCollection)
	{
	}

	template<class FCollectionParameterType, class... FArgs>
	decltype(auto) Add(const FCollectionParameterType& CollectionParameter, FArgs&&... Args) const
	{
		return SelectBaseOverridesMap(CollectionParameter).Add(CollectionParameter.Id, Forward<FArgs>(Args)...);
	}

	template<class FCollectionParameterType, class... FArgs>
	decltype(auto) FindOrAdd(const FCollectionParameterType& CollectionParameter, FArgs&&... Args) const
	{
		return SelectBaseOverridesMap(CollectionParameter).FindOrAdd(CollectionParameter.Id, Forward<FArgs>(Args)...);
	}

	template<class FCollectionParameterType, class... FArgs>
	decltype(auto) Remove(const FCollectionParameterType& CollectionParameter, FArgs&&... Args) const
	{
		return SelectBaseOverridesMap(CollectionParameter).Remove(CollectionParameter.Id, Forward<FArgs>(Args)...);
	}
};

FMaterialParameterCollectionDetails::FBaseOverridesSelector FMaterialParameterCollectionDetails::GetBaseOverridesMap(UMaterialParameterCollection* OverrideCollection)
{
	return FBaseOverridesSelector(OverrideCollection);
}

FMaterialParameterCollectionDetails::FBaseOverridesSelectorConst FMaterialParameterCollectionDetails::GetBaseOverridesMap(const UMaterialParameterCollection* OverrideCollection)
{
	return FBaseOverridesSelectorConst(OverrideCollection);
}

template<class FCollectionParameterType>
auto FMaterialParameterCollectionDetails::GetParameterValue(const UMaterialParameterCollection* OverrideCollection, const FCollectionParameterType& CollectionParameter, const UMaterialParameterCollection* BaseCollection)
{
	for (; OverrideCollection != BaseCollection; OverrideCollection = OverrideCollection->Base)
	{
		if (auto OverrideValue = GetBaseOverridesMap(OverrideCollection).Find(CollectionParameter))
		{
			return *OverrideValue;
		}
	}

	return CollectionParameter.DefaultValue;
}

FProperty* FMaterialParameterCollectionDetails::GetBaseOverridesMapProperty(const FCollectionScalarParameter&)
{
	return FindFProperty<FProperty>(UMaterialParameterCollection::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialParameterCollection, ScalarParameterBaseOverrides));
}

FProperty* FMaterialParameterCollectionDetails::GetBaseOverridesMapProperty(const FCollectionVectorParameter&)
{
	return FindFProperty<FProperty>(UMaterialParameterCollection::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterialParameterCollection, VectorParameterBaseOverrides));
}

template<class FCollectionParameterType>
void FMaterialParameterCollectionDetails::AddParameter(IDetailGroup& DetailGroup, TSharedPtr<IPropertyHandle> PropertyHandle, UMaterialParameterCollection* Collection, const FCollectionParameterType& CollectionParameter, const UMaterialParameterCollection* BaseCollection)
{
	using FValueType = decltype(CollectionParameter.DefaultValue);

	// Set the property display name to the parameter's name.
	PropertyHandle->SetPropertyDisplayName(FText::FromName(CollectionParameter.ParameterName));

	// Initialize the property value with the current parameter value from the collection instance.
	void* ValueData = nullptr;
	if (PropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success && ValueData)
	{
		FValueType& Value = *static_cast<FValueType*>(ValueData);
		Value = GetParameterValue(Collection, CollectionParameter, BaseCollection);
	}

	// Create a delegate to invoke when the property value changes.
	TDelegate<void(const FPropertyChangedEvent&)> OnPropertyValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateWeakLambda(Collection, [Collection = Collection, CollectionParameter, PropertyHandle](const FPropertyChangedEvent& PropertyChangedEvent)
		{
			// Determine if this is the final setting of the property value.
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
			{
				return;
			}

			// Update the collection's parameter value with the new property value.
			void* ValueData = nullptr;
			if (PropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success && ValueData)
			{
				FProperty* CollectionPropertyToChange = GetBaseOverridesMapProperty(CollectionParameter);
				Collection->PreEditChange(CollectionPropertyToChange);

				FValueType& Value = *static_cast<FValueType*>(ValueData);
				FValueType& ParameterValue = GetBaseOverridesMap(Collection).FindOrAdd(CollectionParameter);
				ParameterValue = Value;

				FPropertyChangedEvent CollectionChangedEvent(CollectionPropertyToChange, EPropertyChangeType::ValueSet);
				Collection->PostEditChangeProperty(CollectionChangedEvent);

			}
		});

	// Invoke the delegate when the property value or one of its child property values changes.
	PropertyHandle->SetOnPropertyValueChangedWithData(OnPropertyValueChangedDelegate);
	PropertyHandle->SetOnChildPropertyValueChangedWithData(OnPropertyValueChangedDelegate);

	// If the collection instance includes the parameter in its parameter map, then the parameter is overridden.
	auto IsOverridden = [Collection = Collection, CollectionParameter]()
		{
			return GetBaseOverridesMap(Collection).Contains(CollectionParameter);
		};

	auto OnOverride = [Collection = Collection, BaseCollection, CollectionParameter, PropertyHandle](bool bOverride)
		{
			void* ValueData = nullptr;
			if (PropertyHandle->GetValueData(ValueData) == FPropertyAccess::Success && ValueData)
			{
				// Create a new transaction for undo/redo support - normally handled by IPropertyHandle
				FScopedTransaction EditConditionChangedTransaction(NSLOCTEXT("PropertyEditor", "UpdatedEditConditionFmt", "Edit Condition Changed"));

				FProperty* CollectionPropertyToChange = GetBaseOverridesMapProperty(CollectionParameter);
				Collection->PreEditChange(CollectionPropertyToChange);

				FValueType& Value = *static_cast<FValueType*>(ValueData);

				// Remove the parameter from the collection instance.
				auto&& BaseOverridesMap = GetBaseOverridesMap(Collection);
				BaseOverridesMap.Remove(CollectionParameter);

				// Update the property value with the parameter value from the collection instance's parent.
				Value = GetParameterValue(Collection, CollectionParameter, BaseCollection);

				if (bOverride)
				{
					// Add the parameter to the collection instance, with the value specified by its parent.
					BaseOverridesMap.Add(CollectionParameter, Value);
				}

				FPropertyChangedEvent CollectionChangedEvent(CollectionPropertyToChange, (bOverride ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove));
				Collection->PostEditChangeProperty(CollectionChangedEvent);

				// Update all viewports after we modify the collection - normally handled by IPropertyHandle
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
			}
		};

	// Set the edit condition to use the override lambdas.
	DetailGroup.AddPropertyRow(PropertyHandle.ToSharedRef()).
		EditCondition(
			TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateWeakLambda(Collection, IsOverridden)),
			FOnBooleanValueChanged::CreateWeakLambda(Collection, OnOverride));
}

template<class FCollectionParameterType>
void FMaterialParameterCollectionDetails::AddParameters(IDetailCategoryBuilder& DetailCategory, FName PropertyName, UMaterialParameterCollection* Collection, const TArray<FCollectionParameterType>& CollectionParameters, UMaterialParameterCollection* BaseCollection)
{
	IDetailGroup& DetailGroup = DetailCategory.AddGroup(PropertyName, FText::FromName(PropertyName), false, false);

	if (CollectionParameters.IsEmpty())
	{
		return;
	}

	if (UScriptStruct* CollectionParameterStruct = FCollectionParameterType::StaticStruct())
	{
		for (auto& CollectionParameter : CollectionParameters)
		{
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(CollectionParameterStruct);

			if (IDetailPropertyRow* DetailPropertyRow = DetailCategory.AddExternalStructure(StructOnScope))
			{
				DetailPropertyRow->Visibility(EVisibility::Collapsed);

				if (TSharedPtr<IPropertyHandle> ParentHandle = DetailPropertyRow->GetPropertyHandle())
				{
					if (TSharedPtr<IPropertyHandle> PropertyHandle = ParentHandle->GetChildHandle("DefaultValue"))
					{
						AddParameter(DetailGroup, PropertyHandle, Collection, CollectionParameter, BaseCollection);
					}
				}
			}
		}
	}
}

void FMaterialParameterCollectionDetails::CustomizeCollectionDetails(UMaterialParameterCollection* Collection)
{
	FName ScalarParametersName = "ScalarParameters";
	FName VectorParametersName = "VectorParameters";

	TSharedRef<IPropertyHandle> ScalarParametersHandle = DetailLayout->GetProperty(ScalarParametersName, UMaterialParameterCollection::StaticClass());
	TSharedRef<IPropertyHandle> VectorParametersHandle = DetailLayout->GetProperty(VectorParametersName, UMaterialParameterCollection::StaticClass());
	TSharedRef<IPropertyHandle> BaseHandle = DetailLayout->GetProperty("Base", UMaterialParameterCollection::StaticClass());

	DetailLayout->HideProperty(ScalarParametersHandle);
	DetailLayout->HideProperty(VectorParametersHandle);
	DetailLayout->HideProperty(BaseHandle);

	IDetailCategoryBuilder& MaterialCategory = DetailLayout->EditCategory("Material");
	MaterialCategory.AddProperty(ScalarParametersHandle);
	MaterialCategory.AddProperty(VectorParametersHandle);
	MaterialCategory.AddProperty(BaseHandle);

	// Create a delegate to invoke when the Base changes.
	TDelegate<void(const FPropertyChangedEvent&)> OnPropertyValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateWeakLambda(Collection,
		[Collection, DetailLayout = DetailLayout, BaseHandle, BaseCollectionWeak = TWeakObjectPtr<UMaterialParameterCollection>(Collection->Base)](const FPropertyChangedEvent& PropertyChangedEvent)
		{
			// Determine if this is the final setting of the property value.
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::ValueSet)
			{
				return;
			}

			// Update all properties to account for the new Base.
			DetailLayout->ForceRefreshDetails();
		});

	BaseHandle->SetOnPropertyValueChangedWithData(OnPropertyValueChangedDelegate);
	BaseHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateLambda([DetailLayout = DetailLayout]()
		{
			DetailLayout->ForceRefreshDetails();
		}));

	for (UMaterialParameterCollection* BaseCollection = Collection->Base; BaseCollection; BaseCollection = BaseCollection->Base)
	{
		IDetailCategoryBuilder& BaseCategory = DetailLayout->EditCategory(BaseCollection->GetFName());
		AddParameters(BaseCategory, ScalarParametersName, Collection, BaseCollection->ScalarParameters, BaseCollection);
		AddParameters(BaseCategory, VectorParametersName, Collection, BaseCollection->VectorParameters, BaseCollection);
	}
}

void FMaterialParameterCollectionDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	UnbindDelegates();

	DetailLayout = &InDetailLayout;

	// Bind delegates for detecting undo/redo and property changes in other objects.
	FEditorDelegates::PostUndoRedo.AddRaw(DetailLayout, &IDetailLayoutBuilder::ForceRefreshDetails);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FMaterialParameterCollectionDetails::OnObjectPropertyChanged);

	// Customize the details of each collection being customized.
	TArray<TWeakObjectPtr<UMaterialParameterCollection>> Collections = DetailLayout->GetObjectsOfTypeBeingCustomized<UMaterialParameterCollection>();
	for (TWeakObjectPtr<UMaterialParameterCollection>& CollectionWeak : Collections)
	{
		if (UMaterialParameterCollection* Collection = CollectionWeak.Get())
		{
			CustomizeCollectionDetails(Collection);
		}
	}
}

void FMaterialParameterCollectionDetails::OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!DetailLayout || !Object->IsA<UMaterialParameterCollection>())
	{
		return;
	}

	// Refresh the layout details if the modified object is the base of any of the collections being customized.
	TArray<TWeakObjectPtr<UMaterialParameterCollection>> Collections = DetailLayout->GetObjectsOfTypeBeingCustomized<UMaterialParameterCollection>();
	for (TWeakObjectPtr<UMaterialParameterCollection>& CollectionWeak : Collections)
	{
		if (UMaterialParameterCollection* Collection = CollectionWeak.Get())
		{
			for (UMaterialParameterCollection* BaseCollection = Collection->Base; BaseCollection; BaseCollection = BaseCollection->Base)
			{
				if (BaseCollection == Object)
				{
					DetailLayout->ForceRefreshDetails();
					return;
				}
			}
		}
	}
}

void FMaterialParameterCollectionDetails::UnbindDelegates()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	if (DetailLayout)
	{
		FEditorDelegates::PostUndoRedo.RemoveAll(DetailLayout);
	}
}

#undef LOCTEXT_NAMESPACE
