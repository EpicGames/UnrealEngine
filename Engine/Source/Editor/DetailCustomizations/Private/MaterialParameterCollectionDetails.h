// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FProperty;
class IDetailGroup;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UMaterialParameterCollection;

struct FCollectionScalarParameter;
struct FCollectionVectorParameter;
struct FGuid;
struct FLinearColor;
struct FPropertyChangedEvent;

//-----------------------------------------------------------------------------
//   FMaterialParameterCollectionDetails
//-----------------------------------------------------------------------------

class FMaterialParameterCollectionDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FMaterialParameterCollectionDetails() override;

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	FMaterialParameterCollectionDetails() = default;

	FMaterialParameterCollectionDetails(const FMaterialParameterCollectionDetails&) = delete;
	FMaterialParameterCollectionDetails& operator =(const FMaterialParameterCollectionDetails&) = delete;

	class FBaseOverridesSelector;
	class FBaseOverridesSelectorConst;

	static FBaseOverridesSelector GetBaseOverridesMap(UMaterialParameterCollection* OverrideCollection);
	static FBaseOverridesSelectorConst GetBaseOverridesMap(const UMaterialParameterCollection* OverrideCollection);

	static FProperty* GetBaseOverridesMapProperty(const FCollectionScalarParameter&);
	static FProperty* GetBaseOverridesMapProperty(const FCollectionVectorParameter&);

	template<class FCollectionParameterType>
	static auto GetParameterValue(const UMaterialParameterCollection* Collection, const FCollectionParameterType& CollectionParameter, const UMaterialParameterCollection* BaseCollection = nullptr);

	template<class FCollectionParameterType>
	void AddParameter(IDetailGroup& DetailGroup, TSharedPtr<IPropertyHandle> PropertyHandle, UMaterialParameterCollection* Collection, const FCollectionParameterType& CollectionParameter, const UMaterialParameterCollection* BaseCollection);

	template<class FCollectionParameterType>
	void AddParameters(IDetailCategoryBuilder& DetailCategory, FName PropertyName, UMaterialParameterCollection* Collection, const TArray<FCollectionParameterType>& CollectionParameters, UMaterialParameterCollection* BaseCollection);

	void CustomizeCollectionDetails(UMaterialParameterCollection* Collection);

	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	void UnbindDelegates();

	IDetailLayoutBuilder* DetailLayout = nullptr;
};
