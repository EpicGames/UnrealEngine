// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialProxySettingsCustomizations.h"

#include "Algo/AnyOf.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Engine/MaterialMerging.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IMeshReductionInterfaces.h" // IMeshMerging
#include "IMeshReductionManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "Math/IntPoint.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "RHI.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealType.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#define LOCTEXT_NAMESPACE "MaterialProxySettingsCustomizations"


TSharedRef<IPropertyTypeCustomization> FMaterialProxySettingsCustomizations::MakeInstance()
{
	return MakeShareable(new FMaterialProxySettingsCustomizations);
}

void FMaterialProxySettingsCustomizations::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.
		NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
}

bool FMaterialProxySettingsCustomizations::UseNativeProxyLODTool() const
{
	IMeshMerging* MergeModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetMeshMergingInterface();
	return MergeModule && MergeModule->GetName().Equals("ProxyLODMeshMerging");
}

void FMaterialProxySettingsCustomizations::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren( NumChildren );
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle( ChildIndex ).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Determine if we are using our native module  If so, we will suppress some of the options used by the current thirdparty tool (simplygon).
	// NB: this only needs to be called once (static) since the tool can only change on editor restart
	static bool bUseNativeTool = UseNativeProxyLODTool();


	// Retrieve special case properties
	TextureSizingTypeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, TextureSizingType));
	TextureSizeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, TextureSize));
	MeshMinDrawDistanceHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, MeshMinDrawDistance));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, DiffuseTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, NormalTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, MetallicTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, RoughnessTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, SpecularTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, EmissiveTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, OpacityTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, OpacityMaskTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, AmbientOcclusionTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, TangentTextureSize)));
	PropertyTextureSizeHandles.Add(PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, AnisotropyTextureSize)));

	if (PropertyHandles.Contains(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, MaterialMergeType)))
	{
		MergeTypeHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, MaterialMergeType));
	}

	GutterSpaceHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, GutterSpace));

	auto Parent = StructPropertyHandle->GetParentHandle();

	for( auto Iter(PropertyHandles.CreateIterator()); Iter; ++Iter  )
	{
		// Handle special property cases (done inside the loop to maintain order according to the struct
		if (PropertyTextureSizeHandles.Contains(Iter.Value()))
		{
			IDetailPropertyRow& SizeRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
			SizeRow.Visibility(TAttribute<EVisibility>(this, &FMaterialProxySettingsCustomizations::AreManualOverrideTextureSizesEnabled));
			AddTextureSizeClamping(Iter.Value());
		}
		else if (Iter.Value() == TextureSizeHandle)
		{
			IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FMaterialProxySettingsCustomizations::IsTextureSizeEnabled));
			AddTextureSizeClamping(Iter.Value());
		}
		else if (Iter.Value() == GutterSpaceHandle)
		{
			IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FMaterialProxySettingsCustomizations::IsSimplygonMaterialMergingVisible));
		}
		else if (Iter.Value() == TextureSizingTypeHandle)
		{
			// Remove the simplygon specific option.
			if (bUseNativeTool)
			{
				TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("NoSupport", "Unable to support this option in Merge Actor")));
				const UEnum* const TextureSizingTypeEnum = StaticEnum<ETextureSizingType>();
				EnumRestriction->AddHiddenValue(TextureSizingTypeEnum->GetNameStringByValue((uint8)ETextureSizingType::TextureSizingType_UseSimplygonAutomaticSizing));
				TextureSizingTypeHandle->AddRestriction(EnumRestriction.ToSharedRef());
			}

			ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
		}
		else if (Iter.Value() == MeshMinDrawDistanceHandle)
		{
			IDetailPropertyRow& SettingsRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
			SettingsRow.Visibility(TAttribute<EVisibility>(this, &FMaterialProxySettingsCustomizations::IsMeshMinDrawDistanceVisible));
		}
		else if (Iter.Value() == MergeTypeHandle)
		{
			// Do not show the merge type property
		}
		else
		{
			ChildBuilder.AddProperty(Iter.Value().ToSharedRef());
		}
	}
}

void FMaterialProxySettingsCustomizations::AddTextureSizeClamping(TSharedPtr<IPropertyHandle> TextureSizeProperty)
{
	TSharedPtr<IPropertyHandle> PropertyX = TextureSizeProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, X));
	TSharedPtr<IPropertyHandle> PropertyY = TextureSizeProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIntPoint, Y));
	// NB: the current global value 16384 GetMax2DTextureDimension() will cause int32 overflow for 32bit color formats
	//     with 16 bytes per pixel.
	//     See implimentations of ImageUtils.cpp :: GetRawData()  and ImageCore.cpp :: CopyTo()
	//     11585 = Floor( Sqrt ( Max_int32 / 16)  )
	const int32 TmpMaxSize = FMath::FloorToInt(FMath::Sqrt(static_cast<float>(MAX_int32) / 16));
	const int32 MaxProxyTextureResolution = FMath::Min(TmpMaxSize, (int32)GetMax2DTextureDimension());

	const FString MaxTextureResolutionString = FString::FromInt(MaxProxyTextureResolution);
	TextureSizeProperty->GetProperty()->SetMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	TextureSizeProperty->GetProperty()->SetMetaData(TEXT("UIMax"), *MaxTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("UIMax"), *MaxTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("ClampMax"), *MaxTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("UIMax"), *MaxTextureResolutionString);

	const FString MinTextureResolutionString("1");
	PropertyX->SetInstanceMetaData(TEXT("ClampMin"), *MinTextureResolutionString);
	PropertyX->SetInstanceMetaData(TEXT("UIMin"), *MinTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("ClampMin"), *MinTextureResolutionString);
	PropertyY->SetInstanceMetaData(TEXT("UIMin"), *MinTextureResolutionString);
}

EVisibility FMaterialProxySettingsCustomizations::AreManualOverrideTextureSizesEnabled() const
{
	uint8 TypeValue;
	TextureSizingTypeHandle->GetValue(TypeValue);

	if (TypeValue == TextureSizingType_UseManualOverrideTextureSize)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility FMaterialProxySettingsCustomizations::IsTextureSizeEnabled() const
{
	uint8 TypeValue;
	TextureSizingTypeHandle->GetValue(TypeValue);

	if (TypeValue == TextureSizingType_UseSingleTextureSize || TypeValue == TextureSizingType_UseAutomaticBiasedSizes)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility FMaterialProxySettingsCustomizations::IsSimplygonMaterialMergingVisible() const
{
	uint8 MergeType = EMaterialMergeType::MaterialMergeType_Default;
	if (MergeTypeHandle.IsValid())
	{
		MergeTypeHandle->GetValue(MergeType);
	}

	return ( MergeType == EMaterialMergeType::MaterialMergeType_Simplygon ) ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FMaterialProxySettingsCustomizations::IsMeshMinDrawDistanceVisible() const
{
	uint8 TypeValue;
	TextureSizingTypeHandle->GetValue(TypeValue);
	if (TypeValue != TextureSizingType_AutomaticFromMeshDrawDistance)
	{
		return EVisibility::Hidden;
	}

	TArray<UObject*> OutersList;
	MeshMinDrawDistanceHandle->GetOuterObjects(OutersList);
	const bool bEditingHLODLayer = Algo::AnyOf(OutersList, [](UObject* Outer) { return Outer->IsInA(UHLODLayer::StaticClass()); });

	return bEditingHLODLayer ? EVisibility::Hidden : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
