// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IPropertyHandle;

class FMaterialProxySettingsCustomizations : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
protected:
	void AddTextureSizeClamping(TSharedPtr<IPropertyHandle> TextureSizeProperty);
	bool UseNativeProxyLODTool() const;

protected:
	EVisibility AreManualOverrideTextureSizesEnabled() const;
	EVisibility IsTextureSizeEnabled() const;

	EVisibility IsSimplygonMaterialMergingVisible() const;

	EVisibility IsMeshMinDrawDistanceVisible() const;
	
	TSharedPtr< IPropertyHandle > TextureSizingTypeHandle;

	TSharedPtr< IPropertyHandle > TextureSizeHandle;
	TArray<TSharedPtr<IPropertyHandle>> PropertyTextureSizeHandles;

	TSharedPtr< IPropertyHandle > MergeTypeHandle;
	TSharedPtr< IPropertyHandle > GutterSpaceHandle;

	TSharedPtr< IPropertyHandle > MeshMinDrawDistanceHandle;
};
