// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialShadingModelCustomization.h"

#include "Containers/Array.h"
#include "DetailWidgetRow.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#include "Materials/Material.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ReflectedTypeAccessors.h"

TSharedRef<IPropertyTypeCustomization> FMaterialShadingModelCustomization::MakeInstance() 
{
	return MakeShareable( new FMaterialShadingModelCustomization );
}

void FMaterialShadingModelCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Hide special value by default
	bool bShouldHideFromMaterialExpression = true;
	
	// This piece of code enables the use of the Shading Model material output pin through selecting "From Material Expression" in the Shading Model drop down menu.
	if (PropertyHandle->GetNumOuterObjects() == 1)
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		
		// Don't hide MSM_FromMaterialExpression on a UMaterial or on UMaterialEditorInstanceConstant
		if (OuterObjects[0]->IsA<UMaterial>() || OuterObjects[0]->IsA<UMaterialEditorInstanceConstant>())
		{
			bShouldHideFromMaterialExpression = false;
		}
	}
	
	// Add restrictin to property if asked for
	if (bShouldHideFromMaterialExpression)
	{
		TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(NSLOCTEXT("MaterialShadingModel", "FromMaterialExpression", "FromMaterialExpression is only available on UMaterial")));
		const UEnum* const MaterialShadingModelEnum = StaticEnum<EMaterialShadingModel>();		
		EnumRestriction->AddHiddenValue(MaterialShadingModelEnum->GetNameStringByValue((int64)EMaterialShadingModel::MSM_FromMaterialExpression));
		PropertyHandle->AddRestriction(EnumRestriction.ToSharedRef());
	}

	// Implement the copmbobox for the enum
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(150.0f) // To fit the longer names
	.MaxDesiredWidth(0.0f) // Don't care
	[
		PropertyCustomizationHelpers::MakePropertyComboBox(PropertyHandle)
	];
}
