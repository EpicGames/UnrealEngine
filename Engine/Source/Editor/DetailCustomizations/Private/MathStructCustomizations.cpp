// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MathStructCustomizations.h"

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FrameNumber.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/EnumProperty.h"
#include "UObject/NoExportTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

class FFieldClass;
class SWidget;
struct FSlateBrush;
template <typename NumericType> class SSpinBox;


#define LOCTEXT_NAMESPACE "FMathStructCustomization"


TSharedRef<IPropertyTypeCustomization> FMathStructCustomization::MakeInstance() 
{
	return MakeShareable(new FMathStructCustomization);
}


void FMathStructCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	SortedChildHandles.Empty();
	GetSortedChildren(StructPropertyHandle, SortedChildHandles);
	MakeHeaderRow(StructPropertyHandle, HeaderRow);
}


void FMathStructCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Grab children if CustomizeHeader was never run (e.g. vector mask params in the material editor)
	if (SortedChildHandles.Num() == 0)
	{
		GetSortedChildren(StructPropertyHandle, SortedChildHandles);
	}

	for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

		// Add the individual properties as children as well so the vector can be expanded for more room
		StructBuilder.AddProperty(ChildHandle);
	}
}


namespace MathStructCustomization {
	double ScaleStructComponent(double Val, double Scale)
	{
		return Val * Scale;
	}

	template<typename NumericType>
	void NormalizePropertyVector(TWeakPtr<IPropertyHandle> PropertyHandle)
	{
		double SquareSum = 0;
		TSharedPtr<IPropertyHandle> Property = PropertyHandle.Pin();

		uint32 NumChildren;
		Property->GetNumChildren(NumChildren);

		// Loop through each child object and build a square sum
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> Child = Property->GetChildHandle(ChildIndex);
			FProperty* ChildProperty = Child->GetProperty();
			NumericType Val;
			Child->GetValue(Val);
			SquareSum += Val * Val;
		}
		if (SquareSum > UE_SMALL_NUMBER * UE_SMALL_NUMBER)
		{
			// Calculate the scale each object will need to be multiplied by to achieve a unit vector
			double Scale = FMath::InvSqrt(SquareSum);

			// Loop through each object and scale based on the normalized ratio for each object individually
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedPtr<IPropertyHandle> Child = Property->GetChildHandle(ChildIndex);
				FProperty* ChildProperty = Child->GetProperty();
				NumericType Val;
				Child->GetValue(Val);
				NumericType Result = (NumericType)ScaleStructComponent(Val, Scale);
				Child->SetValue(Result);
			}
		}
	}

	bool IsFloatVector(TSharedRef<class IPropertyHandle>& PropertyHandle)
	{
		// Look at the first child element to see if it's something we can normalize
		TSharedPtr<IPropertyHandle> Child = PropertyHandle->GetChildHandle(0);
		if (Child)
		{
			FNumericProperty* ChildProperty = CastField<FNumericProperty>(Child->GetProperty());

			return ChildProperty && ChildProperty->IsFloatingPoint();
		}
		return false;
	}
}

void FMathStructCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
{
	TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;

	TSharedPtr<SHorizontalBox> HorizontalBox;

	Row.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	// Make enough space for each child handle
	.MinDesiredWidth(125.0f * static_cast<float>(SortedChildHandles.Num()))
	.MaxDesiredWidth(125.0f * static_cast<float>(SortedChildHandles.Num()))
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
		.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
	];

	for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

		// Propagate metadata to child properties so that it's reflected in the nested, individual spin boxes
		ChildHandle->SetInstanceMetaData(TEXT("UIMin"), StructPropertyHandle->GetMetaData(TEXT("UIMin")));
		ChildHandle->SetInstanceMetaData(TEXT("UIMax"), StructPropertyHandle->GetMetaData(TEXT("UIMax")));
		ChildHandle->SetInstanceMetaData(TEXT("SliderExponent"), StructPropertyHandle->GetMetaData(TEXT("SliderExponent")));
		ChildHandle->SetInstanceMetaData(TEXT("Delta"), StructPropertyHandle->GetMetaData(TEXT("Delta")));
		ChildHandle->SetInstanceMetaData(TEXT("LinearDeltaSensitivity"), StructPropertyHandle->GetMetaData(TEXT("LinearDeltaSensitivity")));
		ChildHandle->SetInstanceMetaData(TEXT("ShiftMultiplier"), StructPropertyHandle->GetMetaData(TEXT("ShiftMultiplier")));
		ChildHandle->SetInstanceMetaData(TEXT("CtrlMultiplier"), StructPropertyHandle->GetMetaData(TEXT("CtrlMultiplier")));
		ChildHandle->SetInstanceMetaData(TEXT("SupportDynamicSliderMaxValue"), StructPropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMaxValue")));
		ChildHandle->SetInstanceMetaData(TEXT("SupportDynamicSliderMinValue"), StructPropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMinValue")));
		ChildHandle->SetInstanceMetaData(TEXT("ClampMin"), StructPropertyHandle->GetMetaData(TEXT("ClampMin")));
		ChildHandle->SetInstanceMetaData(TEXT("ClampMax"), StructPropertyHandle->GetMetaData(TEXT("ClampMax")));

		// Handle units directly since we can't set metadata on core object types
		if (FStructProperty* StructProp = CastField<FStructProperty>(StructPropertyHandle->GetProperty()))
		{
			if (StructProp->Struct == TBaseStructure<FRotator>::Get())
			{
				const static int32 EUnitNamespaceSize = FCString::Strlen(TEXT("EUnit::"));
				ChildHandle->SetInstanceMetaData(TEXT("Units"), UEnum::GetValueAsString(EUnit::Degrees).RightChop(EUnitNamespaceSize));
			}
		}

		const bool bLastChild = SortedChildHandles.Num()-1 == ChildIndex;
		// Make a widget for each property.  The vector component properties  will be displayed in the header

		TSharedRef<SWidget> NumericEntryBox = MakeChildWidget(StructPropertyHandle, ChildHandle);
		NumericEntryBoxWidgetList.Add(NumericEntryBox);

		HorizontalBox->AddSlot()
		.Padding(FMargin(0.0f, 2.0f, bLastChild ? 0.0f : 3.0f, 2.0f))
		[
			NumericEntryBox
		];
	}

	if (StructPropertyHandle->HasMetaData("AllowPreserveRatio"))
	{
		if (!GConfig->GetBool(TEXT("SelectionDetails"), *(StructPropertyHandle->GetProperty()->GetName() + TEXT("_PreserveScaleRatio")), bPreserveScaleRatio, GEditorPerProjectIni))
		{
			bPreserveScaleRatio = true;
		}

		HorizontalBox->AddSlot()
		.AutoWidth()
		.MaxWidth(18.0f)
		.VAlign(VAlign_Center)
		[
			// Add a checkbox to toggle between preserving the ratio of x,y,z components of scale when a value is entered
			SNew(SCheckBox)
			.IsChecked(this, &FMathStructCustomization::IsPreserveScaleRatioChecked)
			.OnCheckStateChanged(this, &FMathStructCustomization::OnPreserveScaleRatioToggled, StructWeakHandlePtr)
			.Style(FAppStyle::Get(), "TransparentCheckBox")
			.ToolTipText(LOCTEXT("PreserveScaleToolTip", "When locked, all axis values scale together so the object maintains its proportions in all directions."))
			[
				SNew(SImage)
				.Image(this, &FMathStructCustomization::GetPreserveScaleRatioImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	if (StructPropertyHandle->HasMetaData("ShowNormalize") && MathStructCustomization::IsFloatVector(StructPropertyHandle))
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.MaxWidth(18.0f)
			.VAlign(VAlign_Center)
			[
				// Add a button to scale the vector uniformly to achieve a unit vector
				SNew(SButton)
					.OnClicked(this, &FMathStructCustomization::OnNormalizeClicked, StructWeakHandlePtr)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ToolTipText(LOCTEXT("NormalizeToolTip", "When clicked, if the vector is large enough, it scales the vector uniformly to achieve a unit vector (vector with a length of 1)"))
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::GetBrush(TEXT("Icons.Normalize")))	
					]
			];
	}
}

const FSlateBrush* FMathStructCustomization::GetPreserveScaleRatioImage() const
{
	return bPreserveScaleRatio ? FAppStyle::GetBrush(TEXT("Icons.Lock")) : FAppStyle::GetBrush(TEXT("Icons.Unlock"));
}

ECheckBoxState FMathStructCustomization::IsPreserveScaleRatioChecked() const
{
	return bPreserveScaleRatio ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMathStructCustomization::OnPreserveScaleRatioToggled(ECheckBoxState NewState, TWeakPtr<IPropertyHandle> PropertyHandle)
{
	bPreserveScaleRatio = (NewState == ECheckBoxState::Checked) ? true : false;

	if (PropertyHandle.IsValid() && PropertyHandle.Pin()->GetProperty())
	{
		FString SettingKey = (PropertyHandle.Pin()->GetProperty()->GetName() + TEXT("_PreserveScaleRatio"));
		GConfig->SetBool(TEXT("SelectionDetails"), *SettingKey, bPreserveScaleRatio, GEditorPerProjectIni);
	}
}

FReply FMathStructCustomization::OnNormalizeClicked(TWeakPtr<IPropertyHandle> PropertyHandle)
{
	if (PropertyHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> Property = PropertyHandle.Pin();
		TSharedRef<IPropertyHandle> PropertyRef = Property.ToSharedRef();

		if (MathStructCustomization::IsFloatVector(PropertyRef))
		{
			MathStructCustomization::NormalizePropertyVector<double>(Property);
		}
		else
		{
			ensureMsgf(false, TEXT("Unsupported type to Normalize"));
		}
	}
	return FReply::Handled();
}

void FMathStructCustomization::GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		OutChildren.Add(StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
	}
}

// Deprecated overload, to be removed.
template<typename NumericType>
void FMathStructCustomization::ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, TOptional<NumericType>& MinValue,
		TOptional<NumericType>& MaxValue, TOptional<NumericType>& SliderMinValue, TOptional<NumericType>& SliderMaxValue,
		NumericType& SliderExponent, NumericType& Delta, int32& ShiftMouseMovePixelPerDelta,
		bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue)
{
	FNumericMetadata<NumericType> Metadata;
	ExtractNumericMetadata(PropertyHandle, Metadata);

	MinValue = Metadata.MinValue;
	MaxValue = Metadata.MaxValue;
	SliderMinValue = Metadata.SliderMinValue;
	SliderMaxValue = Metadata.SliderMaxValue;
	SliderExponent = Metadata.SliderExponent;
	Delta = Metadata.Delta;
	bSupportDynamicSliderMaxValue = Metadata.bSupportDynamicSliderMaxValue;
	bSupportDynamicSliderMinValue = Metadata.bSupportDynamicSliderMinValue;
}
// Explicitly instantiate the deprecated overload for the four types that we had implicit instantiations
// for at time of deprecation (float, double, int32, uint8), since we no longer implicitly instantiate
// it in this file due to using the other overload.
template void FMathStructCustomization::ExtractNumericMetadata<float>(TSharedRef<IPropertyHandle>& PropertyHandle,
	TOptional<float>& MinValue, TOptional<float>& MaxValue, TOptional<float>& SliderMinValue,
	TOptional<float>& SliderMaxValue, float& SliderExponent, float& Delta,
	int32& ShiftMouseMovePixelPerDelta, bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue);
template void FMathStructCustomization::ExtractNumericMetadata<double>(TSharedRef<IPropertyHandle>& PropertyHandle,
	TOptional<double>& MinValue, TOptional<double>& MaxValue, TOptional<double>& SliderMinValue,
	TOptional<double>& SliderMaxValue, double& SliderExponent, double& Delta,
	int32& ShiftMouseMovePixelPerDelta, bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue);
template void FMathStructCustomization::ExtractNumericMetadata<int32>(TSharedRef<IPropertyHandle>& PropertyHandle,
	TOptional<int32>& MinValue, TOptional<int32>& MaxValue, TOptional<int32>& SliderMinValue,
	TOptional<int32>& SliderMaxValue, int32& SliderExponent, int32& Delta,
	int32& ShiftMouseMovePixelPerDelta, bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue);
template void FMathStructCustomization::ExtractNumericMetadata<uint8>(TSharedRef<IPropertyHandle>& PropertyHandle,
	TOptional<uint8>& MinValue, TOptional<uint8>& MaxValue, TOptional<uint8>& SliderMinValue,
	TOptional<uint8>& SliderMaxValue, uint8& SliderExponent, uint8& Delta,
	int32& ShiftMouseMovePixelPerDelta, bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue);


template<typename NumericType>
void FMathStructCustomization::ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, FNumericMetadata<NumericType>& MetadataOut)
{
	const FString& MetaUIMinString = PropertyHandle->GetMetaData(TEXT("UIMin"));
	const FString& MetaUIMaxString = PropertyHandle->GetMetaData(TEXT("UIMax"));
	const FString& SliderExponentString = PropertyHandle->GetMetaData(TEXT("SliderExponent"));
	const FString& DeltaString = PropertyHandle->GetMetaData(TEXT("Delta"));
	const FString& LinearDeltaSensitivityString = PropertyHandle->GetMetaData(TEXT("LinearDeltaSensitivity"));
	const FString& ShiftMultiplierString = PropertyHandle->GetMetaData(TEXT("ShiftMultiplier"));
	const FString& CtrlMultiplierString = PropertyHandle->GetMetaData(TEXT("CtrlMultiplier"));
	const FString& SupportDynamicSliderMaxValueString = PropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
	const FString& SupportDynamicSliderMinValueString = PropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
	const FString& ClampMinString = PropertyHandle->GetMetaData(TEXT("ClampMin"));
	const FString& ClampMaxString = PropertyHandle->GetMetaData(TEXT("ClampMax"));

	// If no UIMin/Max was specified then use the clamp string
	const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
	const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;
	bool bAllowSpin = !PropertyHandle->GetBoolMetaData(TEXT("NoSpinBox"));

	NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
	NumericType ClampMax = TNumericLimits<NumericType>::Max();

	if (!ClampMinString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
	}

	if (!ClampMaxString.IsEmpty())
	{
		TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
	}

	NumericType UIMin = TNumericLimits<NumericType>::Lowest();
	NumericType UIMax = TNumericLimits<NumericType>::Max();
	TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
	TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);

	MetadataOut.SliderExponent = NumericType(1);

	if (SliderExponentString.Len())
	{
		TTypeFromString<NumericType>::FromString(MetadataOut.SliderExponent, *SliderExponentString);
	}

	MetadataOut.Delta = NumericType(0);

	if (DeltaString.Len())
	{
		TTypeFromString<NumericType>::FromString(MetadataOut.Delta, *DeltaString);
	}

	MetadataOut.LinearDeltaSensitivity = 0;
	if (LinearDeltaSensitivityString.Len())
	{
		TTypeFromString<int32>::FromString(MetadataOut.LinearDeltaSensitivity, *LinearDeltaSensitivityString);
	}
	// LinearDeltaSensitivity only works in SSpinBox if delta is provided, so add it in if it wasn't.
	MetadataOut.Delta = (MetadataOut.LinearDeltaSensitivity != 0 && MetadataOut.Delta == NumericType(0)) ? NumericType(1) : MetadataOut.Delta;

	MetadataOut.ShiftMultiplier = 10.f;
	if (ShiftMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(MetadataOut.ShiftMultiplier, *ShiftMultiplierString);
	}

	MetadataOut.CtrlMultiplier = 0.1f;
	if (CtrlMultiplierString.Len())
	{
		TTypeFromString<float>::FromString(MetadataOut.CtrlMultiplier, *CtrlMultiplierString);
	}

	if (ClampMin >= ClampMax && (ClampMinString.Len() || ClampMaxString.Len()))
	{
		//UE_LOGF(LogPropertyNode, Warning, "Clamp Min (%ls) >= Clamp Max (%ls) for Ranged Numeric", *ClampMinString, *ClampMaxString);
	}

	const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
	const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

	MetadataOut.MinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
	MetadataOut.MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
	MetadataOut.SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
	MetadataOut.SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

	if (ActualUIMin >= ActualUIMax && (MetaUIMinString.Len() || MetaUIMaxString.Len()))
	{
		//UE_LOGF(LogPropertyNode, Warning, "UI Min (%ls) >= UI Max (%ls) for Ranged Numeric", *UIMinString, *UIMaxString);
	}
	
	MetadataOut.bSupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
	MetadataOut.bSupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
	MetadataOut.bAllowSpinBox = bAllowSpin;

	// By default allow widget to determine default interface
	MetadataOut.TypeInterface = nullptr;

	if (FStructProperty* StructProp = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		if (StructProp->Struct == TBaseStructure<FRotator>::Get())
		{
			// The units for degrees does not support floats
			if constexpr (TIsFloatingPoint<NumericType>::Value && std::is_same<FRotator::FReal, NumericType>::value)
			{
				MetadataOut.TypeInterface = MakeShared<TNumericUnitTypeInterface<FRotator::FReal>>(EUnit::Degrees);
			}
		}
	}
}


template<typename NumericType>
TSharedRef<SWidget> FMathStructCustomization::MakeNumericWidget(
	TSharedRef<IPropertyHandle>& StructurePropertyHandle,
	TSharedRef<IPropertyHandle>& PropertyHandle)
{
	FNumericMetadata<NumericType> Metadata;
	ExtractNumericMetadata(StructurePropertyHandle, Metadata);

	TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;

	return
		SNew(SNumericEntryBox<NumericType>)
			.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, WeakHandlePtr)
			.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
			.Value(this, &FMathStructCustomization::OnGetValue, WeakHandlePtr)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.OnValueCommitted(this, &FMathStructCustomization::OnValueCommitted<NumericType>, WeakHandlePtr)
			.OnValueChanged(this, &FMathStructCustomization::OnValueChanged<NumericType>, WeakHandlePtr)
			.OnBeginSliderMovement(this, &FMathStructCustomization::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FMathStructCustomization::OnEndSliderMovement<NumericType>)
			// Only allow spin on handles with one object.  Otherwise it is not clear what value to spin
			.AllowSpin(PropertyHandle->GetNumOuterObjects() < 2 && Metadata.bAllowSpinBox)
			.ShiftMultiplier(Metadata.ShiftMultiplier)
			.CtrlMultiplier(Metadata.CtrlMultiplier)
			.SupportDynamicSliderMaxValue(Metadata.bSupportDynamicSliderMaxValue)
			.SupportDynamicSliderMinValue(Metadata.bSupportDynamicSliderMinValue)
			.OnDynamicSliderMaxValueChanged(this, &FMathStructCustomization::OnDynamicSliderMaxValueChanged<NumericType>)
			.OnDynamicSliderMinValueChanged(this, &FMathStructCustomization::OnDynamicSliderMinValueChanged<NumericType>)
			.MinValue(Metadata.MinValue)
			.MaxValue(Metadata.MaxValue)
			.MinSliderValue(Metadata.SliderMinValue)
			.MaxSliderValue(Metadata.SliderMaxValue)
			.SliderExponent(Metadata.SliderExponent)
			.Delta(Metadata.Delta)
			// LinearDeltaSensitivity must be left unset if not provided, rather than being set to some default
			.LinearDeltaSensitivity(Metadata.LinearDeltaSensitivity != 0 ? Metadata.LinearDeltaSensitivity : TAttribute<int32>())
			.ToolTipTextFormat(this, &FMathStructCustomization::OnGetValueToolTipTextFormat, WeakHandlePtr)
			.TypeInterface(Metadata.TypeInterface);
}

template <typename NumericType>
void FMathStructCustomization::OnDynamicSliderMaxValueChanged(NumericType NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher)
{
	for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
	{
		TSharedPtr<SNumericEntryBox<NumericType>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<NumericType>>(Widget.Pin());

		if (NumericBox.IsValid())
		{
			TSharedPtr<SSpinBox<NumericType>> SpinBox = StaticCastSharedPtr<SSpinBox<NumericType>>(NumericBox->GetSpinBox());

			if (SpinBox.IsValid())
			{
				if (SpinBox != InValueChangedSourceWidget)
				{
					if ((NewMaxSliderValue > SpinBox->GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
					{
						// Make sure the max slider value is not a getter otherwise we will break the link!
						verifySlow(!SpinBox->IsMaxSliderValueBound());
						SpinBox->SetMaxSliderValue(NewMaxSliderValue);
					}
				}
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMaxValueChanged.Broadcast((float)NewMaxSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfHigher);
	}
}

template <typename NumericType>
void FMathStructCustomization::OnDynamicSliderMinValueChanged(NumericType NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower)
{
	for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
	{
		TSharedPtr<SNumericEntryBox<NumericType>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<NumericType>>(Widget.Pin());

		if (NumericBox.IsValid())
		{
			TSharedPtr<SSpinBox<NumericType>> SpinBox = StaticCastSharedPtr<SSpinBox<NumericType>>(NumericBox->GetSpinBox());

			if (SpinBox.IsValid())
			{
				if (SpinBox != InValueChangedSourceWidget)
				{
					if ((NewMinSliderValue < SpinBox->GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
					{
						// Make sure the min slider value is not a getter otherwise we will break the link!
						verifySlow(!SpinBox->IsMinSliderValueBound());
						SpinBox->SetMinSliderValue(NewMinSliderValue);
					}
				}
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMinValueChanged.Broadcast((float)NewMinSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfLower);
	}
}

TSharedRef<SWidget> FMathStructCustomization::MakeChildWidget(
	TSharedRef<IPropertyHandle>& StructurePropertyHandle,
	TSharedRef<IPropertyHandle>& PropertyHandle)
{
	const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();
	
	if (PropertyClass == FFloatProperty::StaticClass())
	{
		return MakeNumericWidget<float>(StructurePropertyHandle, PropertyHandle);
	}
	
	if (PropertyClass == FDoubleProperty::StaticClass())
	{
		return MakeNumericWidget<double>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FIntProperty::StaticClass())
	{
		return MakeNumericWidget<int32>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FInt64Property::StaticClass())
	{
		return MakeNumericWidget<int64>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FUInt32Property::StaticClass())
	{
		return MakeNumericWidget<uint32>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FUInt64Property::StaticClass())
	{
		return MakeNumericWidget<uint64>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FByteProperty::StaticClass())
	{
		return MakeNumericWidget<uint8>(StructurePropertyHandle, PropertyHandle);
	}

	if (PropertyClass == FEnumProperty::StaticClass())
	{
		const FEnumProperty* EnumPropertyClass = static_cast<const FEnumProperty*>(PropertyHandle->GetProperty());
		const FProperty* Enum = EnumPropertyClass->GetUnderlyingProperty();
		const FFieldClass* EnumClass = Enum->GetClass();
		if (EnumClass == FByteProperty::StaticClass())
		{
			return MakeNumericWidget<uint8>(StructurePropertyHandle, PropertyHandle);
		}
		else if (EnumClass == FIntProperty::StaticClass())
		{
			return MakeNumericWidget<int32>(StructurePropertyHandle, PropertyHandle);
		}
	}

	check(0); // Unsupported class
	return SNullWidget::NullWidget;
}


template<typename NumericType>
TOptional<NumericType> FMathStructCustomization::OnGetValue(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	NumericType NumericVal = 0;
	if (WeakHandlePtr.Pin()->GetValue(NumericVal) == FPropertyAccess::Success)
	{
		return TOptional<NumericType>(NumericVal);
	}

	// Value couldn't be accessed.  Return an unset value
	return TOptional<NumericType>();
}


template<typename NumericType>
void FMathStructCustomization::OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags;
	SetValue(NewValue, Flags, WeakHandlePtr);
}


template<typename NumericType>
void FMathStructCustomization::OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	if (bIsUsingSlider)
	{
		EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable;
		SetValue(NewValue, Flags, WeakHandlePtr);
	}
}


template<typename NumericType>
void FMathStructCustomization::SetValue(NumericType NewValue, EPropertyValueSetFlags::Type Flags, TWeakPtr<IPropertyHandle> WeakHandlePtr)
{
	if (bPreserveScaleRatio)
	{
		// Get the value for each object for the modified component
		TArray<FString> OldValues;
		if (WeakHandlePtr.Pin()->GetPerObjectValues(OldValues) == FPropertyAccess::Success)
		{
			// Loop through each object and scale based on the new ratio for each object individually
			for (int32 OutputIndex = 0; OutputIndex < OldValues.Num(); ++OutputIndex)
			{
				NumericType OldValue;
				TTypeFromString<NumericType>::FromString(OldValue, *OldValues[OutputIndex]);

				// Account for the previous scale being zero.  Just set to the new value in that case?
				// Use double-precision division to compute the ratio, avoiding truncation for integer
				// NumericType (e.g. int32: 2/3 = 0, but 2.0/3.0 = 0.667).
				const double Ratio = (OldValue == NumericType(0))
					? static_cast<double>(NewValue)
					: static_cast<double>(NewValue) / static_cast<double>(OldValue);

				// Loop through all the child handles (each component of the math struct, like X, Y, Z...etc)
				for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
				{
					// Ignore scaling our selves.
					TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
					if (ChildHandle != WeakHandlePtr.Pin())
					{
						// Get the value for each object.
						TArray<FString> ObjectChildValues;
						if (ChildHandle->GetPerObjectValues(ObjectChildValues) == FPropertyAccess::Success)
						{
							// Individually scale each object's components by the same ratio.
							for (int32 ChildOutputIndex = 0; ChildOutputIndex < ObjectChildValues.Num(); ++ChildOutputIndex)
							{
								NumericType ChildOldValue;
								TTypeFromString<NumericType>::FromString(ChildOldValue, *ObjectChildValues[ChildOutputIndex]);

								// Apply ratio in double precision; round to nearest integer for integral types.
								NumericType ChildNewValue;
								if constexpr (TIsIntegral<NumericType>::Value)
								{
									ChildNewValue = static_cast<NumericType>(FMath::RoundToInt64(static_cast<double>(ChildOldValue) * Ratio));
								}
								else
								{
									ChildNewValue = static_cast<NumericType>(static_cast<double>(ChildOldValue) * Ratio);
								}
								ObjectChildValues[ChildOutputIndex] = TTypeToString<NumericType>::ToSanitizedString(ChildNewValue);
							}

							ChildHandle->SetPerObjectValues(ObjectChildValues, Flags);
						}
					}
				}
			}
		}
	}

	WeakHandlePtr.Pin()->SetValue(NewValue, Flags);
}

template <typename NumericType>
FText FMathStructCustomization::OnGetValueToolTip(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	return FText::GetEmpty();
}

TOptional<FTextFormat> FMathStructCustomization::OnGetValueToolTipTextFormat(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakHandlePtr.Pin())
	{
		TStringBuilder<32> ToolTipTextFormatString;
		FText TooltipText = PropertyHandle->GetToolTipText();
		if (TooltipText.IsEmpty())
		{
			TooltipText = PropertyHandle->GetPropertyDisplayName();
		}
		ToolTipTextFormatString.Append(TooltipText.ToString());
		ToolTipTextFormatString.Append(TEXT(": {0}"));

		return TOptional<FTextFormat>(FText::FromString(ToolTipTextFormatString.ToString()));
	}

	return TOptional<FTextFormat>();
}

bool FMathStructCustomization::IsValueEnabled(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	if (WeakHandlePtr.IsValid())
	{
		return !WeakHandlePtr.Pin()->IsEditConst();
	}

	return false;
}


void FMathStructCustomization::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	GEditor->BeginTransaction(LOCTEXT("SetVectorProperty", "Set Vector Property"));
}


template<typename NumericType>
void FMathStructCustomization::OnEndSliderMovement(NumericType NewValue)
{
	bIsUsingSlider = false;

	GEditor->EndTransaction();
}

#undef LOCTEXT_NAMESPACE
