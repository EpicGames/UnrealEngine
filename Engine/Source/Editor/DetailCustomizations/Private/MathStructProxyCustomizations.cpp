// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MathStructProxyCustomizations.h"

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Matrix.h"
#include "Math/Quat.h"
#include "Math/ScaleRotationTranslationMatrix.h"
#include "Math/TransformVectorized.h"
#include "Math/UnrealMathSSE.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AxisDisplayInfo.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/OverridableManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

class SWidget;

#define LOCTEXT_NAMESPACE "MatrixStructCustomization"
void FMathStructProxyCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();
}

void FMathStructProxyCustomization::MakeHeaderRow( TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row )
{

}

void FMathStructProxyCustomization::OnBeginSliderMovement()
{
	bIsUsingSlider = true;
}

template <typename ProxyType, typename NumericType>
FText FMathStructProxyCustomization::OnGetValueToolTip(TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef<TProxyProperty<ProxyType, NumericType>> ProxyValue, FText Label) const
{
	return FText::GetEmpty();
}

TOptional<FTextFormat> FMathStructProxyCustomization::OnGetValueToolTipTextFormat(FText Label) const
{
	if (!Label.IsEmptyOrWhitespace())
	{
		TStringBuilder<32> ToolTipTextFormatString;
		ToolTipTextFormatString.Append(Label.ToString());
		ToolTipTextFormatString.Append(TEXT(": {0}"));
		return TOptional<FTextFormat>(FText::FromString(ToolTipTextFormatString.ToString()));
	}
	return TOptional<FTextFormat>();
}

template<typename T>
TSharedRef<IPropertyTypeCustomization> FMatrixStructCustomization<T>::MakeInstance()
{
	return MakeShareable( new FMatrixStructCustomization<T> );
}

template<typename T>
void FMatrixStructCustomization<T>::MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
{
	Row
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(0.0f)
	.MaxDesiredWidth(0.0f)
	[
		SNullWidget::NullWidget
	];
}

template<typename T>
void FMatrixStructCustomization<T>::CustomizeLocation(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row)
{
	TWeakPtr<IPropertyHandle> WeakHandlePtr = StructPropertyHandle;

	if (Row.IsPasteFromTextBound())
	{
		Row.OnPasteFromTextDelegate.Pin()->AddSP(this, &FMatrixStructCustomization<T>::OnPasteFromText, FTransformField::Location, WeakHandlePtr);
	}

	constexpr int32 NumComponents = 3;
	TStaticArray<TFunction<TSharedRef<SWidget>()>, NumComponents> ComponentConstructors = {
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedTranslationX, AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward), false, AxisDisplayInfo::GetAxisColor(EAxisList::Forward));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedTranslationY, AxisDisplayInfo::GetAxisToolTip(EAxisList::Left), false, AxisDisplayInfo::GetAxisColor(EAxisList::Left));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedTranslationZ, AxisDisplayInfo::GetAxisToolTip(EAxisList::Up), false, AxisDisplayInfo::GetAxisColor(EAxisList::Up));
		}
	};
	const TStaticArray<FMargin, NumComponents> Paddings =
	{
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 0.0f, 2.0f)
	};

	TSharedRef<SHorizontalBox> HBox = MakeShared<SHorizontalBox>();
	FIntVector4 Swizzle = GetSwizzle();
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
	{
		TSharedRef<SWidget> Widget = SNullWidget::NullWidget;
		if (ensure(Swizzle[ComponentIndex] < NumComponents))
		{
			const int32 SwizzledComponentIndex = Swizzle[ComponentIndex];
			Widget = ComponentConstructors[SwizzledComponentIndex]();
		}

		HBox->AddSlot()
			.Padding(Paddings[ComponentIndex])
			[
				Widget
			];
	}

	Row
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnCopy, FTransformField::Location, WeakHandlePtr)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnPaste, FTransformField::Location, WeakHandlePtr)))
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget(LOCTEXT("LocationLabel", "Location"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		HBox
	];
}

template<typename T>
void FMatrixStructCustomization<T>::CustomizeRotation(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row, TOptional<FText> OverrideName)
{
	TWeakPtr<IPropertyHandle> WeakHandlePtr = StructPropertyHandle;

	if (Row.IsPasteFromTextBound())
	{
		Row.OnPasteFromTextDelegate.Pin()->AddSP(this, &FMatrixStructCustomization<T>::OnPasteFromText, FTransformField::Rotation, WeakHandlePtr);
	}
	
	constexpr int32 NumComponents = 3;
	TStaticArray<TFunction<TSharedRef<SWidget>()>, NumComponents> ComponentConstructors = {
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TRotator<T>, T>(StructPropertyHandle, CachedRotationRoll, AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Forward), true, AxisDisplayInfo::GetAxisColor(EAxisList::Forward));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TRotator<T>, T>(StructPropertyHandle, CachedRotationPitch, AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Left), true, AxisDisplayInfo::GetAxisColor(EAxisList::Left));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TRotator<T>, T>(StructPropertyHandle, CachedRotationYaw, AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Up), true, AxisDisplayInfo::GetAxisColor(EAxisList::Up));
		}
	};
	const TStaticArray<FMargin, NumComponents> Paddings =
	{
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 0.0f, 2.0f)
	};

	TSharedRef<SHorizontalBox> HBox = MakeShared<SHorizontalBox>();
	FIntVector4 Swizzle = GetSwizzle();
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
	{
		TSharedRef<SWidget> Widget = SNullWidget::NullWidget;
		if (ensure(Swizzle[ComponentIndex] < NumComponents))
		{
			const int32 SwizzledComponentIndex = Swizzle[ComponentIndex];
			Widget = ComponentConstructors[SwizzledComponentIndex]();
		}

		HBox->AddSlot()
			.Padding(Paddings[ComponentIndex])
			[
				Widget
			];
	}

	Row
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnCopy, FTransformField::Rotation, WeakHandlePtr)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnPaste, FTransformField::Rotation, WeakHandlePtr)))
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget(OverrideName.IsSet() ? OverrideName.GetValue() : LOCTEXT("RotationLabel", "Rotation"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		HBox
	];
}

template<typename T>
void FMatrixStructCustomization<T>::CustomizeScale(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row)
{
	TWeakPtr<IPropertyHandle> WeakHandlePtr = StructPropertyHandle;

	if (Row.IsPasteFromTextBound())
	{
		Row.OnPasteFromTextDelegate.Pin()->AddSP(this, &FMatrixStructCustomization<T>::OnPasteFromText, FTransformField::Scale, WeakHandlePtr);
	}

	constexpr int32 NumComponents = 3;
	TStaticArray<TFunction<TSharedRef<SWidget>()>, NumComponents> ComponentConstructors = {
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedScaleX, AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward), false, AxisDisplayInfo::GetAxisColor(EAxisList::Forward));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedScaleY, AxisDisplayInfo::GetAxisToolTip(EAxisList::Left), false, AxisDisplayInfo::GetAxisColor(EAxisList::Left));
		},
		[this, &StructPropertyHandle]()->TSharedRef<SWidget>
		{
			return MakeNumericProxyWidget<UE::Math::TVector<T>, T>(StructPropertyHandle, CachedScaleZ, AxisDisplayInfo::GetAxisToolTip(EAxisList::Up), false, AxisDisplayInfo::GetAxisColor(EAxisList::Up));
		}
	};
	const TStaticArray<FMargin, NumComponents> Paddings =
	{
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 3.0f, 2.0f),
		FMargin(0.0f, 2.0f, 0.0f, 2.0f)
	};

	TSharedRef<SHorizontalBox> HBox = MakeShared<SHorizontalBox>();
	FIntVector4 Swizzle = GetSwizzle();
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
	{
		TSharedRef<SWidget> Widget = SNullWidget::NullWidget;
		if (ensure(Swizzle[ComponentIndex] < NumComponents))
		{
			const int32 SwizzledComponentIndex = Swizzle[ComponentIndex];
			Widget = ComponentConstructors[SwizzledComponentIndex]();
		}

		HBox->AddSlot()
			.Padding(Paddings[ComponentIndex])
			[
				Widget
			];
	}

	Row
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnCopy, FTransformField::Scale, WeakHandlePtr)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMatrixStructCustomization<T>::OnPaste, FTransformField::Scale, WeakHandlePtr)))
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget(LOCTEXT("ScaleLabel", "Scale"))
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		HBox
	];
}

template<typename T>
void FMatrixStructCustomization<T>::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FMathStructProxyCustomization::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);

	TWeakPtr<IPropertyHandle> WeakHandlePtr = StructPropertyHandle;

	bUseLeftUpForwardAxisDisplayCoordinateSystem =
		AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward
		&& !StructPropertyHandle->GetProperty()->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintReadOnly);

	CustomizeLocation(StructPropertyHandle, StructBuilder.AddCustomRow(LOCTEXT("LocationLabel", "Location")));
	CustomizeRotation(StructPropertyHandle, StructBuilder.AddCustomRow(LOCTEXT("RotationLabel", "Rotation")));
	CustomizeScale(StructPropertyHandle, StructBuilder.AddCustomRow(LOCTEXT("ScaleLabel", "Scale")));
}

template<typename T>
void FMatrixStructCustomization<T>::OnCopy(FTransformField::Type Type, TWeakPtr<IPropertyHandle> PropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FString CopyStr;
	CacheValues(PropertyHandle);

	switch (Type)
	{
		case FTransformField::Location:
		{
			UE::Math::TVector<T> Location = CachedTranslation->Get();
			CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), Location.X, Location.Y, Location.Z);
			break;
		}

		case FTransformField::Rotation:
		{
			UE::Math::TRotator<T> Rotation = CachedRotation->Get();
			CopyStr = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), Rotation.Pitch, Rotation.Yaw, Rotation.Roll);
			break;
		}

		case FTransformField::Scale:
		{
			UE::Math::TVector<T> Scale = CachedScale->Get();
			CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), Scale.X, Scale.Y, Scale.Z);
			break;
		}
	}

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

template<typename T>
void FMatrixStructCustomization<T>::OnPaste(FTransformField::Type Type, TWeakPtr<IPropertyHandle> PropertyHandlePtr)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	PasteFromText(TEXT(""), PastedText, Type, PropertyHandlePtr);
}

template <typename T>
void FMatrixStructCustomization<T>::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	FTransformField::Type Type,
	TWeakPtr<IPropertyHandle> PropertyHandlePtr)
{
	PasteFromText(InTag, InText, Type, PropertyHandlePtr);
}

template <typename T>
void FMatrixStructCustomization<T>::PasteFromText(
	const FString& InTag,
	const FString& InText,
	FTransformField::Type Type,
	TWeakPtr<IPropertyHandle> PropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FString PastedText = InText;

	switch (Type)
	{
		case FTransformField::Location:
		{
			UE::Math::TVector<T> Location;
			if (Location.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteLocation", "Paste Location"));
				CachedTranslationX->Set(Location.X);
				CachedTranslationY->Set(Location.Y);
				CachedTranslationZ->Set(Location.Z);
				FlushValues(PropertyHandle);
			}
			break;
		}

		case FTransformField::Rotation:
		{
			UE::Math::TRotator<T> Rotation;
			PastedText.ReplaceInline(TEXT("Pitch="), TEXT("P="));
			PastedText.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
			PastedText.ReplaceInline(TEXT("Roll="), TEXT("R="));
			if (Rotation.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
				CachedRotationPitch->Set(Rotation.Pitch);
				CachedRotationYaw->Set(Rotation.Yaw);
				CachedRotationRoll->Set(Rotation.Roll);
				FlushValues(PropertyHandle);
			}
			break;
		}

		case FTransformField::Scale:
		{
			UE::Math::TVector<T> Scale;
			if (Scale.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteScale", "Paste Scale"));
				CachedScaleX->Set(Scale.X);
				CachedScaleY->Set(Scale.Y);
				CachedScaleZ->Set(Scale.Z);
				FlushValues(PropertyHandle);
			}
			break;
		}
	}
}

template<typename T>
bool FMatrixStructCustomization<T>::CacheValues( TWeakPtr<IPropertyHandle> PropertyHandlePtr ) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	const UE::Math::TMatrix<T>* FirstMatrixValue = nullptr;
	for(void* RawDataPtr : RawData)
	{
		UE::Math::TMatrix<T>* MatrixValue = reinterpret_cast<UE::Math::TMatrix<T>*>(RawDataPtr);
		if (MatrixValue == nullptr)
		{
			return false;
		}

		if(FirstMatrixValue)
		{
			if(!FirstMatrixValue->Equals(*MatrixValue, 0.0001f))
			{
				return false;
			}
		}
		else
		{
			FirstMatrixValue = MatrixValue;
		}
	}

	if(FirstMatrixValue)
	{
		CachedTranslation->Set(FirstMatrixValue->GetOrigin());
		if (bUseLeftUpForwardAxisDisplayCoordinateSystem)
		{
			CachedTranslationY->Set(-CachedTranslationY->Get());
		}
		CachedRotation->Set(FirstMatrixValue->Rotator());
		CachedScale->Set(FirstMatrixValue->GetScaleVector());
		return true;
	}

	return false;
}

template<typename T>
bool FMatrixStructCustomization<T>::FlushValues( TWeakPtr<IPropertyHandle> PropertyHandlePtr ) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();
	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	// The object array should either be empty or the same size as the raw data array.
	check(!OuterObjects.Num() || OuterObjects.Num() == RawData.Num());

	// Persistent flag that's set when we're in the middle of an interactive change (note: assumes multiple interactive changes do not occur in parallel).
	static bool bIsInteractiveChangeInProgress = false;

	bool bNotifiedPreChange = false;
	for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ValueIndex++)
	{
		UE::Math::TMatrix<T>* MatrixValue = reinterpret_cast<UE::Math::TMatrix<T>*>(RawData[ValueIndex]);
		if (MatrixValue != NULL)
		{
			const UE::Math::TMatrix<T> PreviousValue = *MatrixValue;
			const UE::Math::TRotator<T> CurrentRotation = MatrixValue->Rotator();
			const UE::Math::TVector<T> CurrentTranslation = MatrixValue->GetOrigin();
			const UE::Math::TVector<T> CurrentScale = MatrixValue->GetScaleVector();

			UE::Math::TRotator<T> Rotation(
				CachedRotationPitch->IsSet() ? CachedRotationPitch->Get() : CurrentRotation.Pitch,
				CachedRotationYaw->IsSet() ? CachedRotationYaw->Get() : CurrentRotation.Yaw,
				CachedRotationRoll->IsSet() ? CachedRotationRoll->Get() : CurrentRotation.Roll
				);
			UE::Math::TVector<T> Translation(
				CachedTranslationX->IsSet() ? CachedTranslationX->Get() : CurrentTranslation.X,
				CachedTranslationY->IsSet()
					? (bUseLeftUpForwardAxisDisplayCoordinateSystem ? -CachedTranslationY->Get() : CachedTranslationY->Get())
					: CurrentTranslation.Y,
				CachedTranslationZ->IsSet() ? CachedTranslationZ->Get() : CurrentTranslation.Z
				);
			UE::Math::TVector<T> Scale(
				CachedScaleX->IsSet() ? CachedScaleX->Get() : CurrentScale.X,
				CachedScaleY->IsSet() ? CachedScaleY->Get() : CurrentScale.Y,
				CachedScaleZ->IsSet() ? CachedScaleZ->Get() : CurrentScale.Z
				);

			const UE::Math::TMatrix<T> NewValue = UE::Math::TScaleRotationTranslationMatrix<T>(Scale, Rotation, Translation);

			if (!bNotifiedPreChange && (!MatrixValue->Equals(NewValue, 0.0f) || (!bIsUsingSlider && bIsInteractiveChangeInProgress)))
			{
				if (!bIsInteractiveChangeInProgress)
				{
					GEditor->BeginTransaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), PropertyHandle->GetPropertyDisplayName()));
				}

				PropertyHandle->NotifyPreChange();
				bNotifiedPreChange = true;

				bIsInteractiveChangeInProgress = bIsUsingSlider;
			}

			// Set the new value.
			*MatrixValue = NewValue;

			// Propagate default value changes after updating, for archetypes. As per usual, we only propagate the change if the instance matches the archetype's value.
			// Note: We cannot use the "normal" PropertyNode propagation logic here, because that is string-based and the decision to propagate relies on an exact value match.
			// Here, we're dealing with conversions between UE::Math::TMatrix<T> and UE::Math::TVector<T>/UE::Math::TRotator<T> values, so there is some precision loss that requires a tolerance when comparing values.
			if (ValueIndex < OuterObjects.Num() && OuterObjects[ValueIndex]->IsTemplate())
			{
				TArray<UObject*> ArchetypeInstances;
				OuterObjects[ValueIndex]->GetArchetypeInstances(ArchetypeInstances);
				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					if (!FOverridableManager::Get().IsEnabled(ArchetypeInstance))
					{
						UE::Math::TMatrix<T>* CurrentValue = reinterpret_cast<UE::Math::TMatrix<T>*>(PropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(ArchetypeInstance)));
						if (CurrentValue && CurrentValue->Equals(PreviousValue))
						{
							*CurrentValue = NewValue;
						}
					}
				}
			}
		}
	}

	if (bNotifiedPreChange)
	{
		PropertyHandle->NotifyPostChange(bIsUsingSlider ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);

		if (!bIsUsingSlider)
		{
			GEditor->EndTransaction();
			bIsInteractiveChangeInProgress = false;
		}
	}

	if (PropertyUtilities.IsValid() && !bIsInteractiveChangeInProgress)
	{
		FPropertyChangedEvent ChangeEvent(PropertyHandle->GetProperty(), EPropertyChangeType::ValueSet, OuterObjects);
		PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	return true;
}

template <typename T>
FIntVector4 FMatrixStructCustomization<T>::GetSwizzle() const
{
	return {0, 1, 2, 3};
}

template<typename T>
TSharedRef<IPropertyTypeCustomization> FTransformStructCustomization<T>::MakeInstance() 
{
	return MakeShareable( new FTransformStructCustomization );
}

template<typename T>
bool FTransformStructCustomization<T>::CacheValues( TWeakPtr<IPropertyHandle> PropertyHandlePtr ) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	const UE::Math::TTransform<T>* FirstTransformValue = nullptr;
	for(void* RawDataPtr : RawData)
	{
		UE::Math::TTransform<T>* TransformValue = reinterpret_cast<UE::Math::TTransform<T>*>(RawDataPtr);
		if (TransformValue == nullptr)
		{
			return false;
		}

		if(FirstTransformValue)
		{
			if(!FirstTransformValue->Equals(*TransformValue, 0.0001f))
			{
				return false;
			}
		}
		else
		{
			FirstTransformValue = TransformValue;
		}
	}

	if(FirstTransformValue)
	{
		this->CachedTranslation->Set(FirstTransformValue->GetTranslation());
		if (this->bUseLeftUpForwardAxisDisplayCoordinateSystem)
		{
			this->CachedTranslationY->Set(-this->CachedTranslationY->Get());
		}
		this->CachedRotation->Set(FirstTransformValue->GetRotation().Rotator());
		this->CachedScale->Set(FirstTransformValue->GetScale3D());
		return true;
	}

	return false;
}

template<typename T>
bool FTransformStructCustomization<T>::FlushValues( TWeakPtr<IPropertyHandle> PropertyHandlePtr ) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	// The object array should either be empty or the same size as the raw data array.
	check(!OuterObjects.Num() || OuterObjects.Num() == RawData.Num());

	// Persistent flag that's set when we're in the middle of an interactive change (note: assumes multiple interactive changes do not occur in parallel).
	static bool bIsInteractiveChangeInProgress = false;

	bool bNotifiedPreChange = false;
	for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ValueIndex++)
	{
		UE::Math::TTransform<T>* TransformValue = reinterpret_cast<UE::Math::TTransform<T>*>(RawData[ValueIndex]);
		if (TransformValue != NULL)
		{
			const UE::Math::TTransform<T> PreviousValue = *TransformValue;
			const UE::Math::TRotator<T> CurrentRotation = TransformValue->GetRotation().Rotator();
			const UE::Math::TVector<T> CurrentTranslation = TransformValue->GetTranslation();
			const UE::Math::TVector<T> CurrentScale = TransformValue->GetScale3D();

			UE::Math::TRotator<T> Rotation(
				this->CachedRotationPitch->IsSet() ? this->CachedRotationPitch->Get() : CurrentRotation.Pitch,
				this->CachedRotationYaw->IsSet() ? this->CachedRotationYaw->Get() : CurrentRotation.Yaw,
				this->CachedRotationRoll->IsSet() ? this->CachedRotationRoll->Get() : CurrentRotation.Roll
				);
			UE::Math::TVector<T> Translation(
				this->CachedTranslationX->IsSet() ? this->CachedTranslationX->Get() : CurrentTranslation.X,
				this->CachedTranslationY->IsSet() 
					? (this->bUseLeftUpForwardAxisDisplayCoordinateSystem ? -this->CachedTranslationY->Get() : this->CachedTranslationY->Get())
					: CurrentTranslation.Y,
				this->CachedTranslationZ->IsSet() ? this->CachedTranslationZ->Get() : CurrentTranslation.Z
				);
			UE::Math::TVector<T> Scale(
				this->CachedScaleX->IsSet() ? this->CachedScaleX->Get() : CurrentScale.X,
				this->CachedScaleY->IsSet() ? this->CachedScaleY->Get() : CurrentScale.Y,
				this->CachedScaleZ->IsSet() ? this->CachedScaleZ->Get() : CurrentScale.Z
				);

			const UE::Math::TTransform<T> NewValue = UE::Math::TTransform<T>(Rotation, Translation, Scale);

			if (!bNotifiedPreChange && (!TransformValue->Equals(NewValue, 0.0f) || (!this->bIsUsingSlider && bIsInteractiveChangeInProgress)))
			{
				if (!bIsInteractiveChangeInProgress)
				{
					GEditor->BeginTransaction(FText::Format(NSLOCTEXT("FTransformStructCustomization", "SetPropertyValue", "Set {0}"), PropertyHandle->GetPropertyDisplayName()));
				}

				PropertyHandle->NotifyPreChange();
				bNotifiedPreChange = true;

				bIsInteractiveChangeInProgress = this->bIsUsingSlider;
			}

			// Set the new value.
			*TransformValue = NewValue;

			// Propagate default value changes after updating, for archetypes. As per usual, we only propagate the change if the instance matches the archetype's value.
			// Note: We cannot use the "normal" PropertyNode propagation logic here, because that is string-based and the decision to propagate relies on an exact value match.
			// Here, we're dealing with conversions between UE::Math::TTransform<T> and UE::Math::TVector<T>/UE::Math::TRotator<T> values, so there is some precision loss that requires a tolerance when comparing values.
			if (ValueIndex < OuterObjects.Num() && OuterObjects[ValueIndex]->IsTemplate())
			{
				TArray<UObject*> ArchetypeInstances;
				OuterObjects[ValueIndex]->GetArchetypeInstances(ArchetypeInstances);
				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					if (!FOverridableManager::Get().IsEnabled(ArchetypeInstance))
					{
						UE::Math::TTransform<T>* CurrentValue = reinterpret_cast<UE::Math::TTransform<T>*>(PropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(ArchetypeInstance)));
						if (CurrentValue && CurrentValue->Equals(PreviousValue))
						{
							*CurrentValue = NewValue;
						}
					}
				}
			}
		}
	}
	
	if (bNotifiedPreChange)
	{
		PropertyHandle->NotifyPostChange(this->bIsUsingSlider ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);

		if (!this->bIsUsingSlider)
		{
			GEditor->EndTransaction();
			bIsInteractiveChangeInProgress = false;
		}
	}

	if (this->PropertyUtilities.IsValid() && !bIsInteractiveChangeInProgress)
	{
		FPropertyChangedEvent ChangeEvent(PropertyHandle->GetProperty(), EPropertyChangeType::ValueSet, OuterObjects);
		this->PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	return true;
}

template <typename T>
FIntVector4 FTransformStructCustomization<T>::GetSwizzle() const
{
	return AxisDisplayInfo::GetTransformAxisSwizzle();
}

template<typename T>
TSharedRef<IPropertyTypeCustomization> FQuatStructCustomization<T>::MakeInstance()
{
	return MakeShareable(new FQuatStructCustomization);
}


template<typename T>
void FQuatStructCustomization<T>::MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row)
{
	this->CustomizeRotation(InStructPropertyHandle, Row, InStructPropertyHandle->GetPropertyDisplayName());
}

template<typename T>
void FQuatStructCustomization<T>::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FMathStructProxyCustomization::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
}

template<typename T>
bool FQuatStructCustomization<T>::CacheValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	if (RawData.Num() == 1)
	{
		UE::Math::TQuat<T>* QuatValue = reinterpret_cast<UE::Math::TQuat<T>*>(RawData[0]);
		if (QuatValue != NULL)
		{
			this->CachedRotation->Set(QuatValue->Rotator());
			return true;
		}
	}

	return false;
}

template<typename T>
bool FQuatStructCustomization<T>::FlushValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandlePtr.Pin();

	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	// The object array should either be empty or the same size as the raw data array.
	check(!OuterObjects.Num() || OuterObjects.Num() == RawData.Num());

	// Persistent flag that's set when we're in the middle of an interactive change (note: assumes multiple interactive changes do not occur in parallel).
	static bool bIsInteractiveChangeInProgress = false;

	bool bNotifiedPreChange = false;
	for (int32 ValueIndex = 0; ValueIndex < RawData.Num(); ValueIndex++)
	{
		UE::Math::TQuat<T>* QuatValue = reinterpret_cast<UE::Math::TQuat<T>*>(RawData[0]);
		if (QuatValue != NULL)
		{
			const UE::Math::TQuat<T> PreviousValue = *QuatValue;
			const UE::Math::TRotator<T> CurrentRotation = QuatValue->Rotator();

			UE::Math::TRotator<T> Rotation(
				this->CachedRotationPitch->IsSet() ? this->CachedRotationPitch->Get() : CurrentRotation.Pitch,
				this->CachedRotationYaw->IsSet() ? this->CachedRotationYaw->Get() : CurrentRotation.Yaw,
				this->CachedRotationRoll->IsSet() ? this->CachedRotationRoll->Get() : CurrentRotation.Roll
				);
			
			const UE::Math::TQuat<T> NewValue = Rotation.Quaternion();

			// In some cases the UE::Math::TQuat<T> pointed to in RawData is no longer aligned to 16 bytes.
			// Make a local copy to guarantee the alignment criterions of the vector intrinsics inside UE::Math::TQuat<T>::Equals
			const UE::Math::TQuat<T> AlignedQuatValue = *QuatValue; 

			if (!bNotifiedPreChange && (!AlignedQuatValue.Equals(NewValue, 0.0f) || (!this->bIsUsingSlider && bIsInteractiveChangeInProgress)))
			{
				if (!bIsInteractiveChangeInProgress)
				{
					GEditor->BeginTransaction(FText::Format(NSLOCTEXT("FQuatStructCustomization", "SetPropertyValue", "Set {0}"), PropertyHandle->GetPropertyDisplayName()));
				}

				PropertyHandle->NotifyPreChange();
				bNotifiedPreChange = true;

				bIsInteractiveChangeInProgress = this->bIsUsingSlider;
			}

			// Set the new value.
			*QuatValue = NewValue;

			// Propagate default value changes after updating, for archetypes. As per usual, we only propagate the change if the instance matches the archetype's value.
			// Note: We cannot use the "normal" PropertyNode propagation logic here, because that is string-based and the decision to propagate relies on an exact value match.
			// Here, we're dealing with conversions between UE::Math::TQuat<T> and UE::Math::TRotator<T> values, so there is some precision loss that requires a tolerance when comparing values.
			if (ValueIndex < OuterObjects.Num() && OuterObjects[ValueIndex]->IsTemplate())
			{
				TArray<UObject*> ArchetypeInstances;
				OuterObjects[ValueIndex]->GetArchetypeInstances(ArchetypeInstances);
				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					if (!FOverridableManager::Get().IsEnabled(ArchetypeInstance))
					{
						UE::Math::TQuat<T>* CurrentValue = reinterpret_cast<UE::Math::TQuat<T>*>(PropertyHandle->GetValueBaseAddress(reinterpret_cast<uint8*>(ArchetypeInstance)));
						if (CurrentValue && CurrentValue->Equals(PreviousValue))
						{
							*CurrentValue = NewValue;
						}
					}
				}
			}
		}
	}

	if (bNotifiedPreChange)
	{
		PropertyHandle->NotifyPostChange(this->bIsUsingSlider ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);

		if (!this->bIsUsingSlider)
		{
			GEditor->EndTransaction();
			bIsInteractiveChangeInProgress = false;
		}
	}

	if (this->PropertyUtilities.IsValid() && !bIsInteractiveChangeInProgress)
	{
		FPropertyChangedEvent ChangeEvent(PropertyHandle->GetProperty(), EPropertyChangeType::ValueSet, OuterObjects);
		this->PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	return true;
}

// Instantiate for linker
template class FMatrixStructCustomization<float>;
template class FMatrixStructCustomization<double>;
template class FTransformStructCustomization<float>;
template class FTransformStructCustomization<double>;
template class FQuatStructCustomization<float>;
template class FQuatStructCustomization<double>;

#undef LOCTEXT_NAMESPACE
