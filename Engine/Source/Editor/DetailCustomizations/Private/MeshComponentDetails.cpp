// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshComponentDetails.h"

#include "AssetSelection.h"
#include "Components/MeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"

TSharedRef<IDetailCustomization> FMeshComponentDetails::MakeInstance()
{
	return MakeShareable( new FMeshComponentDetails );
}

void FMeshComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	RenderingCategory = &DetailLayout.EditCategory("Rendering");
	TSharedRef<IPropertyHandle> MaterialProperty = DetailLayout.GetProperty( GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials) );

	if( MaterialProperty->IsValidHandle() )
	{
		// Only show this in the advanced section of the category if we have selected actors (which will show a separate material section)
		TSharedPtr<IDetailsView> DetailsView = DetailLayout.GetDetailsViewSharedPtr();
		bool bIsAdvanced = DetailsView && DetailsView->GetSelectedActorInfo().NumSelected > 0;

		RenderingCategory->AddProperty( MaterialProperty, bIsAdvanced ? EPropertyLocation::Advanced : EPropertyLocation::Default );
	}

}
