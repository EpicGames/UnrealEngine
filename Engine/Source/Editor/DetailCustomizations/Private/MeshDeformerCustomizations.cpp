// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDeformerCustomizations.h"

#include "Animation/MeshDeformer.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"

namespace
{
	/** Pull the class type for filtering from the property. */
	UClass* GetFilterClassFromProperty(TSharedRef<IPropertyHandle> InPropertyHandle)
	{
		TArray<UObject*> OwningObjects;
		InPropertyHandle->GetOuterObjects(OwningObjects);
		if (OwningObjects.Num() == 0)
		{
			return nullptr;
		}

		// Use the owner class by default.
		UClass* FilterClass = OwningObjects[0]->GetClass();

		// Override if we have metadata.
		FProperty* DeformerProperty = InPropertyHandle->GetProperty();
		static const FName FilterName(TEXT("Filter"));
		const FString& FilterClassPath = DeformerProperty->GetMetaData(FilterName);
		if (!FilterClassPath.IsEmpty())
		{
			FSoftClassPath FilterSoftClassPath(FilterClassPath);
			FilterClass = FilterSoftClassPath.ResolveClass();
		}

		// Only UActorComponent classes are valid filters.
		if (FilterClass != nullptr && !FilterClass->IsChildOf(UActorComponent::StaticClass()))
		{
			FilterClass = nullptr;
		}

		return FilterClass;
	}

	/** Pull the class type from the PrimaryBindingClass tag in some AssetData. */
	UClass* GetPrimaryBindingClassFromAssetData(FAssetData const& AssetData)
	{
		UClass* BindingClass = nullptr;
		FString BindingClassPath;
		if (AssetData.GetTagValue<FString>("PrimaryBindingClass", BindingClassPath))
		{
			FSoftClassPath BindingSoftClassPath(BindingClassPath);
			BindingClass = BindingSoftClassPath.ResolveClass();
		}
		return BindingClass;
	}
}


TSharedRef<IPropertyTypeCustomization> FMeshDeformerCustomization::MakeInstance()
{
	return MakeShareable(new FMeshDeformerCustomization);
}

void FMeshDeformerCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	UClass* FilterClass = GetFilterClassFromProperty(InPropertyHandle);

	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(InPropertyHandle)
		.AllowedClass(UMeshDeformer::StaticClass())
		.OnShouldFilterAsset_Lambda([FilterClass](FAssetData const& AssetData)
		{
			// Filter depending on whether the PrimaryBindingClass matches our owning object.
			// First try to load the class type from the asset registry.
			UClass* BindingClass = GetPrimaryBindingClassFromAssetData(AssetData);

			// If we can't find the tag in the registry then load the full object to get the class type (slow).
			if (BindingClass == nullptr)
			{
				if (UObject* Object = AssetData.GetAsset())
				{
					FAssetData AssetDataWithObjectRegistryTags;
					Object->GetAssetRegistryTags(AssetDataWithObjectRegistryTags);
					BindingClass = GetPrimaryBindingClassFromAssetData(AssetDataWithObjectRegistryTags);
				}
			}

			const bool bHideEntry = FilterClass != nullptr && BindingClass != nullptr && !FilterClass->IsChildOf(BindingClass);
			return bHideEntry;
		})
	];
}
