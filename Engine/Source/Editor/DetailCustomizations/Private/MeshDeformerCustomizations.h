// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class FMeshDeformerCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}
};
