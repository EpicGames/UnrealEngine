// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergingSettingsCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "FMeshMergingSettingCustomization"

FMeshMergingSettingsObjectCustomization::~FMeshMergingSettingsObjectCustomization()
{
}

void FMeshMergingSettingsObjectCustomization::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	TSharedRef<IPropertyHandle> SettingsHandle = LayoutBuilder.GetProperty(FName("UMeshMergingSettingsObject.Settings"));
	
	FName MeshCategory("MeshSettings");
	IDetailCategoryBuilder& MeshCategoryBuilder = LayoutBuilder.EditCategory(MeshCategory);

	TArray<TSharedRef<IPropertyHandle>> SimpleDefaultProperties;
	MeshCategoryBuilder.GetDefaultProperties(SimpleDefaultProperties, true, true);
	MeshCategoryBuilder.AddProperty(SettingsHandle);

	FName CategoryMetaData("Category");
	for (TSharedRef<IPropertyHandle> Property: SimpleDefaultProperties)
	{
		const FString& CategoryName = Property->GetMetaData(CategoryMetaData);

		IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(*CategoryName);
		IDetailPropertyRow& PropertyRow = CategoryBuilder.AddProperty(Property);

		if (Property->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FMeshMergingSettings, SpecificLOD))
		{
			static const FName EditConditionName = "EnumCondition";
			int32 EnumCondition = Property->GetIntMetaData(EditConditionName);
			PropertyRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FMeshMergingSettingsObjectCustomization::ArePropertiesVisible, EnumCondition)));
		}
		else if (Property->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FMeshMergingSettings, LODSelectionType))
		{
			EnumProperty = Property;
						
			TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShareable(new FPropertyRestriction(LOCTEXT("NoSupport","Unable to support this option in Merge Actor")));
			const UEnum* const MeshLODSelectionTypeEnum = StaticEnum<EMeshLODSelectionType>();		
			EnumRestriction->AddDisabledValue(MeshLODSelectionTypeEnum->GetNameStringByValue((uint8)EMeshLODSelectionType::CalculateLOD));
			EnumProperty->AddRestriction(EnumRestriction.ToSharedRef());
		}
	}

	FName MaterialCategory("MaterialSettings");
	IDetailCategoryBuilder& MaterialCategoryBuilder = LayoutBuilder.EditCategory(MaterialCategory);
	SimpleDefaultProperties.Empty();
	MaterialCategoryBuilder.GetDefaultProperties(SimpleDefaultProperties, true, true);

	for (TSharedRef<IPropertyHandle> Property : SimpleDefaultProperties)
	{
		const FString& CategoryName = Property->GetMetaData(CategoryMetaData);

		IDetailCategoryBuilder& CategoryBuilder = LayoutBuilder.EditCategory(*CategoryName);
		IDetailPropertyRow& PropertyRow = CategoryBuilder.AddProperty(Property);

		// Disable material settings if we are exporting all LODs (no support for material baking in this case)
		if (CategoryName.Compare("MaterialSettings") == 0)
		{
			PropertyRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FMeshMergingSettingsObjectCustomization::AreMaterialPropertiesEnabled)));
		}
	}
}

TSharedRef<IDetailCustomization> FMeshMergingSettingsObjectCustomization::MakeInstance()
{
	return MakeShareable(new FMeshMergingSettingsObjectCustomization);
}

EVisibility FMeshMergingSettingsObjectCustomization::ArePropertiesVisible(const int32 VisibleType) const
{
	uint8 CurrentEnumValue = 0;
	EnumProperty->GetValue(CurrentEnumValue);
	return (CurrentEnumValue == VisibleType) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FMeshMergingSettingsObjectCustomization::AreMaterialPropertiesEnabled() const
{
	uint8 CurrentEnumValue = 0;
	EnumProperty->GetValue(CurrentEnumValue);

	return !(CurrentEnumValue == (uint8)EMeshLODSelectionType::AllLODs);
}

TSharedRef<IPropertyTypeCustomization> FMeshMergingSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMeshMergingSettingsCustomization);
}

void FMeshMergingSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
			StructPropertyHandle->CreatePropertyNameWidget(StructPropertyHandle->GetPropertyDisplayName())
	]
	.ValueContent()
	[
			StructPropertyHandle->CreatePropertyValueWidget(false)
	];
}

void FMeshMergingSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren = 0;
	
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		IDetailPropertyRow& NewRow = ChildBuilder.AddProperty(ChildHandle);

		AddResetToDefaultOverrides(NewRow);
	}
}

#undef LOCTEXT_NAMESPACE
