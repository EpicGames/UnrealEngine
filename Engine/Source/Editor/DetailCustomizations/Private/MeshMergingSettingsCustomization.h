// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMerge/MeshMergingSettings.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "OverrideResetToDefault.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailLayoutBuilder;
class IPropertyHandle;

class FMeshMergingSettingsObjectCustomization : public IDetailCustomization
{
public:
	~FMeshMergingSettingsObjectCustomization();

	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	/** End IDetailCustomization interface */

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
protected:
	EVisibility ArePropertiesVisible(const int32 VisibleType) const;
	bool AreMaterialPropertiesEnabled() const;
	TSharedPtr<IPropertyHandle> EnumProperty;
private:
};


class FMeshMergingSettingsCustomization : public IPropertyTypeCustomization, public TOverrideResetToDefaultWithStaticUStruct<FMeshMergingSettings>
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};
