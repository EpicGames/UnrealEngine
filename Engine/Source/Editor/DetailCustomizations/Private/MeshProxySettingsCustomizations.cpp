// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProxySettingsCustomizations.h"

#include "Algo/AnyOf.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#define LOCTEXT_NAMESPACE "MeshProxySettingsCustomizations"

void FMeshProxySettingsCustomizations::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}


TSharedRef<IPropertyTypeCustomization> FMeshProxySettingsCustomizations::MakeInstance()
{
	return MakeShareable(new FMeshProxySettingsCustomizations);
}


bool FMeshProxySettingsCustomizations::UseNativeProxyLODTool() const
{
	IMeshMerging* MergeModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetMeshMergingInterface();
	return MergeModule && MergeModule->GetName().Equals("ProxyLODMeshMerging");
}

void FMeshProxySettingsCustomizations::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	TArray<UObject*> OutersList;
	StructPropertyHandle->GetOuterObjects(OutersList);
	bIsEditingHLODLayer = Algo::AnyOf(OutersList, [](UObject* Outer) { return Outer->IsInA(UHLODLayer::StaticClass()); });

	// Determine if we are using our native module  If so, we will suppress some of the options used by the current thirdparty tool (simplygon).

	IMeshReductionManagerModule& ModuleManager = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	IMeshMerging* MergeModule = ModuleManager.GetMeshMergingInterface();

	// Respect the ShowInnerProperties property from the struct property.
	static const FName ShowOnlyInners("ShowOnlyInnerProperties");
	bool bCreateSettingsGroup = !StructPropertyHandle->HasMetaData(ShowOnlyInners);
	IDetailGroup* MeshSettingsGroup = nullptr;

	if (bCreateSettingsGroup)
	{
		MeshSettingsGroup = &ChildBuilder.AddGroup(NAME_None, FText::FromString("Proxy Settings"));
	}

	auto AddPropertyToGroup = [bCreateSettingsGroup, &ChildBuilder, &MeshSettingsGroup](const TSharedRef<IPropertyHandle>& prop) -> IDetailPropertyRow& {
		if (bCreateSettingsGroup)
		{
			return MeshSettingsGroup->AddPropertyRow(prop);
		}
		else
		{
			return ChildBuilder.AddProperty(prop);
		}
	};


	TSharedPtr< IPropertyHandle > HardAngleThresholdPropertyHandle        = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, HardAngleThreshold));
	TSharedPtr< IPropertyHandle > NormalCalcMethodPropertyHandle          = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, NormalCalculationMethod));
	TSharedPtr< IPropertyHandle > MaxRayCastDistdPropertyHandle           = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, MaxRayCastDist));
	TSharedPtr< IPropertyHandle > RecalculateNormalsPropertyHandle        = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, bRecalculateNormals));
	TSharedPtr< IPropertyHandle > UseLandscapeCullingPropertyHandle       = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, bUseLandscapeCulling));
	TSharedPtr< IPropertyHandle > LandscapeCullingPrecisionPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, LandscapeCullingPrecision));
	TSharedPtr< IPropertyHandle > MergeDistanceHandle                     = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, MergeDistance));
	TSharedPtr< IPropertyHandle > UnresolvedGeometryColorHandle           = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, UnresolvedGeometryColor));
	TSharedPtr< IPropertyHandle > VoxelSizeHandle                         = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, VoxelSize));
	TSharedPtr< IPropertyHandle > ScreenSizeHandle						  = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FMeshProxySettings, ScreenSize));

	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// Handle special property cases (done inside the loop to maintain order according to the struct
		if (Iter.Value() == HardAngleThresholdPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.ToolTip(FText::FromString(FString("Angle at which a hard edge is introduced between faces.  Note: Increases vertex count and may introduce additional UV seams.  It is only recommended if not using normals maps")));
			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsHardAngleThresholdVisible));
		}
		else if (Iter.Value() == NormalCalcMethodPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsNormalCalcMethodVisible));
		}
		else if (Iter.Value() == MaxRayCastDistdPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsSearchDistanceVisible));
		}
		else if (Iter.Value() == RecalculateNormalsPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsRecalculateNormalsVisible));
		}
		else if (Iter.Value() == UseLandscapeCullingPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.DisplayName(FText::FromString(FString("Enable Volume Culling")));
			MeshProxySettingsRow.ToolTip(FText::FromString(FString("Allow culling volumes to exclude geometry.")));
			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsUseLandscapeCullingVisible));
		}
		else if (Iter.Value() == LandscapeCullingPrecisionPropertyHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsUseLandscapeCullingPrecisionVisible));
		}
		else if (Iter.Value() == MergeDistanceHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsMergeDistanceVisible));
		}
		else if (Iter.Value() == UnresolvedGeometryColorHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsUnresolvedGeometryColorVisible));
		}
		else if (Iter.Value() == VoxelSizeHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsVoxelSizeVisible));
		}
		else if (Iter.Value() == ScreenSizeHandle)
		{
			IDetailPropertyRow& MeshProxySettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(MeshProxySettingsRow);

			MeshProxySettingsRow.Visibility(TAttribute<EVisibility>(this, &FMeshProxySettingsCustomizations::IsScreenSizeVisible));
		}
		else
		{
			IDetailPropertyRow& SettingsRow = AddPropertyToGroup(Iter.Value().ToSharedRef());
			AddResetToDefaultOverrides(SettingsRow);
		}
	}
}

EVisibility FMeshProxySettingsCustomizations::IsThirdPartySpecificVisible() const
{
	// Static assignment.  The tool can only change during editor restart.
	static bool bUseNativeTool = UseNativeProxyLODTool();

	if (bUseNativeTool)
	{
		return EVisibility::Hidden;
	}

	return EVisibility::Visible;
}

EVisibility FMeshProxySettingsCustomizations::IsProxyLODSpecificVisible() const
{
	// Static assignment.  The tool can only change during editor restart.
	static bool bUseNativeTool = UseNativeProxyLODTool();

	if (bUseNativeTool)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

EVisibility FMeshProxySettingsCustomizations::IsHardAngleThresholdVisible() const
{
	// Only proxyLOD actually uses this setting.  Historically, it has been exposed for
	// simplygon, but it was not actually connected!
	return IsProxyLODSpecificVisible();
}

EVisibility FMeshProxySettingsCustomizations::IsNormalCalcMethodVisible() const
{
	// Only ProxyLOD
	return IsProxyLODSpecificVisible();
}

EVisibility FMeshProxySettingsCustomizations::IsRecalculateNormalsVisible() const
{
	return IsThirdPartySpecificVisible();
}

EVisibility FMeshProxySettingsCustomizations::IsUseLandscapeCullingVisible() const
{
	return EVisibility::Visible;
}

EVisibility FMeshProxySettingsCustomizations::IsUseLandscapeCullingPrecisionVisible() const
{
	return IsThirdPartySpecificVisible();
}

EVisibility FMeshProxySettingsCustomizations::IsMergeDistanceVisible() const
{
	return EVisibility::Visible;
}
EVisibility FMeshProxySettingsCustomizations::IsUnresolvedGeometryColorVisible() const
{   // visible for proxylod but not third party tool (e.g. simplygon)
	return IsProxyLODSpecificVisible();
}
EVisibility FMeshProxySettingsCustomizations::IsSearchDistanceVisible() const
{
	return IsProxyLODSpecificVisible();
}
EVisibility FMeshProxySettingsCustomizations::IsVoxelSizeVisible() const
{
	return IsProxyLODSpecificVisible();
}
EVisibility FMeshProxySettingsCustomizations::IsScreenSizeVisible() const
{
	return bIsEditingHLODLayer ? EVisibility::Hidden : IsProxyLODSpecificVisible();
}




#undef LOCTEXT_NAMESPACE