// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "MeshMerge/MeshProxySettings.h"
#include "OverrideResetToDefault.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IPropertyHandle;

class FMeshProxySettingsCustomizations : public IPropertyTypeCustomization, public TOverrideResetToDefaultWithStaticUStruct<FMeshProxySettings>
{

public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	EVisibility IsHardAngleThresholdVisible() const;
	EVisibility IsRecalculateNormalsVisible() const;
	EVisibility IsUseLandscapeCullingVisible() const;
	EVisibility IsUseLandscapeCullingPrecisionVisible() const;
	EVisibility IsMergeDistanceVisible() const;
	EVisibility IsUnresolvedGeometryColorVisible() const;
	EVisibility IsVoxelSizeVisible() const;
	EVisibility IsScreenSizeVisible() const;	
	EVisibility IsNormalCalcMethodVisible() const;
	EVisibility IsSearchDistanceVisible() const;

	EVisibility IsThirdPartySpecificVisible() const;
	EVisibility IsProxyLODSpecificVisible() const;
	bool UseNativeProxyLODTool() const;

	bool bIsEditingHLODLayer;
};
