// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MobilityCustomization.h"

#include "RenderUtils.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Text/STextBlock.h"

TAutoConsoleVariable<bool> CVarHideLightStaticMobilityWhenStaticLightingDisabled(
	TEXT("r.Editor.HideLightStaticMobilityWhenStaticLightingDisabled"),
	false,
	TEXT("Hide Static mobility on Light components when project has Static Lighting disabled."),
	ECVF_RenderThreadSafe);

#define LOCTEXT_NAMESPACE "MobilityCustomization" 

FMobilityCustomization::FMobilityCustomization(TSharedPtr<IPropertyHandle> InMobilityHandle, uint8 InRestrictedMobilityBits, bool InForLight)
{
	MobilityHandle = InMobilityHandle;
	MobilityHandle->MarkHiddenByCustomization();
	RestrictedMobilityBits = InRestrictedMobilityBits;
	bForLight = InForLight;
}

FName FMobilityCustomization::GetName() const
{
	const FProperty* Property = MobilityHandle->GetProperty();
	if (Property != nullptr)
	{
		return Property->GetFName();
	}
	return NAME_None;
}

void FMobilityCustomization::GenerateHeaderRowContent(FDetailWidgetRow& WidgetRow)
{
	TSharedRef<SSegmentedControl<EComponentMobility::Type>> ButtonOptionsPanel =
		SNew(SSegmentedControl<EComponentMobility::Type>)
		.Value(this, &FMobilityCustomization::GetActiveMobility)
		.OnValueChanged(this, &FMobilityCustomization::OnMobilityChanged);
		
	WidgetRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Mobility", "Mobility"))
		.ToolTipText(this, &FMobilityCustomization::GetMobilityToolTip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(0)
	[
		ButtonOptionsPanel
	]
	.FilterString(LOCTEXT("Mobility", "Mobility"));

	bool bShowStatic = !( RestrictedMobilityBits & StaticMobilityBitMask );
	bool bShowStationary = !( RestrictedMobilityBits & StationaryMobilityBitMask );

	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	if (bForLight && CVarHideLightStaticMobilityWhenStaticLightingDisabled.GetValueOnGameThread())
	{
		bShowStatic &= bAllowStaticLighting;
	}

	int32 ColumnIndex = 0;

	if ( bShowStatic )
	{
		FText StaticTooltip = bForLight
			? LOCTEXT("Mobility_Static_Light_Tooltip", "A static light can't be changed in game.\n* Fully Baked Lighting\n* Fastest Rendering")
			: LOCTEXT("Mobility_Static_Tooltip", "A static object can't be changed in game.\n* Allows Baked Lighting\n* Fastest Rendering");

		// Static Mobility
		ButtonOptionsPanel->AddSlot(EComponentMobility::Static)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.MobilityFont"))
			.Text(LOCTEXT("Static", "Static"))
		]
		.ToolTip(StaticTooltip);
	}

	// Stationary Mobility
	if ( bShowStationary )
	{
		FText StationaryTooltip = bForLight
			? LOCTEXT("Mobility_Stationary_Tooltip", "A stationary light will only have its shadowing and bounced lighting from static geometry baked by Lightmass, all other lighting will be dynamic.  It can change color and intensity in game.\n* Can't Move\n* Allows Partially Baked Lighting\n* Dynamic Shadows from Movable objects")
			: LOCTEXT("Mobility_Stationary_Object_Tooltip", "A stationary object can be changed in game but not moved, and enables cached lighting methods. \n* Cached Dynamic Shadows.");

		ButtonOptionsPanel->AddSlot(EComponentMobility::Stationary)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.MobilityFont"))
			.Text(LOCTEXT("Stationary", "Stationary"))
		]
		.ToolTip(StationaryTooltip);
	}

	FText MovableTooltip = bForLight
			? LOCTEXT("Mobility_Movable_Light_Tooltip", "Movable lights can be moved and changed in game.\n* Totally Dynamic\n* Whole Scene Dynamic Shadows\n* Slowest Rendering")
			: LOCTEXT("Mobility_Movable_Tooltip", "Movable objects can be moved and changed in game.\n* Totally Dynamic\n* Casts a Dynamic Shadow \n* Slowest Rendering");

	// Movable Mobility
	ButtonOptionsPanel->AddSlot(EComponentMobility::Movable)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.MobilityFont"))
		.Text(LOCTEXT("Movable", "Movable"))
	]
	.ToolTip(MovableTooltip);

	ButtonOptionsPanel->RebuildChildren();
}

EComponentMobility::Type FMobilityCustomization::GetActiveMobility() const
{
	if (MobilityHandle.IsValid())
	{
		uint8 MobilityByte;
		MobilityHandle->GetValue(MobilityByte);

		return (EComponentMobility::Type)MobilityByte;
	}

	return EComponentMobility::Static;
}

FSlateColor FMobilityCustomization::GetMobilityTextColor(EComponentMobility::Type InMobility) const
{
	if (MobilityHandle.IsValid())
	{
		uint8 MobilityByte;
		MobilityHandle->GetValue(MobilityByte);

		return MobilityByte == InMobility ? FSlateColor(FLinearColor(0, 0, 0)) : FSlateColor(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));
	}

	return FSlateColor(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));
}

void FMobilityCustomization::OnMobilityChanged(EComponentMobility::Type InMobility)
{
	if (MobilityHandle.IsValid())
	{
		MobilityHandle->SetValue((uint8)InMobility);
	}
}

FText FMobilityCustomization::GetMobilityToolTip() const
{
	if (MobilityHandle.IsValid())
	{
		return MobilityHandle->GetToolTipText();
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE

