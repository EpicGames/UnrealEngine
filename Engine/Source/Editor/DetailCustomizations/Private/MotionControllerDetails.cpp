// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionControllerDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "MotionControllerComponent.h"
#include "MotionControllerSourceCustomization.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "MotionControllerDetails"

TSharedRef<IDetailCustomization> FMotionControllerDetails::MakeInstance()
{
	return MakeShareable(new FMotionControllerDetails);
}

void FMotionControllerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	MotionSourceProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UMotionControllerComponent, MotionSource));
	MotionSourceProperty->MarkHiddenByCustomization();

	IDetailCategoryBuilder& MotionSourcePropertyGroup = DetailLayout.EditCategory(*MotionSourceProperty->GetMetaData(TEXT("Category")));
	MotionSourcePropertyGroup.AddCustomRow(LOCTEXT("MotionSourceLabel", "Motion Source"))
	.NameContent()
	[
		MotionSourceProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SMotionSourceWidget)
		.OnGetMotionSourceText(this, &FMotionControllerDetails::GetMotionSourceValueText)
		.OnMotionSourceChanged(this, &FMotionControllerDetails::OnMotionSourceChanged)
	];
}

void FMotionControllerDetails::OnMotionSourceChanged(FName NewMotionSource)
{
	MotionSourceProperty->SetValue(NewMotionSource);
}

FText FMotionControllerDetails::GetMotionSourceValueText() const
{
	FName MotionSource;
	MotionSourceProperty->GetValue(MotionSource);
	return FText::FromName(MotionSource);
}

#undef LOCTEXT_NAMESPACE
