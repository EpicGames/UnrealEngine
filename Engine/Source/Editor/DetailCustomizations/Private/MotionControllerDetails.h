// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

class FMotionControllerDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:

	TSharedPtr<IPropertyHandle> MotionSourceProperty;

	// Delegate handler for when UI changes motion source
	void OnMotionSourceChanged(FName NewMotionSource);

	// Return text for motion source combo box
	FText GetMotionSourceValueText() const;
};
