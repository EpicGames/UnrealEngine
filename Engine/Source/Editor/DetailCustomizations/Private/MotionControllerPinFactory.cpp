// Copyright Epic Games, Inc. All Rights Reserved.
#include "MotionControllerPinFactory.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_VariableSet.h"
#include "Misc/AssertionMacros.h"
#include "MotionControllerComponent.h"
#include "MotionControllerSourceCustomization.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

class SMotionControllerSourceGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SMotionControllerSourceGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		return	SNew(SMotionSourceWidget)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.OnMotionSourceChanged(this, &SMotionControllerSourceGraphPin::OnMotionSourceChanged)
			.OnGetMotionSourceText(this, &SMotionControllerSourceGraphPin::GetMotionSourceValueText);
	}
	//~ End SGraphPin Interface

private:

	void OnMotionSourceChanged(FName NewMotionSource)
	{
		FString NewMotionSourceString = NewMotionSource.ToString();

		if (!GraphPinObj->GetDefaultAsString().Equals(NewMotionSourceString))
		{
			const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeMotionSourcePinValue", "Change Motion Source Pin Value"));
			GraphPinObj->Modify();

			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewMotionSourceString);
		}
	}

	FText GetMotionSourceValueText() const
	{
		return FText::FromString(GraphPinObj->GetDefaultAsString());
	}

};

TSharedPtr<class SGraphPin> FMotionControllerPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		if (UK2Node_VariableSet* Setter = Cast<UK2Node_VariableSet>(InPin->GetOuter()))
		{
			if (Setter->GetVariableSourceClass() == UMotionControllerComponent::StaticClass() && Setter->GetVarName() == GET_MEMBER_NAME_CHECKED(UMotionControllerComponent, MotionSource))
			{
				return SNew(SMotionControllerSourceGraphPin, InPin);
			}
		}
	}

	return nullptr;
}
