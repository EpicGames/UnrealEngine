// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "Templates/SharedPointer.h"

class FMotionControllerPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;
};