// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionControllerSourceCustomization.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "IMotionController.h"
#include "Input/Events.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

void SMotionSourceWidget::Construct(const FArguments& InArgs)
{
	check(InArgs._OnMotionSourceChanged.IsBound());
	check(InArgs._OnGetMotionSourceText.IsBound());

	OnMotionSourceChanged = InArgs._OnMotionSourceChanged;
	OnGetMotionSourceText = InArgs._OnGetMotionSourceText;

	this->ChildSlot
	[
		SNew(SComboButton)
		.ContentPadding(1.f)
		.OnGetMenuContent(this, &SMotionSourceWidget::BuildMotionSourceMenu)
		.ButtonContent()
		[
			SNew(SEditableTextBox)
			.RevertTextOnEscape(true)
			.SelectAllTextWhenFocused(true)
			.Text(this, &SMotionSourceWidget::GetMotionSourceText)
			.OnTextCommitted(this, &SMotionSourceWidget::OnMotionSourceValueTextComitted)
		]
	];
}

TSharedRef<SWidget> SMotionSourceWidget::BuildMotionSourceMenu()
{
	TMap<FName, TArray<FName>> CategorizedMotionSources;

	TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
	for (IMotionController* MotionController : MotionControllers)
	{
		if (MotionController)
		{
			TArray<FMotionControllerSource> MotionSources;
			MotionController->EnumerateSources(MotionSources);

			for (FMotionControllerSource& Source : MotionSources)
			{
				CategorizedMotionSources.FindOrAdd(Source.EditorCategory).AddUnique(Source.SourceName);
			}
		}
	}

	FMenuBuilder MenuBuilder(true, NULL);

	for (TPair<FName, TArray<FName>> Pair : CategorizedMotionSources)
	{
		const bool bWrapInSection = !Pair.Key.IsNone();
		if (bWrapInSection)
		{
			MenuBuilder.BeginSection(Pair.Key, FText::FromName(Pair.Key));
		}

		for (FName SourceName : Pair.Value)
		{
			FUIAction MenuAction(FExecuteAction::CreateSP(this, &SMotionSourceWidget::OnMotionSourceComboValueCommited, SourceName));
			MenuBuilder.AddMenuEntry(FText::FromName(SourceName), FText(), FSlateIcon(), MenuAction);
		}

		if (bWrapInSection)
		{
			MenuBuilder.EndSection();
		}
	}

	return MenuBuilder.MakeWidget();
}

FText SMotionSourceWidget::GetMotionSourceText() const
{
	return OnGetMotionSourceText.Execute();
}

void SMotionSourceWidget::OnMotionSourceValueTextComitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FName NewMotionSource = *InNewText.ToString();
	OnMotionSourceChanged.Execute(NewMotionSource);
}

void SMotionSourceWidget::OnMotionSourceComboValueCommited(FName InMotionSource)
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
	OnMotionSourceChanged.Execute(InMotionSource);
}
