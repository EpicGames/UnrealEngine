// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

DECLARE_DELEGATE_RetVal(FText, FOnGetMotionSourceText)
DECLARE_DELEGATE_OneParam(FOnMotionSourceChanged, FName)

class SMotionSourceWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMotionSourceWidget)
	{}
		SLATE_EVENT(FOnGetMotionSourceText, OnGetMotionSourceText)
		SLATE_EVENT(FOnMotionSourceChanged, OnMotionSourceChanged)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

private:

	// Generate the motion source combo box popup menu
	TSharedRef<SWidget> BuildMotionSourceMenu();

	// Handler for setting combo box text
	FText GetMotionSourceText() const;

	// Handler new values from the motion source combo box
	void OnMotionSourceValueTextComitted(const FText& InNewText, ETextCommit::Type InTextCommit);
	void OnMotionSourceComboValueCommited(FName InMotionSource);

	// Delegates for interaction with source
	FOnGetMotionSourceText OnGetMotionSourceText;
	FOnMotionSourceChanged OnMotionSourceChanged;
};
