// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePlayerSettingsDetails.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/Dialogs.h"
#include "EditorDirectories.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Paths.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SourceControlHelpers.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Notifications/SNotificationList.h"


#define LOCTEXT_NAMESPACE "MoviePlayerSettingsDetails"


FMoviePlayerSettingsDetails::~FMoviePlayerSettingsDetails()
{
}

TSharedRef<IDetailCustomization> FMoviePlayerSettingsDetails::MakeInstance()
{
	return MakeShareable(new FMoviePlayerSettingsDetails);
}


void FMoviePlayerSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	IDetailCategoryBuilder& MoviesCategory = DetailLayout.EditCategory("Movies");

	StartupMoviesPropertyHandle = DetailLayout.GetProperty("StartupMovies");

	TSharedRef<FDetailArrayBuilder> StartupMoviesBuilder = MakeShareable( new FDetailArrayBuilder( StartupMoviesPropertyHandle.ToSharedRef() ) );
	StartupMoviesBuilder->OnGenerateArrayElementWidget( FOnGenerateArrayElementWidget::CreateSP(this, &FMoviePlayerSettingsDetails::GenerateArrayElementWidget) );

	MoviesCategory.AddProperty( "bWaitForMoviesToComplete" );
	MoviesCategory.AddProperty( "bMoviesAreSkippable" );

	const bool bForAdvanced = false;
	MoviesCategory.AddCustomBuilder( StartupMoviesBuilder, bForAdvanced );
}


void FMoviePlayerSettingsDetails::GenerateArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& FilePathRow = ChildrenBuilder.AddProperty( PropertyHandle );
	{
		FilePathRow.CustomWidget(false)
			.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(0.0f)
			.MinDesiredWidth(125.0f)
			[
				SNew(SFilePathPicker)
					.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
					.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
					.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
					.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
					.FilePath(this, &FMoviePlayerSettingsDetails::HandleFilePathPickerFilePath, PropertyHandle)
					.FileTypeFilter(TEXT("MPEG-4 Movie (*.mp4)|*.mp4|Web Movie (*.webm)|*.webm"))
					.OnPathPicked(this, &FMoviePlayerSettingsDetails::HandleFilePathPickerPathPicked, PropertyHandle)
			];
	}
};


void FMoviePlayerSettingsDetails::HandleFilePathPickerPathPicked( const FString& PickedPath, TSharedRef<IPropertyHandle> Property )
{
	// Ignore if value is unchanged.
	FString OldValue;
	if (Property->GetValueAsFormattedString(OldValue) == FPropertyAccess::Success)
	{
		if (PickedPath == OldValue)
		{
			return;
		}
	}

	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(PickedPath));

	// sanitize the location of the chosen movies to the content/movies directory
	const FString MoviesBaseDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir() + TEXT("Movies/"));
	const FString FullPath = FPaths::ConvertRelativePathToFull(PickedPath);

	if (FullPath.StartsWith(MoviesBaseDir))
	{
		// mark for add/checkout
		FText FailReason;
		
		if (SourceControlHelpers::CheckoutOrMarkForAdd(PickedPath, LOCTEXT("MovieFileDescription", "movie"), FOnPostCheckOut(), FailReason))
		{
			// already in the movies dir, so just trim the path so we just have a partial path with no extension (the movie player expects this)
			Property->SetValue(FPaths::GetBaseFilename(FullPath.RightChop(MoviesBaseDir.Len())));
		}
		else
		{
			FNotificationInfo Info(FailReason);
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
	else if (!PickedPath.IsEmpty())
	{
		// If the path of PickedPath is empty then we are dealing with a movie already in the correct directory.
		if (FPaths::GetPath(PickedPath).IsEmpty())
		{
			FString FullPickedPath = FPaths::Combine(MoviesBaseDir, PickedPath);

			// Look for this movie.
			TArray<FString> ExistingMovieFiles;
			IFileManager::Get().FindFiles(ExistingMovieFiles, *MoviesBaseDir);
			bool bHasValidMovie = ExistingMovieFiles.ContainsByPredicate(
				[&PickedPath](const FString& ExistingMovie)
			{
				return ExistingMovie.Contains(PickedPath);
			});

			// Did we find a movie?
			if (bHasValidMovie)
			{
				// Yes.
				Property->SetValue(FPaths::GetBaseFilename(PickedPath));
			}
			else
			{
				// Nope. Bring up a dialog box informing the user.
				FSuppressableWarningDialog::FSetupInfo Info(
					LOCTEXT("ExternalMovieImportNotExistWarning", "This movie does not exist."),
					LOCTEXT("ExternalMovieImportNotExistTitle", "Does not exist"),
					TEXT("ImportMovieIntoProjectNotExist"));
				Info.ConfirmText = LOCTEXT("ExternalMovieImportNotExist_Confirm", "OK");
				Info.CancelText = LOCTEXT("ExternalMovieImportNotExist_Cancel", "Cancel");

				FSuppressableWarningDialog ImportWarningDialog(Info);
				if (ImportWarningDialog.ShowModal() == FSuppressableWarningDialog::EResult::Cancel)
				{
					Property->SetValue(FPaths::GetBaseFilename(PickedPath));
				}
			}
		}
		else
		{
			// ask the user if they want to import this movie
			FSuppressableWarningDialog::FSetupInfo Info(
				LOCTEXT("ExternalMovieImportWarning", "This movie needs to be copied into your project, would you like to copy the file now?"),
				LOCTEXT("ExternalMovieImportTitle", "Copy Movie"),
				TEXT("ImportMovieIntoProject"));
			Info.ConfirmText = LOCTEXT("ExternalMovieImport_Confirm", "Copy");
			Info.CancelText = LOCTEXT("ExternalMovieImport_Cancel", "Don't Copy");

			FSuppressableWarningDialog ImportWarningDialog(Info);

			if (ImportWarningDialog.ShowModal() != FSuppressableWarningDialog::EResult::Cancel)
			{
				const FString FileName = FPaths::GetCleanFilename(PickedPath);
				const FString DestPath = MoviesBaseDir / FileName;

				FText FailReason;

				if (SourceControlHelpers::CopyFileUnderSourceControl(DestPath, PickedPath, LOCTEXT("MovieFileDescription", "movie"), FailReason))
				{
					// trim the path so we just have a partial path with no extension (the movie player expects this)
					Property->SetValue(FPaths::GetBaseFilename(DestPath.RightChop(MoviesBaseDir.Len())));
				}
				else
				{
					FNotificationInfo FailureInfo(FailReason);
					FailureInfo.ExpireDuration = 3.0f;
					FSlateNotificationManager::Get().AddNotification(FailureInfo);
				}
			}
		}
	}
	else
	{
		Property->SetValue(PickedPath);
	}
}


FString FMoviePlayerSettingsDetails::HandleFilePathPickerFilePath( TSharedRef<IPropertyHandle> Property ) const
{
	FString FilePath;
	Property->GetValue(FilePath);

	return FilePath;
}


#undef LOCTEXT_NAMESPACE
