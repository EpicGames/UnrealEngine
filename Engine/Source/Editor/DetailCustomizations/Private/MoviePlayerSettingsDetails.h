// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;

class FMoviePlayerSettingsDetails : public IDetailCustomization
{
public:
	~FMoviePlayerSettingsDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:

	/** Generate a widget for a movie array element */
	void GenerateArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/** Callback for getting the selected path in the picker widget. */
	FString HandleFilePathPickerFilePath( TSharedRef<IPropertyHandle> Property ) const;

	/** Delegate handler for when a new movie path is picked */
	void HandleFilePathPickerPathPicked( const FString& InOutPath, TSharedRef<IPropertyHandle> Property );

private:
	/** Handle to the movies array property */
	TSharedPtr<IPropertyHandle> StartupMoviesPropertyHandle;
};
