// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneBindingOverrideDataCustomization.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneBindingOverrides.h"
#include "MovieSceneBindingOwnerInterface.h"
#include "PropertyHandle.h"

class UObject;

#define LOCTEXT_NAMESPACE "MovieSceneBindingOverrideDataCustomization"

void FMovieSceneBindingOverrideDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	StructProperty = PropertyHandle;

	ObjectProperty = StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneBindingOverrideData, Object));

	check(ObjectProperty.IsValid());

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		StructProperty->CreatePropertyValueWidget(false)
	];
}

void FMovieSceneBindingOverrideDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	IMovieSceneBindingOwnerInterface* Interface = GetInterface();
	if (!Interface)
	{
		return;
	}

	ChildBuilder.AddProperty(StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneBindingOverrideData, ObjectBindingId)).ToSharedRef());

	ObjectPickerProxy = Interface->GetObjectPickerProxy(ObjectProperty);
	if (ObjectPickerProxy.IsValid())
	{
		// When any property is changed in the proxy sruct, we update the object mapping (OnGetObjectFromProxy)
		for (TSharedPtr<IPropertyHandle> ChildHandle : StructProperty->AddChildStructure(ObjectPickerProxy.ToSharedRef()))
		{
			ChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMovieSceneBindingOverrideDataCustomization::OnGetObjectFromProxy));
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}

	ChildBuilder.AddProperty(StructProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneBindingOverrideData, bOverridesDefault)).ToSharedRef());
}

IMovieSceneBindingOwnerInterface* FMovieSceneBindingOverrideDataCustomization::GetInterface() const
{
	TArray<UObject*> OuterObjects;
	StructProperty->GetOuterObjects(OuterObjects);

	return OuterObjects.Num() == 1 ? IMovieSceneBindingOwnerInterface::FindFromObject(OuterObjects[0]) : nullptr;
}

UMovieSceneSequence* FMovieSceneBindingOverrideDataCustomization::GetSequence() const
{
	IMovieSceneBindingOwnerInterface* OwnerInterface = GetInterface();
	return OwnerInterface ? OwnerInterface->RetrieveOwnedSequence() : nullptr;
}

void FMovieSceneBindingOverrideDataCustomization::OnGetObjectFromProxy()
{
	IMovieSceneBindingOwnerInterface* Interface = GetInterface();
	if (Interface)
	{
		Interface->UpdateObjectFromProxy(*ObjectPickerProxy, *ObjectProperty);
	}
}

#undef LOCTEXT_NAMESPACE
