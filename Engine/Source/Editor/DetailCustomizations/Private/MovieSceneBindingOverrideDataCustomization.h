// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class FStructOnScope;
class IMovieSceneBindingOwnerInterface;
class IPropertyHandle;
class UMovieSceneSequence;


class FMovieSceneBindingOverrideDataCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShareable(new FMovieSceneBindingOverrideDataCustomization); }

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void OnGetObjectFromProxy();

	IMovieSceneBindingOwnerInterface* GetInterface() const;
	UMovieSceneSequence* GetSequence() const;

	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> ObjectProperty;
	TSharedPtr<FStructOnScope> ObjectPickerProxy;
};