// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCaptureCustomization.h"

#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneCapture.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

class UObject;

TSharedRef<IDetailCustomization> FMovieSceneCaptureCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneCaptureCustomization);
}

FMovieSceneCaptureCustomization::FMovieSceneCaptureCustomization()
{
	ObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FMovieSceneCaptureCustomization::OnObjectsReplaced);
	PropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FMovieSceneCaptureCustomization::OnObjectPostEditChange);
}

FMovieSceneCaptureCustomization::~FMovieSceneCaptureCustomization()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PropertyChangedHandle);
	FCoreUObjectDelegates::OnObjectsReplaced.Remove(ObjectsReplacedHandle);
}

void FMovieSceneCaptureCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PropertyUtilities = DetailBuilder.GetPropertyUtilities();
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
}

void FMovieSceneCaptureCustomization::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	static bool bQueuedRefresh = false;

	if (bQueuedRefresh)
	{
		return;
	}
	bQueuedRefresh = true;

	// Defer the update 1 frame to ensure that we don't end up in a recursive loop adding bindings to the OnObjectsReplaced delegate that is currently being triggered
	// (since the bindings are added in FMovieSceneCaptureCustomization::CustomizeDetails)
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateLambda([LocalPropertyUtilities = PropertyUtilities] { bQueuedRefresh = false; LocalPropertyUtilities->ForceRefresh(); }));
}

void FMovieSceneCaptureCustomization::OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent )
{
	if (ObjectsBeingCustomized.Contains(Object))
	{
		static FName ImageCaptureProtocolTypeName = GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, ImageCaptureProtocolType);
		static FName ImageCaptureProtocolName     = GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, ImageCaptureProtocol);
		static FName AudioCaptureProtocolTypeName = GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, AudioCaptureProtocolType);
		static FName AudioCaptureProtocolName	  = GET_MEMBER_NAME_CHECKED(UMovieSceneCapture, AudioCaptureProtocol);

		FName PropertyName = PropertyChangedEvent.GetPropertyName();
		if (PropertyName == ImageCaptureProtocolTypeName || ( PropertyName == ImageCaptureProtocolName && PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet ) ||
			PropertyName == AudioCaptureProtocolTypeName || ( PropertyName == AudioCaptureProtocolName && PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet ))
		{
			// Defer the update 1 frame to ensure that we don't end up in a recursive loop adding bindings to the OnObjectPropertyChanged delegate that is currently being triggered
			// (since the bindings are added in FMovieSceneCaptureCustomization::CustomizeDetails)
			PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateLambda([LocalPropertyUtilities = PropertyUtilities]{ LocalPropertyUtilities->ForceRefresh(); }));
		}
	}
}
