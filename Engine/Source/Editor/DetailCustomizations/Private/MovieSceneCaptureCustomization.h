// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyUtilities;
class UObject;
struct FPropertyChangedEvent;

class FMovieSceneCaptureCustomization : public IDetailCustomization
{
public:
	FMovieSceneCaptureCustomization();
	~FMovieSceneCaptureCustomization();

	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:

	void OnObjectPostEditChange( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent );
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	TArray<TWeakObjectPtr<>> ObjectsBeingCustomized;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	FDelegateHandle PropertyChangedHandle;
	FDelegateHandle ObjectsReplacedHandle;
};
