// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneEvalOptionsCustomization.h"

#include "Containers/Array.h"
#include "IDetailChildrenBuilder.h"
#include "Misc/AssertionMacros.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "PropertyHandle.h"

TSharedRef<IPropertyTypeCustomization> FMovieSceneSectionEvalOptionsCustomization::MakeInstance()
{
	return MakeShared<FMovieSceneSectionEvalOptionsCustomization>();
}

void FMovieSceneSectionEvalOptionsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FMovieSceneSectionEvalOptionsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	bool bCanEditCompletionMode = !RawData.ContainsByPredicate(
		[](void* Ptr){
			return !static_cast<FMovieSceneSectionEvalOptions*>(Ptr)->bCanEditCompletionMode;
		}
	);

	TSharedPtr<IPropertyHandle> CompletionModeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneSectionEvalOptions, CompletionMode));
	if (bCanEditCompletionMode && CompletionModeHandle.IsValid())
	{
		ChildBuilder.AddProperty(CompletionModeHandle.ToSharedRef());
	}
}
