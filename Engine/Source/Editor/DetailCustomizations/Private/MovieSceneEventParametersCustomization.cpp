// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneEventParametersCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailsView.h"
#include "IPropertyUtilities.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/WidgetPath.h"
#include "Misc/Attribute.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneEventSection.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "MovieSceneEventParameters"

TSharedRef<IPropertyTypeCustomization> FMovieSceneEventParametersCustomization::MakeInstance()
{
	return MakeShared<FMovieSceneEventParametersCustomization>();
}

void FMovieSceneEventParametersCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();
}

void FMovieSceneEventParametersCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	if (RawData.Num() != 1)
	{
		return;
	}

	EditStructData = MakeShared<FStructOnScope>(nullptr);
	static_cast<FMovieSceneEventParameters*>(RawData[0])->GetInstance(*EditStructData);

	ChildBuilder.AddCustomRow(LOCTEXT("ParametersText", "Parameters"))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(CustomizationUtils.GetRegularFont())
		.Text(LOCTEXT("ParameterStructType", "Parameter Struct"))
	]
	.ValueContent()
	.VAlign(VAlign_Top)
	.MaxDesiredWidth(TOptional<float>())
	[
		SNew(SObjectPropertyEntryBox)
		.ObjectPath(EditStructData->GetStruct() ? EditStructData->GetStruct()->GetPathName() : FString())
		.AllowedClass(UScriptStruct::StaticClass())
		.OnObjectChanged(this, &FMovieSceneEventParametersCustomization::OnStructChanged)
	];

	if (EditStructData->GetStructMemory())
	{
		FSimpleDelegate OnEditStructChildContentsChangedDelegate = FSimpleDelegate::CreateSP(this, &FMovieSceneEventParametersCustomization::OnEditStructChildContentsChanged);

		TArray<TSharedPtr<IPropertyHandle>> ExternalHandles = ChildBuilder.AddAllExternalStructureProperties(EditStructData.ToSharedRef());
		for (TSharedPtr<IPropertyHandle> Handle : ExternalHandles)
		{
			Handle->SetOnPropertyValueChanged(OnEditStructChildContentsChangedDelegate);
			Handle->SetOnChildPropertyValueChanged(OnEditStructChildContentsChangedDelegate);
		}
	}

	if (TSharedPtr<const SWidget> DetailsView = ChildBuilder.GetParentCategory().GetParentLayout().GetDetailsViewSharedPtr())
	{
		PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateLambda(
			[DetailsView]
			{
				FWidgetPath WidgetPath;
				bool bFound = FSlateApplication::Get().FindPathToWidget(DetailsView.ToSharedRef(), WidgetPath);
				if (bFound)
				{
					FSlateApplication::Get().SetAllUserFocus(WidgetPath, EFocusCause::SetDirectly);
				}
			}));
	}
}

void FMovieSceneEventParametersCustomization::OnStructChanged(const FAssetData& AssetData)
{
	UScriptStruct* NewStruct = nullptr;

	if (AssetData.IsValid())
	{
		NewStruct = Cast<UScriptStruct>(AssetData.GetAsset());
		if (!NewStruct)
		{
			// Don't clear the type if the user managed to just choose the wrong type of object
			return;
		}
	}

	// Open a transaction
	FScopedTransaction Transaction(LOCTEXT("SetParameterStructText", "Set Parameter Structure"));

	FEditPropertyChain PropertyChain;
	PropertyChain.AddHead(PropertyHandle->GetProperty());

	// Fire off the pre-notify
	FNotifyHook* Hook = PropertyUtilities->GetNotifyHook();
	if (Hook)
	{
		Hook->NotifyPreChange(&PropertyChain);
	}

	// Modify outer objects
	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);
	for (UObject* Object : Outers)
	{
		Object->Modify();
	}

	// Reassign the struct
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	for (void* Value : RawData)
	{
		static_cast<FMovieSceneEventParameters*>(Value)->Reassign(NewStruct);
	}

	FPropertyChangedEvent BubbleChangeEvent(PropertyHandle->GetProperty(), EPropertyChangeType::ValueSet);

	// post notify
	if (Hook)
	{
		Hook->NotifyPostChange(BubbleChangeEvent, &PropertyChain);
	}

	// Force refresh
	PropertyUtilities->NotifyFinishedChangingProperties(BubbleChangeEvent);

	PropertyUtilities->ForceRefresh();
}

void FMovieSceneEventParametersCustomization::OnEditStructChildContentsChanged()
{
	// @todo: call modify on the outer object if possible
	UScriptStruct* Struct = Cast<UScriptStruct>((UStruct*)EditStructData->GetStruct());
	if (!Struct)
	{
		return;
	}

	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);

	for (void* Value : RawData)
	{
		static_cast<FMovieSceneEventParameters*>(Value)->OverwriteWith(EditStructData->GetStructMemory());
	}

	FPropertyChangedEvent BubbleChangeEvent(PropertyHandle->GetProperty(), EPropertyChangeType::ValueSet);
	PropertyUtilities->NotifyFinishedChangingProperties(BubbleChangeEvent);
}

#undef LOCTEXT_NAMESPACE
