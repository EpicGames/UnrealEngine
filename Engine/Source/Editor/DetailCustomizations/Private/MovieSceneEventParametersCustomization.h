// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class FStructOnScope;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class IStructureDetailsView;
struct FAssetData;

class FMovieSceneEventParametersCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;

private:
	void OnStructChanged(const FAssetData& AssetData);
	void OnEditStructChildContentsChanged();

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> PropertyHandle;

	TSharedPtr<IStructureDetailsView> EventPayloadDetails;
	TSharedPtr<FStructOnScope> EditStructData;
};
