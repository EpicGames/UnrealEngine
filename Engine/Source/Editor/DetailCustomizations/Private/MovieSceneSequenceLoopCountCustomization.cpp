// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSequenceLoopCountCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "MovieSceneSequencePlayer.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "MovieSceneSequenceLoopCountCustomization"


TSharedRef<IPropertyTypeCustomization> FMovieSceneSequenceLoopCountCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneSequenceLoopCountCustomization);
}


void FMovieSceneSequenceLoopCountCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	LoopCountProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMovieSceneSequenceLoopCount, Value));

	// Set up the initial environment
	{
		LoopModes.Add(MakeShareable( new FLoopMode{ LOCTEXT("DontLoop", "Don't Loop"), 0 } ));
		LoopModes.Add(MakeShareable( new FLoopMode{ LOCTEXT("Indefinitely", "Loop Indefinitely"), -1 }) );
		LoopModes.Add(MakeShareable( new FLoopMode{ LOCTEXT("Exactly", "Loop Exactly..."), 1 }) );

		int32 CurrentValue = -1;
		LoopCountProperty->GetValue(CurrentValue);

		TSharedPtr<FLoopMode>* FoundMode = LoopModes.FindByPredicate([&](const TSharedPtr<FLoopMode>& In){ return In->Value == CurrentValue; });
		if (FoundMode)
		{
			CurrentMode = *FoundMode;
		}
		else
		{
			CurrentMode = LoopModes.Last();
		}
	}

	HeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.MaxDesiredWidth(200)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SComboBox<TSharedPtr<FLoopMode>>)
				.OptionsSource(&LoopModes)
				.OnSelectionChanged_Lambda([&](TSharedPtr<FLoopMode> Mode, ESelectInfo::Type){
					CurrentMode = Mode;
					UpdateProperty();
				})
				.OnGenerateWidget_Lambda([&](TSharedPtr<FLoopMode> InMode){
					return SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(InMode->DisplayName);
				})
				.InitiallySelectedItem(CurrentMode)
				[
					SAssignNew(CurrentText, STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(CurrentMode->DisplayName)
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.f,0))
			.VAlign(VAlign_Center)
			[
				SAssignNew(LoopEntry, SHorizontalBox)
				.Visibility(CurrentMode == LoopModes.Last() ? EVisibility::Visible : EVisibility::Collapsed)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 0, 4.f, 0))
				[
					LoopCountProperty->CreatePropertyValueWidget()
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Raw(this, &FMovieSceneSequenceLoopCountCustomization::GetCustomSuffix)
				]
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FMovieSceneSequenceLoopCountCustomization::OnLoopResetClicked)
				.Visibility(this, &FMovieSceneSequenceLoopCountCustomization::GetLoopResetVisibility)
				.ContentPadding(FMargin(5.f, 0.f))
				.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.Content()
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault") )
				]
			]
		];
}


void FMovieSceneSequenceLoopCountCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{}


FText FMovieSceneSequenceLoopCountCustomization::GetCustomSuffix() const
{
	int32 NumLoops = 0;
	LoopCountProperty->GetValue(NumLoops);

	return FText::Format(LOCTEXT("TimeFormat", "{0}|plural(one=time, other=times)"), NumLoops);
}

EVisibility FMovieSceneSequenceLoopCountCustomization::GetLoopResetVisibility() const
{
	return CurrentMode == LoopModes[0] ? EVisibility::Hidden : EVisibility::Visible;
}

FReply FMovieSceneSequenceLoopCountCustomization::OnLoopResetClicked()
{
	CurrentMode = LoopModes[0];

	UpdateProperty();

	return FReply::Handled();
}

void FMovieSceneSequenceLoopCountCustomization::UpdateProperty()
{
	if (CurrentMode == LoopModes.Last())
	{
		// Show custom loop entry
		LoopEntry->SetVisibility(EVisibility::Visible);
	}
	else
	{
		// Hide custom loop entry
		LoopEntry->SetVisibility(EVisibility::Collapsed);
	}

	LoopCountProperty->SetValue(CurrentMode->Value);
	CurrentText->SetText(CurrentMode->DisplayName);
}


#undef LOCTEXT_NAMESPACE
