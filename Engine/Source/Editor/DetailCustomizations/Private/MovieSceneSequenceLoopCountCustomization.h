// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class STextBlock;
class SWidget;

/**
 * Details view customization for the FMovieSceneSequencePlaybackSettings struct.
 */
class FMovieSceneSequenceLoopCountCustomization
	: public IPropertyTypeCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:

	// IDetailCustomization interface

	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) override;

protected:

	/** Respond to a selection change event from the combo box. */
	void UpdateProperty();
	
private:

	/** Get the suffix to display after the custom entry box. */
	FText GetCustomSuffix() const;

	/** Get whether the loop reset button should be visible. */
	EVisibility GetLoopResetVisibility() const;

	/** Called when the "Reset to Default" button for the loop mode has been clicked. */
	FReply OnLoopResetClicked();

	/** Array of loop modes. */
	struct FLoopMode
	{
		FText DisplayName;
		int32 Value;
	};

	TArray<TSharedPtr<FLoopMode>> LoopModes;

	/** The loop mode we're currently displaying. */
	TSharedPtr<FLoopMode> CurrentMode;
	/** The text of the current selection. */
	TSharedPtr<STextBlock> CurrentText;
	/** The loop number entry to be hidden and shown based on combo box selection. */
	TSharedPtr<SWidget> LoopEntry;

	/** Property handles of the properties we're editing */
	TSharedPtr<IPropertyHandle> LoopCountProperty;
};
