// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAgentSelectorCustomization.h"

#include "AI/Navigation/NavigationTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Engine/Engine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "NavigationSystem.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FNavAgentSelectorCustomization"

namespace NavAgentSelectorCustoPrivate
{
	const UNavigationSystemV1* GetNavSysCDO()
	{
		return (*GEngine->NavigationSystemClass != nullptr && GEngine->NavigationSystemClass->IsChildOf(UNavigationSystemV1::StaticClass()))
			? GetDefault<UNavigationSystemV1>(GEngine->NavigationSystemClass)
			: GetDefault<UNavigationSystemV1>();
	}

	static const FString AgentPrefix = "bSupportsAgent";
}

TSharedRef<IPropertyTypeCustomization> FNavAgentSelectorCustomization::MakeInstance()
{
	return MakeShareable(new FNavAgentSelectorCustomization);
}

void FNavAgentSelectorCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructHandle = StructPropertyHandle;
	OnAgentStateChanged();

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(400.0f)
	.VAlign(VAlign_Center)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &FNavAgentSelectorCustomization::OnHeaderCheckStateChanged)
			.IsChecked(this, &FNavAgentSelectorCustomization::IsHeaderChecked)
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillContentWidth(/*grow*/0.f, /*shrink*/1.f)
		.Padding(FMargin(3, 0, 0, 0))
		[
			SNew(STextBlock)
			.Text(this, &FNavAgentSelectorCustomization::GetSupportedDesc)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Font(StructCustomizationUtils.GetRegularFont())
		]
	];
}

void FNavAgentSelectorCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren = 0;
	StructPropertyHandle->GetNumChildren(NumChildren);

	const UNavigationSystemV1* NavSysCDO = NavAgentSelectorCustoPrivate::GetNavSysCDO();
	if (NavSysCDO == nullptr)
	{
		return;
	}

	const int32 NumAgents = FMath::Min(NavSysCDO->GetSupportedAgents().Num(), FNavAgentSelector::AgentsMaxCount);

	for (uint32 Idx = 0; Idx < NumChildren; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructPropertyHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetName().StartsWith(NavAgentSelectorCustoPrivate::AgentPrefix))
		{
			PropHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FNavAgentSelectorCustomization::OnAgentStateChanged));

			int32 AgentIdx = -1;
			TTypeFromString<int32>::FromString(AgentIdx, *(PropHandle->GetProperty()->GetName().Mid(NavAgentSelectorCustoPrivate::AgentPrefix.Len()) ));

			if (AgentIdx >= 0 && AgentIdx < NumAgents)
			{
				FText PropName = FText::FromName(NavSysCDO->GetSupportedAgents()[AgentIdx].Name);
				StructBuilder.AddCustomRow(PropName)
					.NameContent()
					[
						SNew(STextBlock)
						.Text(PropName)
						.Font(StructCustomizationUtils.GetRegularFont())
					]
					.ValueContent()
					[
						PropHandle->CreatePropertyValueWidget()
					];
			}

			continue;
		}

		StructBuilder.AddProperty(PropHandle.ToSharedRef());
	}
}

void FNavAgentSelectorCustomization::OnAgentStateChanged()
{
	const UNavigationSystemV1* NavSysCDO = NavAgentSelectorCustoPrivate::GetNavSysCDO();
	if (NavSysCDO == nullptr)
	{
		return;
	}
	int32 NumAgents, NumSupported, FirstSupportedIdx;
	if (!ComputeSupportedAgentCount(NavSysCDO, NumAgents, NumSupported, FirstSupportedIdx))
	{
		return;
	}

	if (NumSupported == NumAgents)
	{
		SupportedDesc = LOCTEXT("AllAgents", "All");
	}
	else if (NumSupported == 0)
	{
		SupportedDesc = LOCTEXT("NoAgents", "None");
	}
	else if (NumSupported == 1)
	{
		SupportedDesc = FText::FromName(NavSysCDO->GetSupportedAgents()[FirstSupportedIdx].Name);
	}
	else
	{
		SupportedDesc = FText::Format(FText::FromString("{0}, ..."), FText::FromName(NavSysCDO->GetSupportedAgents()[FirstSupportedIdx].Name));
	}
}

FText FNavAgentSelectorCustomization::GetSupportedDesc() const
{
	return SupportedDesc;
}

void FNavAgentSelectorCustomization::OnHeaderCheckStateChanged(ECheckBoxState InNewState)
{
	bool bNewValue = false;
	switch (InNewState)
	{
		case ECheckBoxState::Checked:
			bNewValue = true;
			break;
		case ECheckBoxState::Unchecked:
			bNewValue = false;
			break;
		case ECheckBoxState::Undetermined:
		default:
			return;
	}

	const UNavigationSystemV1* NavSysCDO = NavAgentSelectorCustoPrivate::GetNavSysCDO();
	if (!NavSysCDO)
	{
		return;
	}

	uint32 NumChildren = 0;
	StructHandle->GetNumChildren(NumChildren);

	for (uint32 Idx = 0; Idx < NumChildren; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetName().StartsWith(NavAgentSelectorCustoPrivate::AgentPrefix))
		{
			PropHandle->SetValue(bNewValue);
		}
	}

	OnAgentStateChanged();
}

ECheckBoxState FNavAgentSelectorCustomization::IsHeaderChecked() const
{
	const UNavigationSystemV1* NavSysCDO = NavAgentSelectorCustoPrivate::GetNavSysCDO();
	int32 NumAgents, NumSupported, FirstSupportedIdx;
	if (!ComputeSupportedAgentCount(NavSysCDO, NumAgents, NumSupported, FirstSupportedIdx))
	{
		return ECheckBoxState::Unchecked;
	}

	if (NumSupported == NumAgents)
	{
		return ECheckBoxState::Checked;
	}
	else if (NumSupported == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	else
	{
		return ECheckBoxState::Undetermined;
	}
}

bool FNavAgentSelectorCustomization::ComputeSupportedAgentCount(const UNavigationSystemV1* NavSysCDO, int32& OutNumAgents, int32& OutNumSupported, int32& OutFirstSupportedIdx) const
{
	if (NavSysCDO == nullptr)
	{
		return false;
	}

	const int32 NumAgents = FMath::Min(NavSysCDO->GetSupportedAgents().Num(), FNavAgentSelector::AgentsMaxCount);

	uint32 NumChildren = 0;
	StructHandle->GetNumChildren(NumChildren);

	int32 NumSupported = 0;
	int32 FirstSupportedIdx = INDEX_NONE;

	for (uint32 Idx = 0; Idx < NumChildren; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetName().StartsWith(NavAgentSelectorCustoPrivate::AgentPrefix))
		{
			bool bSupportsAgent = false;
			FPropertyAccess::Result Result = PropHandle->GetValue(bSupportsAgent);
			if (Result == FPropertyAccess::Success && bSupportsAgent)
			{
				int32 AgentIdx = -1;
				TTypeFromString<int32>::FromString(AgentIdx, *(PropHandle->GetProperty()->GetName().Mid(NavAgentSelectorCustoPrivate::AgentPrefix.Len())));

				if (AgentIdx >= 0 && AgentIdx < NumAgents)
				{
					NumSupported++;
					if (FirstSupportedIdx == INDEX_NONE)
					{
						FirstSupportedIdx = AgentIdx;
					}
				}
			}
		}
	}

	OutNumAgents = NumAgents;
	OutNumSupported = NumSupported;
	OutFirstSupportedIdx = FirstSupportedIdx;
	return true;
}

#undef LOCTEXT_NAMESPACE
