// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavLinkStructCustomization.h"

#include "AI/Navigation/NavLinkDefinition.h"
#include "Containers/UnrealString.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IPropertyTypeCustomization> FNavLinkStructCustomization::MakeInstance( )
{
	return MakeShareable(new FNavLinkStructCustomization);
}

void FNavLinkStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> CommentHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNavigationLinkBase,Description));
	FString Desc;

	if (CommentHandle.IsValid())
	{
		CommentHandle->GetValue(Desc);
	}

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(400.0f)
	[
		SNew(STextBlock)
		.Text(FText::FromString(Desc))
		.Font(StructCustomizationUtils.GetRegularFont())
	];
}

void FNavLinkStructCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{ 
	uint32 NumChildProps = 0;
	StructPropertyHandle->GetNumChildren(NumChildProps);

	for (uint32 Idx = 0; Idx < NumChildProps; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructPropertyHandle->GetChildHandle(Idx);
		StructBuilder.AddProperty(PropHandle.ToSharedRef());
	}
}
