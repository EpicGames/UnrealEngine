// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

class FNavLinkStructCustomization : public IPropertyTypeCustomization
{
public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for Keys.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
};
