// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectDetails.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Math/NumericLimits.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "ObjectEditorUtils.h"
#include "SWarningOrErrorBox.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"

#define LOCTEXT_NAMESPACE "ObjectDetails"

TSharedRef<IDetailCustomization> FObjectDetails::MakeInstance()
{
	return MakeShareable(new FObjectDetails);
}

void FObjectDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	AddExperimentalWarningCategory(DetailBuilder);
	AddCallInEditorMethods(DetailBuilder);
}

void FObjectDetails::AddExperimentalWarningCategory(IDetailLayoutBuilder& DetailBuilder)
{
	bool bBaseClassIsExperimental = false;
	bool bBaseClassIsEarlyAccess = false;
	FString MostDerivedDevelopmentClassName;
	FObjectEditorUtils::GetClassDevelopmentStatus(DetailBuilder.GetBaseClass(), bBaseClassIsExperimental, bBaseClassIsEarlyAccess, MostDerivedDevelopmentClassName);

	FDetailCustomizationsModule& DetailCustomizationsModule = FModuleManager::Get().GetModuleChecked<FDetailCustomizationsModule>(TEXT("DetailCustomizations"));
	if (DetailCustomizationsModule.IsDevelopmentStatusWarningSuppressed(DetailBuilder.GetBaseClass()))
	{
		return;
	}

	if (bBaseClassIsExperimental || bBaseClassIsEarlyAccess)
	{
		const FName CategoryName(TEXT("Warning"));
		const FText CategoryDisplayName = LOCTEXT("WarningCategoryDisplayName", "Warning");
		const FText WarningText = bBaseClassIsExperimental ? FText::Format( LOCTEXT("ExperimentalClassWarning", "Uses experimental class: {0}") , FText::FromString(MostDerivedDevelopmentClassName) )
			: FText::Format( LOCTEXT("EarlyAccessClassWarning", "Uses beta class {0}"), FText::FromString(MostDerivedDevelopmentClassName) );
		const FText SearchString = WarningText;

		IDetailCategoryBuilder& WarningCategory = DetailBuilder.EditCategory(CategoryName, CategoryDisplayName, ECategoryPriority::Transform);

		FDetailWidgetRow& WarningRow = WarningCategory.AddCustomRow(SearchString)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.f, 4.f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(WarningText)
				]
			];
	}
}

void FObjectDetails::AddCallInEditorMethods(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.GetObjectsBeingCustomized(/*out*/ SelectedObjectsList);
	SelectedObjectsList.RemoveAllSwap([](TWeakObjectPtr<UObject> ObjPtr) { UObject* Obj = ObjPtr.Get(); return (Obj == nullptr) || Obj->HasAnyFlags(RF_ArchetypeObject); });

	// Don't show CallInEditor buttons if there are no valid execution contexts (e.g. in Blueprint Editor)
	if (SelectedObjectsList.IsEmpty())
	{
		return;
	}

	TArray<UFunction*> CallInEditorFunctions;
	PropertyCustomizationHelpers::GetCallInEditorFunctionsForClass(
		DetailBuilder.GetBaseClass(),
		CallInEditorFunctions);

	PropertyCustomizationHelpers::AddFunctionCallWidgets(
		DetailBuilder,
		CallInEditorFunctions,
		FPropertyFunctionCallDelegates(
			FPropertyFunctionCallDelegates::FOnGetExecutionContext::CreateSP(this, &FObjectDetails::GetFunctionCallExecutionContext)
		));
}

TArray<TWeakObjectPtr<UObject>> FObjectDetails::GetFunctionCallExecutionContext(TWeakObjectPtr<UFunction> InWeakFunction) const
{
	return SelectedObjectsList;
}

#undef LOCTEXT_NAMESPACE
