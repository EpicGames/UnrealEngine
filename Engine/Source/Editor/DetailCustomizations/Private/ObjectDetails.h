// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class UFunction;
class UObject;

class FObjectDetails : public IDetailCustomization
{
public:
	// Creates an instance of FObjectDetails
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	// Adds a warning banner if the class is marked as Experimental or EarlyAccessPreview
	void AddExperimentalWarningCategory(IDetailLayoutBuilder& DetailBuilder);

	// Creates a button strip in each category that contains reflected functions marked as CallInEditor
	void AddCallInEditorMethods(IDetailLayoutBuilder& DetailBuilder);

	TArray<TWeakObjectPtr<UObject>> GetFunctionCallExecutionContext(TWeakObjectPtr<UFunction> InWeakFunction) const;

private:
	// The list of selected objects, used when invoking a CallInEditor method
	TArray<TWeakObjectPtr<UObject>> SelectedObjectsList;
};
