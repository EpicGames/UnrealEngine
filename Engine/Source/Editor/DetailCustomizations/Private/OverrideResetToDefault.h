// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailPropertyRow.h"
#include "Layout/Visibility.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "UObject/UnrealType.h"

/**
 * Class template aiming to override ResetToDefault methods by comparing properties with an internal static DefaultObject.
 * 
 * This behavior was originally written for settings UI and does not properly support inheritance (resetting a parent class's
 * struct member to default and have it propagate to loaded children). If a struct wants to override ResetToDefault behavior
 * using this class and must support the outer-object being a parent class, OnResetToDefault should be revisited to properly
 * reset archetype instances as well.
 */
template<typename UStructType>
class TOverrideResetToDefaultWithStaticUStruct
{
public:
	/** Called by the UI to show/hide the reset widgets */
	static bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle);

	/** Reset to default triggered in UI */
	static void OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

protected:
	/** Adds callbacks used by the UI to determine if a property can be reset and reset it. */
	static void AddResetToDefaultOverrides(IDetailPropertyRow& InDetailPropertyRow);

private:
	/**
	 * Resolves the correct default value pointer for a property that may be nested within UStructType.
	 * Walks the property handle parent chain to figure out the correct offset from DefaultObject.
	 */
	static const void* ResolveDefaultValuePtr(TSharedPtr<IPropertyHandle> InPropertyHandle);

	static const UStructType DefaultObject;
};

template<typename UStructType>
const UStructType TOverrideResetToDefaultWithStaticUStruct<UStructType>::DefaultObject;

template<typename UStructType>
const void* TOverrideResetToDefaultWithStaticUStruct<UStructType>::ResolveDefaultValuePtr(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty* Property = InPropertyHandle->GetProperty();
	if (!Property)
	{
		return nullptr;
	}

	// If this property is a direct member of UStructType, use DefaultObject as the container
	if (Property->GetOwnerStruct() == UStructType::StaticStruct())
	{
		return Property->ContainerPtrToValuePtr<void>(&DefaultObject);
	}

	// Property is nested deeper within UStructType - resolve the parent's default value to use as container
	TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle();
	if (!ParentHandle.IsValid())
	{
		return nullptr;
	}

	const void* ParentDefaultValue = ResolveDefaultValuePtr(ParentHandle);
	if (!ParentDefaultValue)
	{
		return nullptr;
	}

	return Property->ContainerPtrToValuePtr<void>(ParentDefaultValue);
}

template<typename UStructType>
bool TOverrideResetToDefaultWithStaticUStruct<UStructType>::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty*		Property = InPropertyHandle->GetProperty();
	const void*		DefaultValuePtr = nullptr;
	void*			ValuePtr = nullptr;

	check(Property != nullptr);

	DefaultValuePtr = ResolveDefaultValuePtr(InPropertyHandle);
	InPropertyHandle->GetValueData(ValuePtr);

	if ((DefaultValuePtr != nullptr) && (ValuePtr != nullptr))
	{
		return !Property->Identical(DefaultValuePtr, ValuePtr);
	}

	return false;
}

template<typename UStructType>
void TOverrideResetToDefaultWithStaticUStruct<UStructType>::OnResetToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FProperty*	Property = InPropertyHandle->GetProperty();
	const void* DefaultValuePtr = nullptr;
	void*		ValuePtr = nullptr;

	check(Property != nullptr);

	DefaultValuePtr = ResolveDefaultValuePtr(InPropertyHandle);
	InPropertyHandle->GetValueData(ValuePtr);

	if ((DefaultValuePtr != nullptr) && (ValuePtr != nullptr))
	{
		Property->CopySingleValue(ValuePtr, DefaultValuePtr);
	}
}

template<typename UStructType>
void TOverrideResetToDefaultWithStaticUStruct<UStructType>::AddResetToDefaultOverrides(IDetailPropertyRow& InDetailPropertyRow)
{
	InDetailPropertyRow.OverrideResetToDefault(FResetToDefaultOverride::Create(FIsResetToDefaultVisible::CreateStatic(&TOverrideResetToDefaultWithStaticUStruct<UStructType>::IsResetToDefaultVisible),
																			   FResetToDefaultHandler::CreateStatic(&TOverrideResetToDefaultWithStaticUStruct<UStructType>::OnResetToDefault),
																			   true));
}
