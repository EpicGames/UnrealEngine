// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleModuleDetails.h"

#include "Containers/UnrealString.h"
#include "DetailLayoutBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Particles/ParticleEmitter.h"
#include "Particles/ParticleModule.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

DECLARE_LOG_CATEGORY_CLASS(LogParticleModuleDetails, Log, All);

static const FText& GetNotAllowedOnGPUEmitterText()
{
	static FText RetText = NSLOCTEXT("ParticleModuleDetails", "NotAllowedOnGPU", "Not allowed on a GPU emitter.");
	return RetText;
}

TSharedRef<const FPropertyRestriction> FParticleModuleDetailsBase::GetDistributionsForGPURestriction()
{
	static TSharedPtr<FPropertyRestriction> Restriction;
	static TArray<FString> RestrictedDistributions;
	if( !Restriction.IsValid() )
	{
		Restriction = MakeShareable(new FPropertyRestriction(GetNotAllowedOnGPUEmitterText()));
		UParticleModule::GetDistributionsRestrictedOnGPU(RestrictedDistributions);	
		
		for( int32 ValueIdx=0; ValueIdx < RestrictedDistributions.Num() ; ++ValueIdx )
		{
			Restriction->AddDisabledValue(RestrictedDistributions[ValueIdx]);
		}
	}

	return Restriction.ToSharedRef();
}

//////////////////////////////////////////////////////////////////////////

void FParticleModuleDetailsBase::RestrictPropertiesOnGPUEmitter( IDetailLayoutBuilder& DetailBuilder, TArray<FString>& PropertyNames, TRestrictionList& Restrictions )
{
	check(PropertyNames.Num() > 0);
	check(Restrictions.Num() > 0);

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	for( int32 i = 0 ; i < Objects.Num() ; ++i )
	{
		if(Objects[i].IsValid())
		{
			UParticleModule* Module = Cast<UParticleModule>(Objects[i].Get());
			if( Module && Module->IsUsedInGPUEmitter() )
			{
				for( int32 PropertyIdx = 0; PropertyIdx < PropertyNames.Num() ; ++PropertyIdx )
				{
					TSharedRef<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(FName(*PropertyNames[PropertyIdx]));
					if( PropertyHandle->IsValidHandle() )
					{
						for( int32 RestrictionIdx = 0; RestrictionIdx < Restrictions.Num() ; ++RestrictionIdx )
						{
							PropertyHandle->AddRestriction(Restrictions[RestrictionIdx]);
						}
					}
				}
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleRequiredDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleRequiredDetails );
}

void FParticleModuleRequiredDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> PropertiesToRestrict;
	PropertiesToRestrict.Add(TEXT("InterpolationMethod"));

	static TSharedPtr<FPropertyRestriction> RandomRandomBlendRestriction;
	if( !RandomRandomBlendRestriction.IsValid() )
	{
		RandomRandomBlendRestriction = MakeShareable( new FPropertyRestriction(GetNotAllowedOnGPUEmitterText()) );
		const UEnum* const ParticleSubUVInterpMethodEnum = StaticEnum<EParticleSubUVInterpMethod>();		
		RandomRandomBlendRestriction->AddDisabledValue(ParticleSubUVInterpMethodEnum->GetNameStringByValue((uint8)EParticleSubUVInterpMethod::PSUVIM_Random));
		RandomRandomBlendRestriction->AddDisabledValue(ParticleSubUVInterpMethodEnum->GetNameStringByValue((uint8)EParticleSubUVInterpMethod::PSUVIM_Random_Blend));
	}

	TRestrictionList RestrictionList;
	RestrictionList.Add(RandomRandomBlendRestriction.ToSharedRef());
	RestrictPropertiesOnGPUEmitter( DetailBuilder, PropertiesToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleSubUVDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleSubUVDetails );
}

void FParticleModuleSubUVDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("SubImageIndex.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleAccelerationDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleAccelerationDetails );
}

void FParticleModuleAccelerationDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("Acceleration.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleAccelerationDragDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleAccelerationDragDetails );
}

void FParticleModuleAccelerationDragDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("DragCoefficient"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleAccelerationDragScaleOverLifeDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleAccelerationDragScaleOverLifeDetails );
}

void FParticleModuleAccelerationDragScaleOverLifeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("DragScale"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleCollisionGPUDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleCollisionGPUDetails );
}

void FParticleModuleCollisionGPUDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("ResilienceScaleOverLife.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleOrbitDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleOrbitDetails );
}

void FParticleModuleOrbitDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("OffsetAmount.Distribution"));
	DistributionsToRestrict.Add(TEXT("RotationAmount.Distribution"));
	DistributionsToRestrict.Add(TEXT("RotationRateAmount.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleSizeMultiplyLifeDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleSizeMultiplyLifeDetails );
}

void FParticleModuleSizeMultiplyLifeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("LifeMultiplier.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleSizeScaleDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleSizeScaleDetails );
}

void FParticleModuleSizeScaleDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("SizeScale.Distribution"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleVectorFieldScaleDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleVectorFieldScaleDetails );
}

void FParticleModuleVectorFieldScaleDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("VectorFieldScale"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FParticleModuleVectorFieldScaleOverLifeDetails::MakeInstance()
{
	return MakeShareable( new FParticleModuleVectorFieldScaleOverLifeDetails );
}

void FParticleModuleVectorFieldScaleOverLifeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<FString> DistributionsToRestrict;
	DistributionsToRestrict.Add(TEXT("VectorFieldScaleOverLife"));

	TRestrictionList RestrictionList;
	RestrictionList.Add( GetDistributionsForGPURestriction() );
	RestrictPropertiesOnGPUEmitter( DetailBuilder, DistributionsToRestrict, RestrictionList );
}
