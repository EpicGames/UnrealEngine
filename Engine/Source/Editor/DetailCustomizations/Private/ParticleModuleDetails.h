// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class FPropertyRestriction;
class FString;
class IDetailLayoutBuilder;

typedef TArray<TSharedRef<const class FPropertyRestriction>> TRestrictionList;

class FParticleModuleDetailsBase : public IDetailCustomization
{
protected:
	void RestrictPropertiesOnGPUEmitter( IDetailLayoutBuilder& DetailBuilder, TArray<FString>& PropertyNames, TRestrictionList& Restrictions );

	TSharedRef<const FPropertyRestriction> GetDistributionsForGPURestriction();
};

class FParticleModuleRequiredDetails: public FParticleModuleDetailsBase
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

class FParticleModuleSubUVDetails : public FParticleModuleDetailsBase
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

class FParticleModuleAccelerationDetails : public FParticleModuleDetailsBase
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

class FParticleModuleAccelerationDragDetails : public FParticleModuleDetailsBase
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};

 class FParticleModuleAccelerationDragScaleOverLifeDetails : public FParticleModuleDetailsBase
 {
 public:
 	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
 	static TSharedRef<IDetailCustomization> MakeInstance();
 
 	/** IDetailCustomization interface */
 	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleCollisionGPUDetails : public FParticleModuleDetailsBase
 {
 public:
 	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
 	static TSharedRef<IDetailCustomization> MakeInstance();
 
 	/** IDetailCustomization interface */
 	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleOrbitDetails : public FParticleModuleDetailsBase
 {
 public:
	 /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	 static TSharedRef<IDetailCustomization> MakeInstance();

	 /** IDetailCustomization interface */
	 virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleSizeMultiplyLifeDetails : public FParticleModuleDetailsBase
 {
 public:
	 /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	 static TSharedRef<IDetailCustomization> MakeInstance();

	 /** IDetailCustomization interface */
	 virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleSizeScaleDetails : public FParticleModuleDetailsBase
 {
 public:
	 /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	 static TSharedRef<IDetailCustomization> MakeInstance();

	 /** IDetailCustomization interface */
	 virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleVectorFieldScaleDetails : public FParticleModuleDetailsBase
 {
 public:
	 /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	 static TSharedRef<IDetailCustomization> MakeInstance();

	 /** IDetailCustomization interface */
	 virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };

 class FParticleModuleVectorFieldScaleOverLifeDetails : public FParticleModuleDetailsBase
 {
 public:
	 /** Makes a new instance of this detail layout class for a specific detail view requesting it */
	 static TSharedRef<IDetailCustomization> MakeInstance();

	 /** IDetailCustomization interface */
	 virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
 };
 
