// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleSysParamStructCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Particles/ParticleSystemComponent.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


#define LOCTEXT_NAMESPACE "ParticleSysParamStructCustomization"


TSharedRef<IPropertyTypeCustomization> FParticleSysParamStructCustomization::MakeInstance()
{
	return MakeShareable(new FParticleSysParamStructCustomization);
}


FParticleSysParamStructCustomization::FParticleSysParamStructCustomization()
{
	ParameterType = PSPT_None;
}


void FParticleSysParamStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}


void FParticleSysParamStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Cache struct property handle
	PropertyHandle = StructPropertyHandle;

	// Add name property
	TSharedPtr<IPropertyHandle> NameHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Name));
	StructBuilder.AddProperty(NameHandle.ToSharedRef());

	// Add type property - this is a custom widget which remembers its type so that other widgets can alter their visibility accordingly
	TSharedPtr<IPropertyHandle> ParamTypeHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, ParamType));

	uint8 Type;
	FPropertyAccess::Result Result = ParamTypeHandle->GetValue(Type);
	check(Result != FPropertyAccess::Fail);

	if (Result == FPropertyAccess::Success)
	{
		ParameterType = Type;
	}
	else
	{
		// For multiple values
		ParameterType = INDEX_NONE;
	}

	TArray<bool> RestrictedList;
	ParamTypeHandle->GeneratePossibleValues(ParameterTypeNames, ParameterTypeToolTips, RestrictedList);

	StructBuilder.AddCustomRow(LOCTEXT("ParamType", "Param Type"))
		.NameContent()
		[
			ParamTypeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&ParameterTypeNames)
			.OnGenerateWidget(this, &FParticleSysParamStructCustomization::OnGenerateComboWidget)
			.OnSelectionChanged(this, &FParticleSysParamStructCustomization::OnComboSelectionChanged)
			.InitiallySelectedItem(ParameterTypeNames[(ParameterType == INDEX_NONE) ? 0 : ParameterType])
			[
				SNew(STextBlock)
				.Text(this, &FParticleSysParamStructCustomization::GetParameterTypeName)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
			]
		];

	// Add other properties
	TSharedPtr<IPropertyHandle> ScalarHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Scalar));
	StructBuilder.AddProperty(ScalarHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetScalarVisibility));

	TSharedPtr<IPropertyHandle> ScalarLowHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Scalar_Low));
	StructBuilder.AddProperty(ScalarLowHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetScalarLowVisibility));

	TSharedPtr<IPropertyHandle> VectorHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Vector));
	StructBuilder.AddProperty(VectorHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetVectorVisibility));

	TSharedPtr<IPropertyHandle> VectorLowHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Vector_Low));
	StructBuilder.AddProperty(VectorLowHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetVectorLowVisibility));

	TSharedPtr<IPropertyHandle> ColorHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Color));
	StructBuilder.AddProperty(ColorHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetColorVisibility));

	TSharedPtr<IPropertyHandle> ActorHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Actor));
	StructBuilder.AddProperty(ActorHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetActorVisibility));

	TSharedPtr<IPropertyHandle> MaterialHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, Material));
	StructBuilder.AddProperty(MaterialHandle.ToSharedRef())
		.Visibility(TAttribute<EVisibility>(this, &FParticleSysParamStructCustomization::GetMaterialVisibility));
}


EVisibility FParticleSysParamStructCustomization::GetScalarVisibility() const
{
	return (ParameterType == PSPT_Scalar || ParameterType == PSPT_ScalarRand) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetScalarLowVisibility() const
{
	return (ParameterType == PSPT_ScalarRand) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetVectorVisibility() const
{
	return (ParameterType == PSPT_Vector || ParameterType == PSPT_VectorRand || ParameterType == PSPT_VectorUnitRand) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetVectorLowVisibility() const
{
	return (ParameterType == PSPT_VectorRand || ParameterType == PSPT_VectorUnitRand) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetColorVisibility() const
{
	return (ParameterType == PSPT_Vector || ParameterType == PSPT_VectorRand || ParameterType == PSPT_Color || ParameterType == PSPT_VectorUnitRand) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetActorVisibility() const
{
	return (ParameterType == PSPT_Actor) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FParticleSysParamStructCustomization::GetMaterialVisibility() const
{
	return (ParameterType == PSPT_Material) ? EVisibility::Visible : EVisibility::Collapsed;
}


TSharedRef<SWidget> FParticleSysParamStructCustomization::OnGenerateComboWidget(TSharedPtr<FString> InComboString)
{
	// Find ToolTip which corresponds to string
	FText ToolTip;
	check(ParameterTypeNames.Num() == ParameterTypeToolTips.Num());
	if (ParameterTypeToolTips.Num() > 0)
	{
		int32 Index = ParameterTypeNames.IndexOfByKey(InComboString);
		if (ensure(Index >= 0))
		{
			ToolTip = ParameterTypeToolTips[Index];
		}
	}

	return
		SNew(SBox)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*InComboString))
			.ToolTipText(ToolTip)
			.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
		];
}


void FParticleSysParamStructCustomization::OnComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo)
{
	TSharedPtr<IPropertyHandle> ParamTypeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FParticleSysParam, ParamType));
	if (ParamTypeHandle.IsValid())
	{
		ParameterType = static_cast<EParticleSysParamType>(ParameterTypeNames.IndexOfByKey(InSelectedItem));
		if (ensure(ParameterType >= 0))
		{
			ensure(ParamTypeHandle->SetValue(static_cast<uint8>(ParameterType)) == FPropertyAccess::Success);
		}
	}
}


FText FParticleSysParamStructCustomization::GetParameterTypeName() const
{
	if (ParameterType == INDEX_NONE)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromString(*ParameterTypeNames[ParameterType]);
}

#undef LOCTEXT_NAMESPACE
