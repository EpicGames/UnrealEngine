// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"

class FDetailWidgetRow;
class FString;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;

/**
* Class for FParticleSysParam struct customization
*/
class FParticleSysParamStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FParticleSysParamStructCustomization();

	/** IPropertyTypeCustomization instance */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	/** Called by the combo box to generate each row of the widget */
	TSharedRef<SWidget> OnGenerateComboWidget(TSharedPtr<FString> InComboString);

	/** Called when the combo box selection changes, when a new parameter type is selected */
	void OnComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo);

	/** Called to obtain an FText representing the current combo box selection */
	FText GetParameterTypeName() const;

private:

	/** Cached handle to the struct property */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Current parameter type */
	int32 ParameterType;

	/** A list of parameter type names */
	TArray<TSharedPtr<FString>> ParameterTypeNames;

	/** A list of parameter type tooltips */
	TArray<FText> ParameterTypeToolTips;

	/** Delegates called by each property widget to determine their visibility */
	EVisibility GetScalarVisibility() const;
	EVisibility GetScalarLowVisibility() const;
	EVisibility GetVectorVisibility() const;
	EVisibility GetVectorLowVisibility() const;
	EVisibility GetColorVisibility() const;
	EVisibility GetActorVisibility() const;
	EVisibility GetMaterialVisibility() const;
};
