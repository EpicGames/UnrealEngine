// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleSystemComponentDetails.h"

// @todo Can remove this if we switch to instance-based editing in the Blueprint editor.
#include "ActorEditorUtils.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ParticleSystemComponentDetails"

TSharedRef<IDetailCustomization> FParticleSystemComponentDetails::MakeInstance()
{
	return MakeShareable(new FParticleSystemComponentDetails);
}

void FParticleSystemComponentDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	this->DetailLayout = &InDetailLayout;

	InDetailLayout.EditCategory("Particles", FText::GetEmpty(), ECategoryPriority::Important);

	IDetailCategoryBuilder& CustomCategory = InDetailLayout.EditCategory("EmitterActions", NSLOCTEXT("ParticleSystemComponentDetails", "EmitterActionCategoryName", "Emitter Actions"), ECategoryPriority::Important);

	CustomCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew( SBox )
			.MaxDesiredWidth(300.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.0f)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FParticleSystemComponentDetails::OnAutoPopulateClicked)
					.ToolTipText(NSLOCTEXT("ParticleSystemComponentDetails", "AutoPopulateButtonTooltip", "Copies properties from the source particle system into the instanced parameters of this system"))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ParticleSystemComponentDetails", "AutoPopulateButton", "Expose Parameter"))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.OnClicked(this, &FParticleSystemComponentDetails::OnResetEmitter)
					.ToolTipText(NSLOCTEXT("ParticleSystemComponentDetails", "ResetEmitterButtonTooltip", "Resets the selected particle system."))
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("ParticleSystemComponentDetails", "ResetEmitterButton", "Reset Emitter"))
					]
				]
			]
		];
}

FReply FParticleSystemComponentDetails::OnAutoPopulateClicked()
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();

	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			if (UParticleSystemComponent* PSC = Cast<UParticleSystemComponent>(SelectedObjects[Idx].Get()))
			{
				PSC->AutoPopulateInstanceProperties();
			}
			else if (AEmitter* Emitter = Cast<AEmitter>(SelectedObjects[Idx].Get()))
			{
				Emitter->AutoPopulateInstanceProperties();
			}
		}
	}

	return FReply::Handled();
}

FReply FParticleSystemComponentDetails::OnResetEmitter()
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout->GetSelectedObjects();
	// Iterate over selected Actors.
	for (int32 Idx = 0; Idx < SelectedObjects.Num(); ++Idx)
	{
		if (SelectedObjects[Idx].IsValid())
		{
			UParticleSystemComponent* PSC = Cast<UParticleSystemComponent>(SelectedObjects[Idx].Get());
			if (!PSC)
			{
				if (AEmitter* Emitter = Cast<AEmitter>(SelectedObjects[Idx].Get()))
				{
					PSC = Emitter->GetParticleSystemComponent();
				}
			}

			// If the object selected to the details view is a template, then we need to redirect the reset to the preview instance (e.g. in the Blueprint editor).
			// @todo Can remove this if we switch to instance-based editing in the Blueprint editor.
			if (PSC && PSC->IsTemplate())
			{
				TArray<UObject*> Instances;
				PSC->GetArchetypeInstances(Instances);
				UObject** ElementPtr = Instances.FindByPredicate([](const UObject* Element)
				{
					const AActor* Owner = Cast<AActor>(Element->GetOuter());
					return Owner != nullptr && FActorEditorUtils::IsAPreviewOrInactiveActor(Owner);
				});

				if (ElementPtr)
				{
					PSC = Cast<UParticleSystemComponent>(*ElementPtr);
				}
			}

			if (PSC)
			{
				PSC->ResetParticles();
				PSC->ActivateSystem();
			}
		}
	}
	return FReply::Handled();
}



#undef LOCTEXT_NAMESPACE
