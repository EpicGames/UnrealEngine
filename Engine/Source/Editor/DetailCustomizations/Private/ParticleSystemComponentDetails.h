// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

class FParticleSystemComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& InDetailLayout ) override;

	/** Handles the Auto-Populate button's on click event */
	FReply OnAutoPopulateClicked();

	/** Handles the Emitter Reset button's on click event */
	FReply OnResetEmitter();

private:

	/** Cached off reference to the layout builder */
	IDetailLayoutBuilder* DetailLayout;
};
