// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerPlatformPropertyCustomization.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "UObject/PerPlatformProperties.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SPerPlatformPropertiesWidget.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "PerPlatformPropertyCustomization"

template<typename PerPlatformType>
void FPerPlatformPropertyCustomization<PerPlatformType>::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	TAttribute<TArray<FName>> PlatformOverrideNames = TAttribute<TArray<FName>>::Create(TAttribute<TArray<FName>>::FGetter::CreateSP(this, &FPerPlatformPropertyCustomization<PerPlatformType>::GetPlatformOverrideNames, StructPropertyHandle));

	FPerPlatformPropertyCustomNodeBuilderArgs Args;
	Args.FilterText = StructPropertyHandle->GetPropertyDisplayName();
	Args.OnGenerateNameWidget = FOnGetContent::CreateLambda([StructPropertyHandle]()
	{
		return StructPropertyHandle->CreatePropertyNameWidget();
	});
	Args.PlatformOverrideNames = PlatformOverrideNames;
	Args.OnAddPlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FPerPlatformPropertyCustomization<PerPlatformType>::AddPlatformOverride, StructPropertyHandle);
	Args.OnRemovePlatformOverride = FOnPlatformOverrideAction::CreateSP(this, &FPerPlatformPropertyCustomization<PerPlatformType>::RemovePlatformOverride, StructPropertyHandle);
	Args.OnGenerateWidgetForPlatformRow = FOnGenerateWidget::CreateLambda([this, StructPropertyHandle, &StructBuilder](FName PlatformGroupName)
	{
		return GetWidget(PlatformGroupName, StructPropertyHandle, StructBuilder);
	});
	Args.IsEnabled = TAttribute<bool>::CreateLambda([StructPropertyHandle]()
	{		
		return StructPropertyHandle->IsEditable();
	});		
	
	StructBuilder.AddCustomBuilder(MakeShared<FPerPlatformPropertyCustomNodeBuilder>(MoveTemp(Args)));
}


template<typename PerPlatformType>
TSharedRef<SWidget> FPerPlatformPropertyCustomization<PerPlatformType>::GetWidget(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder) const
{
	TSharedPtr<IPropertyHandle>	EditProperty;

	if (PlatformGroupName == NAME_None)
	{
		EditProperty = StructPropertyHandle->GetChildHandle(FName("Default"));
	}
	else
	{
		TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
		if (MapProperty.IsValid())
		{
			uint32 NumChildren = 0;
			MapProperty->GetNumChildren(NumChildren);
			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = MapProperty->GetChildHandle(ChildIdx);
				if (ChildProperty.IsValid())
				{
					TSharedPtr<IPropertyHandle> KeyProperty = ChildProperty->GetKeyHandle();
					if (KeyProperty.IsValid())
					{
						FName KeyName;
						if(KeyProperty->GetValue(KeyName) == FPropertyAccess::Success && KeyName == PlatformGroupName)
						{
							EditProperty = ChildProperty;
							break;
						}
					}
				}
			}
		}
	
	}

	// Push down struct metadata to per-platform properties
	if (EditProperty.IsValid())
	{
		// First get the source map
		if (const TMap<FName, FString>* SourceMap = StructPropertyHandle->GetMetaDataProperty()->GetMetaDataMap())
		{
			// Iterate through source map, setting each key/value pair in the destination
			for (const auto& It : *SourceMap)
			{
				EditProperty->SetInstanceMetaData(*It.Key.ToString(), *It.Value);
			}
		}

		// Copy instance metadata as well
		if (const TMap<FName, FString>* InstanceSourceMap = StructPropertyHandle->GetInstanceMetaDataMap())
		{
			for (const auto& It : *InstanceSourceMap)
			{
				EditProperty->SetInstanceMetaData(*It.Key.ToString(), *It.Value);
			}
		}

		if (EditProperty->GetProperty()->IsA<FStructProperty>())
		{
			return StructBuilder.GenerateStructValueWidget(EditProperty->AsShared());
		}

		return EditProperty->CreatePropertyValueWidget(false);
	}
	
	return SNullWidget::NullWidget;
}


template<typename PerPlatformType>
bool FPerPlatformPropertyCustomization<PerPlatformType>::AddPlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("AddPlatformOverride", "Add Platform Override"));

	TSharedPtr<IPropertyHandle>	PerPlatformProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	TSharedPtr<IPropertyHandle>	DefaultProperty = StructPropertyHandle->GetChildHandle(FName("Default"));
	if (PerPlatformProperty.IsValid() && DefaultProperty.IsValid())
	{
		TSharedPtr<IPropertyHandleMap> MapProperty = PerPlatformProperty->AsMap();
		if (MapProperty.IsValid())
		{
			MapProperty->AddItem();
			uint32 NumChildren = 0;
			PerPlatformProperty->GetNumChildren(NumChildren);
			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = PerPlatformProperty->GetChildHandle(ChildIdx);
				if (ChildProperty.IsValid())
				{
					TSharedPtr<IPropertyHandle> KeyProperty = ChildProperty->GetKeyHandle();
					if (KeyProperty.IsValid())
					{
						FName KeyName;
						if (KeyProperty->GetValue(KeyName) == FPropertyAccess::Success && KeyName == NAME_None)
						{
							// Set Key
							KeyProperty->SetValue(PlatformGroupName);

							// Set Value
							FString PropertyValueString;
							DefaultProperty->GetValueAsFormattedString(PropertyValueString);
							ChildProperty->SetValueFromFormattedString(PropertyValueString);

							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

template<typename PerPlatformType>
bool FPerPlatformPropertyCustomization<PerPlatformType>::RemovePlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("RemovePlatformOverride", "Remove Platform Override"));

	TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			TMap<FName, typename PerPlatformType::ValueType>* PerPlatformMap = (TMap<FName, typename PerPlatformType::ValueType>*)(Data);
			check(PerPlatformMap);
			TArray<FName> KeyArray;
			PerPlatformMap->GenerateKeyArray(KeyArray);
			for (FName PlatformName : KeyArray)
			{
				if (PlatformName == PlatformGroupName)
				{
					PerPlatformMap->Remove(PlatformName);

					return true;
				}
			}
		}
	}
	return false;

}

template<typename PerPlatformType>
TArray<FName> FPerPlatformPropertyCustomization<PerPlatformType>::GetPlatformOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const
{
	TArray<FName> PlatformOverrideNames;

	TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			const TMap<FName, typename PerPlatformType::ValueType>* PerPlatformMap = (const TMap<FName, typename PerPlatformType::ValueType>*)(Data);
			check(PerPlatformMap);
			TArray<FName> KeyArray;
			PerPlatformMap->GenerateKeyArray(KeyArray);
			for (FName PlatformName : KeyArray)
			{
				PlatformOverrideNames.AddUnique(PlatformName);
			}
		}

	}
	return PlatformOverrideNames;
}

template<typename PerPlatformType>
TSharedRef<IPropertyTypeCustomization> FPerPlatformPropertyCustomization<PerPlatformType>::MakeInstance()
{
	return MakeShareable(new FPerPlatformPropertyCustomization<PerPlatformType>);
}

/* Only explicitly instantiate the types which are supported
*****************************************************************************/

template class FPerPlatformPropertyCustomization<FPerPlatformInt>;
template class FPerPlatformPropertyCustomization<FPerPlatformFloat>;
template class FPerPlatformPropertyCustomization<FPerPlatformBool>;
template class FPerPlatformPropertyCustomization<FPerPlatformFrameRate>;

#undef LOCTEXT_NAMESPACE

void FPerPlatformPropertyCustomNodeBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRebuildChildren = InOnRegenerateChildren;
}

void FPerPlatformPropertyCustomNodeBuilder::SetOnToggleExpansion(FOnToggleNodeExpansion InOnToggleExpansion)
{
	OnToggleExpansion = InOnToggleExpansion;
}

void FPerPlatformPropertyCustomNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& HeaderRow)
{
	// Build Platform menu
	FMenuBuilder AddPlatformMenuBuilder(true, nullptr, nullptr, true);

	const TArray<const FDataDrivenPlatformInfo*>& SortedPlatforms = FDataDrivenPlatformInfoRegistry::GetSortedPlatformInfos(EPlatformInfoType::TruePlatformsOnly);
	// Platform (group) names
//	const TArray<FName>& PlatformGroupNameArray = PlatformInfo::GetAllPlatformGroupNames();

	// Sanitized platform names
	TArray<FName> BasePlatformNameArray;
	// Mapping from platform group name to individual platforms
	TMultiMap<FName, FName> GroupToPlatform;

	TArray<FName> PlatformOverrides = Args.PlatformOverrideNames.Get();
	TArray<FName> PlatformGroupNameArray;

	// Create mapping from platform to platform groups and remove postfixes and invalid platform names
	for (const FDataDrivenPlatformInfo* DDPI : SortedPlatforms)
	{
		// Add platform name if it isn't already set, and also add to group mapping
		if (!PlatformOverrides.Contains(DDPI->IniPlatformName))
		{
			BasePlatformNameArray.AddUnique(DDPI->IniPlatformName);
			GroupToPlatform.AddUnique(DDPI->PlatformGroupName, DDPI->IniPlatformName);
			PlatformGroupNameArray.AddUnique(DDPI->PlatformGroupName);
		}
	}

	// Create section for platform groups 
	const FName PlatformGroupSection(TEXT("PlatformGroupSection"));
	AddPlatformMenuBuilder.BeginSection(PlatformGroupSection, FText::FromString(TEXT("Platform Groups")));
	for (const FName& GroupName : PlatformGroupNameArray)
	{
		if (!PlatformOverrides.Contains(GroupName))
		{
			const FTextFormat Format = NSLOCTEXT("SPerPlatformPropertiesWidget", "AddOverrideGroupFor", "Add Override for Platforms part of the {0} Platform Group");
			AddPlatformToMenu(GroupName, Format, AddPlatformMenuBuilder);
		}
	}
	AddPlatformMenuBuilder.EndSection();

	for (const FName& GroupName : PlatformGroupNameArray)
	{
		// Create a section for each platform group and their respective platforms
		AddPlatformMenuBuilder.BeginSection(GroupName, FText::FromName(GroupName));

		TArray<FName> PlatformNames;
		GroupToPlatform.MultiFind(GroupName, PlatformNames);
		// these come out reversed for whatever MultiFind reason, even tho they went in sorted
		Algo::Reverse(PlatformNames);

		const FTextFormat Format = NSLOCTEXT("SPerPlatformPropertiesWidget", "AddOverrideFor", "Add Override specifically for {0}");
		for (const FName& PlatformName : PlatformNames)
		{
			AddPlatformToMenu(PlatformName, Format, AddPlatformMenuBuilder);
		}

		AddPlatformMenuBuilder.EndSection();
	}

	HeaderRow
	.FilterString(Args.FilterText)
	.IsEnabled(Args.IsEnabled)
	.NameContent()
	[
		Args.OnGenerateNameWidget.Execute()
	]
	.ValueContent()
	.MinDesiredWidth(125+28.0f)
	[
		SNew(SHorizontalBox)
		.ToolTipText(NSLOCTEXT("SPerPlatformPropertiesWidget", "DefaultPlatformDesc", "This property can have per-platform or platform group overrides.\nThis is the default value used when no override has been set for a platform or platform group."))
		+SHorizontalBox::Slot()
		[
			SNew(SPerPlatformPropertiesRow, NAME_None)
			.OnGenerateWidget(Args.OnGenerateWidgetForPlatformRow)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f, 0.0f, 0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.HasDownArrow(false)
			.ToolTipText(NSLOCTEXT("SPerPlatformPropertiesWidget", "AddOverrideToolTip", "Add an override for a specific platform or platform group"))
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			.MenuContent()
			[
				AddPlatformMenuBuilder.MakeWidget()
			]	
		]
	];
}

void FPerPlatformPropertyCustomNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	TArray<FName> PlatformOverrides = Args.PlatformOverrideNames.Get();
	for (FName PlatformName : PlatformOverrides)
	{
		FText PlatformDisplayName = FText::AsCultureInvariant(PlatformName.ToString());
		FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(PlatformDisplayName);
		Row.IsEnabled(Args.IsEnabled);

		Row.NameContent()
		[
			SNew(STextBlock)
			.Text(PlatformDisplayName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		Row.ValueContent()
		[
			SNew(SPerPlatformPropertiesRow, PlatformName)
			.OnGenerateWidget(Args.OnGenerateWidgetForPlatformRow)
			.OnRemovePlatform(this, &FPerPlatformPropertyCustomNodeBuilder::OnRemovePlatformOverride)
		];
	}
}

FName FPerPlatformPropertyCustomNodeBuilder::GetName() const
{
	return Args.Name;
}

void FPerPlatformPropertyCustomNodeBuilder::OnAddPlatformOverride(const FName PlatformName)
{
	if (Args.OnAddPlatformOverride.IsBound() && Args.OnAddPlatformOverride.Execute(PlatformName))
	{
		OnRebuildChildren.ExecuteIfBound();
		OnToggleExpansion.ExecuteIfBound(true);
	}
}

bool FPerPlatformPropertyCustomNodeBuilder::OnRemovePlatformOverride(const FName PlatformName)
{
	if (Args.OnRemovePlatformOverride.IsBound() && Args.OnRemovePlatformOverride.Execute(PlatformName))
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	return true;
}
void FPerPlatformPropertyCustomNodeBuilder::AddPlatformToMenu(const FName PlatformName, const FTextFormat Format, FMenuBuilder& AddPlatformMenuBuilder)
{
	const FText MenuText = FText::Format(FText::FromString(TEXT("{0}")), FText::AsCultureInvariant(PlatformName.ToString()));
	const FText MenuTooltipText = FText::Format(Format, FText::AsCultureInvariant(PlatformName.ToString()));
	AddPlatformMenuBuilder.AddMenuEntry(
		MenuText,
		MenuTooltipText,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "PerPlatformWidget.AddPlatform"),
		FUIAction(FExecuteAction::CreateSP(this, &FPerPlatformPropertyCustomNodeBuilder::OnAddPlatformOverride, PlatformName))
	);
}
