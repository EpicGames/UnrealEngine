// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerQualityLevelPropertyCustomization.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PerQualityLevelProperties.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SPerQualityLevelPropertiesWidget.h"
#include "ScopedTransaction.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "PerOverridePropertyCustomization"

template<typename OverrideType>
TSharedRef<SWidget> FPerQualityLevelPropertyCustomization<OverrideType>::GetWidget(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle) const
{
	TSharedPtr<IPropertyHandle>	EditProperty;

	if (InQualityLevelName == NAME_None)
	{
		EditProperty = StructPropertyHandle->GetChildHandle(FName("Default"));
	}
	else
	{
		TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerQuality"));
		if (MapProperty.IsValid())
		{
			uint32 NumChildren = 0;
			MapProperty->GetNumChildren(NumChildren);
			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = MapProperty->GetChildHandle(ChildIdx);
				if (ChildProperty.IsValid())
				{
					TSharedPtr<IPropertyHandle> KeyProperty = ChildProperty->GetKeyHandle();
					if (KeyProperty.IsValid())
					{
						int32 InQualityLevelNameToInt = INDEX_NONE;
						if (KeyProperty->GetValue(InQualityLevelNameToInt) == FPropertyAccess::Success && InQualityLevelNameToInt == QualityLevelProperty::FNameToQualityLevel(InQualityLevelName))
						{
							EditProperty = ChildProperty;
							break;
						}
					}
				}
			}
		}

	}

	// Push down struct metadata to per-quality level properties
	{
		// First get the source map
		const TMap<FName, FString>* SourceMap = StructPropertyHandle->GetMetaDataProperty()->GetMetaDataMap();
		// Iterate through source map, setting each key/value pair in the destination
		for (const auto& It : *SourceMap)
		{
			EditProperty->SetInstanceMetaData(*It.Key.ToString(), *It.Value);
		}

		// Copy instance metadata as well
		const TMap<FName, FString>* InstanceSourceMap = StructPropertyHandle->GetInstanceMetaDataMap();
		for (const auto& It : *InstanceSourceMap)
		{
			EditProperty->SetInstanceMetaData(*It.Key.ToString(), *It.Value);
		}
	}

	if (EditProperty.IsValid())
	{
		return EditProperty->CreatePropertyValueWidget(false);
	}
	else
	{
		return
			SNew(STextBlock)
			.Text(NSLOCTEXT("FPerQualityLevelPropertyCustomization", "GetWidget", "Could not find valid property"))
			.ColorAndOpacity(FLinearColor::Red);
	}
}

template<typename OverrideType>
float FPerQualityLevelPropertyCustomization<OverrideType>::CalcDesiredWidth(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	int32 NumOverrides = 0;
	TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerQuality"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			const TMap<int32, typename OverrideType::ValueType>* PerQualityLevelMap = (const TMap<int32, typename OverrideType::ValueType>*)(Data);
			NumOverrides = FMath::Max<int32>(PerQualityLevelMap->Num(), NumOverrides);
		}
	}
	return (float)(1 + NumOverrides) * 125.f;
}

template<typename OverrideType>
bool FPerQualityLevelPropertyCustomization<OverrideType>::AddOverride(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("AddOverride", "Add Quality Level Override"));

	bool bSucces = false;

	TSharedPtr<IPropertyHandle>	PerQualityLevelProperty = StructPropertyHandle->GetChildHandle(FName("PerQuality"));
	TSharedPtr<IPropertyHandle>	DefaultProperty = StructPropertyHandle->GetChildHandle(FName("Default"));
	if (PerQualityLevelProperty.IsValid() && DefaultProperty.IsValid())
	{
		TSharedPtr<IPropertyHandleMap> MapProperty = PerQualityLevelProperty->AsMap();
		if (MapProperty.IsValid())
		{
			// get the Tmap containing the Key,value
			TArray<const void*> RawData;
			PerQualityLevelProperty->AccessRawData(RawData);

			if (!RawData.IsEmpty())
			{
				TMap<int32, typename OverrideType::ValueType>* PerQualityLevelMap = (TMap<int32, typename OverrideType::ValueType>*)(RawData[0]);
				check(PerQualityLevelMap);

				int32 InQualityLevelNameToInt = QualityLevelProperty::FNameToQualityLevel(InQualityLevelName);
				if (PerQualityLevelMap->Find(InQualityLevelNameToInt) == nullptr)
				{
					typename OverrideType::ValueType DefaultValue;
					DefaultProperty->GetValue(DefaultValue);

					PerQualityLevelMap->Add(InQualityLevelNameToInt, DefaultValue);
					if (PropertyUtilities.IsValid())
					{
						PropertyUtilities.Pin()->ForceRefresh();
					}
					bSucces = true;
				}
			}
		}
	}
	return bSucces;
}

template<typename OverrideType>
bool FPerQualityLevelPropertyCustomization<OverrideType>::RemoveOverride(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{

	FScopedTransaction Transaction(LOCTEXT("RemoveQualityLevelOverride", "Remove Quality Level Override"));

	TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerQuality"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			TMap<int32, typename OverrideType::ValueType>* PerQualityLevelMap = (TMap<int32, typename OverrideType::ValueType>*)(Data);
			check(PerQualityLevelMap);
			TArray<int32> KeyArray;
			PerQualityLevelMap->GenerateKeyArray(KeyArray);
			int32 InQualityLevelNameToInt = QualityLevelProperty::FNameToQualityLevel(InQualityLevelName);
			for (int32 QL : KeyArray)
			{
				if (QL == InQualityLevelNameToInt)
				{
					PerQualityLevelMap->Remove(InQualityLevelNameToInt);
					if (PropertyUtilities.IsValid())
					{
						PropertyUtilities.Pin()->ForceRefresh();
					}
					return true;
				}
			}
		}
	}
	return false;

}

template<typename OverrideType>
TArray<FName> FPerQualityLevelPropertyCustomization<OverrideType>::GetOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const
{
	TArray<FName> QualityLevelOverrideNames;

	TSharedPtr<IPropertyHandle>	MapProperty = StructPropertyHandle->GetChildHandle(FName("PerQuality"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			const TMap<int32, typename OverrideType::ValueType>* PerQualityLevelMap = (const TMap<int32, typename OverrideType::ValueType>*)(Data);
			check(PerQualityLevelMap);
			TArray<int32> KeyArray;
			PerQualityLevelMap->GenerateKeyArray(KeyArray);
			for (int32 QL : KeyArray)
			{
				QualityLevelOverrideNames.AddUnique(QualityLevelProperty::QualityLevelToFName(QL));
			}
		}

	}
	return QualityLevelOverrideNames;
}

template<typename OverrideType>
void FPerQualityLevelPropertyCustomization<OverrideType>::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FPerQualityLevelPropertyCustomization<OverrideType>::PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(FPerQualityLevelPropertyCustomization<OverrideType>::CalcDesiredWidth(StructPropertyHandle))
		.MaxDesiredWidth((float)(static_cast<int32>(EPerQualityLevels::Num) + 1) * 125.0f)
		[
			SNew(SPerQualityLevelPropertiesWidget)
			.OnGenerateWidget(this, &FPerQualityLevelPropertyCustomization<OverrideType>::GetWidget, StructPropertyHandle)
			.OnAddEntry(this, &FPerQualityLevelPropertyCustomization<OverrideType>::AddOverride, StructPropertyHandle)
			.OnRemoveEntry(this, &FPerQualityLevelPropertyCustomization<OverrideType>::RemoveOverride, StructPropertyHandle)
			.EntryNames(this, &FPerQualityLevelPropertyCustomization<OverrideType>::GetOverrideNames, StructPropertyHandle)
		];
}

template<typename OverrideType>
TSharedRef<IPropertyTypeCustomization> FPerQualityLevelPropertyCustomization<OverrideType>::MakeInstance()
{
	return MakeShareable(new FPerQualityLevelPropertyCustomization<OverrideType>);
}



/* Only explicitly instantiate the types which are supported
*****************************************************************************/
template class FPerQualityLevelPropertyCustomization<FPerQualityLevelInt>;
template class FPerQualityLevelPropertyCustomization<FPerQualityLevelFloat>;

#undef LOCTEXT_NAMESPACE
