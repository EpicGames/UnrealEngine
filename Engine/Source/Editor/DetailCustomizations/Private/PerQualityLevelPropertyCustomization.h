// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class SWidget;

/**
* Implements a details panel customization for the FPerQualityLevel structures.
*/
template<typename OverrideType>
class FPerQualityLevelPropertyCustomization : public IPropertyTypeCustomization
{
public:
	FPerQualityLevelPropertyCustomization()
	{}

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

	/**
	* Creates a new instance.
	*
	* @return A new customization for FPerQualityLevel structs.
	*/
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	TSharedRef<SWidget> GetWidget(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle) const;
	TArray<FName> GetOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const;
	bool AddOverride(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle);
	bool RemoveOverride(FName InQualityLevelName, TSharedRef<IPropertyHandle> StructPropertyHandle);
	float CalcDesiredWidth(TSharedRef<IPropertyHandle> StructPropertyHandle);

private:
	/** Cached utils used for resetting customization when layout changes */
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
};

