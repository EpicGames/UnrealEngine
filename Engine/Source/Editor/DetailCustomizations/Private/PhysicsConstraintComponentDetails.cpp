// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsConstraintComponentDetails.h"

#include "CoreGlobals.h"
#include "Customizations/MathStructProxyCustomizations.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Features/IModularFeatures.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IPhysicsAssetRenderInterface.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Axis.h"
#include "Math/Matrix.h"
#include "Math/RotationMatrix.h"
#include "Math/Rotator.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/EnumClassFlags.h"
#include "PhysicsEngine/ConstraintDrives.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/ConstraintTypes.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintActor.h"
#include "PhysicsEngine/PhysicsConstraintComponent.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyTypeCustomization;

#define LOCTEXT_NAMESPACE "PhysicsConstraintComponentDetails"

namespace ConstraintCopyPasteStringTokens
{
	static const FString Position = "Position";
	static const FString Rotation = "Rotation";
};

// File scope utility functions.

bool IsNearlyEqual(const FVector A, const FVector B, float ErrorTolerance)
{
	return FMath::IsNearlyEqual(A.X, B.X, ErrorTolerance) && FMath::IsNearlyEqual(A.Y, B.Y, ErrorTolerance) && FMath::IsNearlyEqual(A.Z, B.Z, ErrorTolerance);
}

template< typename TPropertyValueType > bool IsChildPropertyNearlyEqual(TSharedPtr<IPropertyHandle> ParentPropertyHandle, const FName& LhsPropertyName, const TPropertyValueType& RhsValue)
{
	if (ParentPropertyHandle.IsValid())
	{
		TSharedPtr<IPropertyHandle> LhsPropertyHandle = ParentPropertyHandle->GetChildHandle(LhsPropertyName);

		if (LhsPropertyHandle.IsValid())
		{
			TPropertyValueType LhsValue;
			LhsPropertyHandle->GetValue(LhsValue);

			return IsNearlyEqual(LhsValue, RhsValue, SMALL_NUMBER);
		}
	}

	return false;
}

// Class FOrthonormalVectorPairStructCustomization.
FConstraintTransformCustomization::FConstraintTransformCustomization()
	: CachedRotation(MakeShareable(new TProxyValue<FRotator>(FRotator::ZeroRotator)))
	, CachedRotationYaw(MakeShareable(new TProxyProperty<FRotator, FRotator::FReal>(CachedRotation, CachedRotation->Get().Yaw)))
	, CachedRotationPitch(MakeShareable(new TProxyProperty<FRotator, FRotator::FReal>(CachedRotation, CachedRotation->Get().Pitch)))
	, CachedRotationRoll(MakeShareable(new TProxyProperty<FRotator, FRotator::FReal>(CachedRotation, CachedRotation->Get().Roll)))
	, CachedPosition(MakeShareable(new TProxyValue<FVector>(FVector::ZeroVector)))
	, CachedPositionX(MakeShareable(new TProxyProperty<FVector, FRotator::FReal>(CachedPosition, CachedPosition->Get().X)))
	, CachedPositionY(MakeShareable(new TProxyProperty<FVector, FRotator::FReal>(CachedPosition, CachedPosition->Get().Y)))
	, CachedPositionZ(MakeShareable(new TProxyProperty<FVector, FRotator::FReal>(CachedPosition, CachedPosition->Get().Z)))
	, DefaultTransform(FTransform::Identity)
	, InverseDefaultTransform(FTransform::Identity)
	, bPositionDisplayRelativeToDefault(false)
	, bRotationDisplayRelativeToDefault(false)
{}

TSharedRef<IPropertyTypeCustomization> FConstraintTransformCustomization::MakeInstance()
{
	return MakeShareable(new FConstraintTransformCustomization);
}

void FConstraintTransformCustomization::MakeRotationRow(TSharedRef<class IPropertyHandle>& InPriAxisPropertyHandle, TSharedRef<class IPropertyHandle>& InSecAxisPropertyHandle, FDetailWidgetRow& Row, TSharedRef<SWidget> EditSpaceToggleButtonWidget)
{
	TWeakPtr<IPropertyHandle> WeakPriAxisHandlePtr = InPriAxisPropertyHandle;
	TWeakPtr<IPropertyHandle> WeakSecAxisHandlePtr = InSecAxisPropertyHandle;

	const FText ComponentToolTipFormatText = LOCTEXT("ConstraintTransformRotationToolTip", "{0} component of the constraint rotation relative to the {1} bone (in degrees).");

	if (Row.IsPasteFromTextBound())
	{
		Row.OnPasteFromTextDelegate.Pin()->AddSP(this, &FConstraintTransformCustomization::OnPasteRotationFromText, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr);
	}
	
	Row
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FConstraintTransformCustomization::OnCopyRotation, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FConstraintTransformCustomization::OnPasteRotation, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr)))
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
		[
			InPriAxisPropertyHandle->CreatePropertyNameWidget(LOCTEXT("RotationLabel", "Rotation"), FText::Format(LOCTEXT("RotationToolTip", "Constraint rotation relative to the {0} bone (in degrees)."), FrameLabelText))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			EditSpaceToggleButtonWidget
		]
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeNumericProxyWidget<FRotator, FRotator::FReal>(InPriAxisPropertyHandle, InSecAxisPropertyHandle, CachedRotationRoll, FText::Format(ComponentToolTipFormatText, LOCTEXT("RotationRoll", "Roll"), FrameLabelText), true, AxisDisplayInfo::GetAxisColor(EAxisList::X))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeNumericProxyWidget<FRotator, FRotator::FReal>(InPriAxisPropertyHandle, InSecAxisPropertyHandle, CachedRotationPitch, FText::Format(ComponentToolTipFormatText, LOCTEXT("RotationPitch", "Pitch"), FrameLabelText), true, AxisDisplayInfo::GetAxisColor(EAxisList::Y))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
		[
			MakeNumericProxyWidget<FRotator, FRotator::FReal>(InPriAxisPropertyHandle, InSecAxisPropertyHandle, CachedRotationYaw, FText::Format(ComponentToolTipFormatText, LOCTEXT("RotationYaw", "Yaw"), FrameLabelText), true, AxisDisplayInfo::GetAxisColor(EAxisList::Z))
		]
	];
}

void FConstraintTransformCustomization::MakePositionRow(TSharedRef<class IPropertyHandle>& InPositionPropertyHandle, FDetailWidgetRow& Row, TSharedRef<SWidget> EditSpaceToggleButtonWidget)
{
	TWeakPtr<IPropertyHandle> WeakPositionHandlePtr = InPositionPropertyHandle;

	const FText ComponentToolTipFormatText = LOCTEXT("ConstraintTransformPositionToolTip", "{0} component of the constraint position relative to the {1} bone.");

	if (Row.IsPasteFromTextBound())
	{
		Row.OnPasteFromTextDelegate.Pin()->AddSP(this, &FConstraintTransformCustomization::OnPastePositionFromText, WeakPositionHandlePtr);
	}
	
	Row
	.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FConstraintTransformCustomization::OnCopyPosition, WeakPositionHandlePtr)))
	.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FConstraintTransformCustomization::OnPastePosition, WeakPositionHandlePtr)))
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
		[
			InPositionPropertyHandle->CreatePropertyNameWidget(LOCTEXT("PositionLabel", "Position"), FText::Format(LOCTEXT("PositionToolTip", "Constraint position relative to the {0} bone."), FrameLabelText))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			EditSpaceToggleButtonWidget
		]
	]
	.ValueContent()
	.MinDesiredWidth(375.0f)
	.MaxDesiredWidth(375.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeNumericProxyWidget<FVector, FVector::FReal>(InPositionPropertyHandle, CachedPositionX, FText::Format(ComponentToolTipFormatText, AxisDisplayInfo::GetAxisDisplayName(EAxisList::X), FrameLabelText), AxisDisplayInfo::GetAxisColor(EAxisList::X))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			MakeNumericProxyWidget<FVector, FVector::FReal>(InPositionPropertyHandle, CachedPositionY, FText::Format(ComponentToolTipFormatText, AxisDisplayInfo::GetAxisDisplayName(EAxisList::Y), FrameLabelText), AxisDisplayInfo::GetAxisColor(EAxisList::Y))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
		[
			MakeNumericProxyWidget<FVector, FVector::FReal>(InPositionPropertyHandle, CachedPositionZ, FText::Format(ComponentToolTipFormatText, AxisDisplayInfo::GetAxisDisplayName(EAxisList::Z), FrameLabelText), AxisDisplayInfo::GetAxisColor(EAxisList::Z))
		]
	];
}

void FConstraintTransformCustomization::OrthonormalVectorPairToDisplayedRotator(const FVector& PriAxis, const FVector& SecAxis, FRotator& OutRotator) const
{
	FMatrix Rotation = FRotationMatrix::MakeFromXY(PriAxis, SecAxis);

	// If rotation should be displayed relative to default then transform it from bone space.
	if (bRotationDisplayRelativeToDefault)
	{
		Rotation = Rotation * InverseDefaultTransform.ToMatrixNoScale();
	}

	OutRotator = Rotation.Rotator();
}

void FConstraintTransformCustomization::DisplayedRotatorToOrthonormalVectorPair(const FRotator& InRotator, FVector& OutPriAxis, FVector& OutSecAxis) const
{
	FMatrix Rotation = FRotationMatrix::Make(InRotator);

	// If rotation is displayed relative to default then transform it back to bone space.
	if (bRotationDisplayRelativeToDefault)
	{
		Rotation = Rotation * DefaultTransform.ToMatrixNoScale();
	}

	OutPriAxis = Rotation.GetUnitAxis(EAxis::X);
	OutSecAxis = Rotation.GetUnitAxis(EAxis::Y);
}

bool FConstraintTransformCustomization::CacheValues(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr) const
{
	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid())
	{
		return false;
	}

	FVector Position;

	if (PositionPropertyHandle->GetValue(Position) == FPropertyAccess::Success)
	{
		if (bPositionDisplayRelativeToDefault)
		{
			Position = InverseDefaultTransform.TransformPosition(Position);
		}

		CachedPosition->Set(Position);
		CachedPositionX->Set(Position.X);
		CachedPositionY->Set(Position.Y);
		CachedPositionZ->Set(Position.Z);

		return true;
	}

	return false;
}

bool FConstraintTransformCustomization::CacheValues(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr) const
{
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return false;
	}

	FVector PriAxis;
	FVector SecAxis;

	if ((PriAxisPropertyHandle->GetValue(PriAxis) == FPropertyAccess::Success) && (SecAxisPropertyHandle->GetValue(SecAxis) == FPropertyAccess::Success))
	{
		FRotator CurrentRotation;
		OrthonormalVectorPairToDisplayedRotator(PriAxis, SecAxis, CurrentRotation);

		CachedRotation->Set(CurrentRotation);
		CachedRotationPitch->Set(CurrentRotation.Pitch);
		CachedRotationYaw->Set(CurrentRotation.Yaw);
		CachedRotationRoll->Set(CurrentRotation.Roll);

		return true;
	}

	return false;
}

bool FConstraintTransformCustomization::FlushValues(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr) const
{
	// Update the constraint position from the one displayed in the details panel.

	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid())
	{
		return false;
	}

	// Read position from the property
	FVector Position;
	PositionPropertyHandle->GetValue(Position);

	// Update position with any changes from the UI.
	FVector ModifiedPosition(
		CachedPositionX->IsSet() ? CachedPositionX->Get() : Position.X,
		CachedPositionY->IsSet() ? CachedPositionY->Get() : Position.Y,
		CachedPositionZ->IsSet() ? CachedPositionZ->Get() : Position.Z
	);

	// If position is displayed relative to default then transform it back to bone space.
	if (bPositionDisplayRelativeToDefault)
	{
		ModifiedPosition = DefaultTransform.TransformPosition(ModifiedPosition);
	}

	// Write back to the property.
	return PositionPropertyHandle->SetValue(ModifiedPosition) == FPropertyAccess::Success;
}

bool FConstraintTransformCustomization::FlushValues(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr) const
{
	// Update the constraint rotation from the one displayed in the details panel.

	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return false;
	}

	FVector PriAxis;
	FVector SecAxis;
	PriAxisPropertyHandle->GetValue(PriAxis);
	SecAxisPropertyHandle->GetValue(SecAxis);
	
	FRotator CurrentRotation;
	OrthonormalVectorPairToDisplayedRotator(PriAxis, SecAxis, CurrentRotation);

	FRotator Rotation(
		CachedRotationPitch->IsSet() ? CachedRotationPitch->Get() : CurrentRotation.Pitch,
		CachedRotationYaw->IsSet() ? CachedRotationYaw->Get() : CurrentRotation.Yaw,
		CachedRotationRoll->IsSet() ? CachedRotationRoll->Get() : CurrentRotation.Roll
	);

	DisplayedRotatorToOrthonormalVectorPair(Rotation, PriAxis, SecAxis);

	bool Success = true;
	Success &= (PriAxisPropertyHandle->SetValue(PriAxis) == FPropertyAccess::Result::Success);
	Success &= (SecAxisPropertyHandle->SetValue(SecAxis) == FPropertyAccess::Result::Success);

	return Success;
}

void FConstraintTransformCustomization::SetDefaultTransform(const FTransform& InTransform, TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{ 
	DefaultTransform = InTransform;
	InverseDefaultTransform = InTransform.Inverse();

	CacheValues(PositionPropertyHandlePtr);
	CacheValues(PriAxisPropertyHandlePtr, SecAxisPropertyHandlePtr); // Update display values from actual properties (which as always stored as absolutes).
}

void FConstraintTransformCustomization::GetPositionAsFormattedString(FString& OutString)
{
	FVector Position = CachedPosition->Get();
	OutString = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), Position.X, Position.Y, Position.Z);
}

void FConstraintTransformCustomization::GetRotationAsFormattedString(FString& OutString)
{
	FRotator Rotation = CachedRotation->Get();
	OutString = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), Rotation.Pitch, Rotation.Yaw, Rotation.Roll);
}

void FConstraintTransformCustomization::GetValueAsFormattedString(FString& OutString)
{
	FString Buffer;
	GetPositionAsFormattedString(Buffer);
	OutString += ConstraintCopyPasteStringTokens::Position + Buffer;
	Buffer.Reset();
	GetRotationAsFormattedString(Buffer);
	OutString += ConstraintCopyPasteStringTokens::Rotation + Buffer;
}

bool FConstraintTransformCustomization::SetPositionFromFormattedString(const FString& InString)
{
	FVector Position;

	if (Position.InitFromString(InString))
	{
		CachedPosition->Set(Position);
		CachedPositionX->Set(Position.X);
		CachedPositionY->Set(Position.Y);
		CachedPositionZ->Set(Position.Z);

		return true;
	}

	return false;
}

bool FConstraintTransformCustomization::SetRotationFromFormattedString(const FString& InString)
{
	FString MutableString = InString;

	FRotator Rotation;
	MutableString.ReplaceInline(TEXT("Pitch="), TEXT("P="));
	MutableString.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
	MutableString.ReplaceInline(TEXT("Roll="), TEXT("R="));

	if (Rotation.InitFromString(MutableString))
	{
		CachedRotation->Set(Rotation);
		CachedRotationPitch->Set(Rotation.Pitch);
		CachedRotationYaw->Set(Rotation.Yaw);
		CachedRotationRoll->Set(Rotation.Roll);

		return true;
	}

	return false;
}

bool FConstraintTransformCustomization::SetValueFromFormattedString(const FString& InString)
{
	bool Success = true;

	FString FormattedString = InString;
	FString SubString;
	
	FormattedString.Split(ConstraintCopyPasteStringTokens::Rotation, &FormattedString, &SubString);
	Success &= SetRotationFromFormattedString(SubString);

	FormattedString.Split(ConstraintCopyPasteStringTokens::Position, &FormattedString, &SubString);
	Success &= SetPositionFromFormattedString(SubString);

	return Success;
}

void FConstraintTransformCustomization::OnCopy(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid() || !PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return;
	}

	CacheValues(PositionPropertyHandle);
	CacheValues(PriAxisPropertyHandle, SecAxisPropertyHandle);

	FString CopyStr;
	GetValueAsFormattedString(CopyStr);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FConstraintTransformCustomization::OnPaste(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	OnPasteFromText(TEXT(""), PastedText, {}, PositionPropertyHandlePtr, PriAxisPropertyHandlePtr, SecAxisPropertyHandlePtr);
}

void FConstraintTransformCustomization::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr,
	TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr,
	TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid() || !PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PastePosition", "Paste Position"));
		SetValueFromFormattedString(InText);
		FlushValues(PriAxisPropertyHandle, SecAxisPropertyHandle);
		FlushValues(PositionPropertyHandle);
	}
}

void FConstraintTransformCustomization::OnCopyPosition(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid())
	{
		return;
	}

	CacheValues(PositionPropertyHandle);

	FString CopyStr;
	GetPositionAsFormattedString(CopyStr);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FConstraintTransformCustomization::OnCopyRotation(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return;
	}

	CacheValues(PriAxisPropertyHandle, SecAxisPropertyHandle);

	FString CopyStr;
	GetRotationAsFormattedString(CopyStr);

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FConstraintTransformCustomization::OnPastePosition(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	PastePositionFromText(TEXT(""), PastedText, PositionPropertyHandlePtr);
}

void FConstraintTransformCustomization::OnPastePositionFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr)
{
	PastePositionFromText(InTag, InText, PositionPropertyHandlePtr);
}

void FConstraintTransformCustomization::PastePositionFromText(
	const FString& InTag,
	const FString& InText,
	TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PositionPropertyHandle = PositionPropertyHandlePtr.Pin();

	if (!PositionPropertyHandle.IsValid())
	{
		return;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PastePosition", "Paste Position"));
		SetPositionFromFormattedString(InText);
		FlushValues(PositionPropertyHandle);
	}
}

void FConstraintTransformCustomization::OnPasteRotation(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	PasteRotationFromText(TEXT(""), PastedText, PriAxisPropertyHandlePtr, SecAxisPropertyHandlePtr);
}

void FConstraintTransformCustomization::OnPasteRotationFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr,
	TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	PasteRotationFromText(InTag, InText, PriAxisPropertyHandlePtr, SecAxisPropertyHandlePtr);
}

void FConstraintTransformCustomization::PasteRotationFromText(
	const FString& InTag,
	const FString& InText,
	TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr,
	TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr)
{
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle = PriAxisPropertyHandlePtr.Pin();
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle = SecAxisPropertyHandlePtr.Pin();

	if (!PriAxisPropertyHandle.IsValid() || !SecAxisPropertyHandle.IsValid())
	{
		return;
	}

	{
		FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
		SetRotationFromFormattedString(InText);
		FlushValues(PriAxisPropertyHandle, SecAxisPropertyHandle);
	}
}

template<typename ProxyType, typename NumericType> TSharedRef<SWidget> FConstraintTransformCustomization::MakeNumericProxyWidget(TSharedRef<IPropertyHandle>& PriAxisPropertyHandle, TSharedRef<IPropertyHandle>& SecAxisPropertyHandle, TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue, const FText& ToolTipText, bool bRotationInDegrees, const FLinearColor& LabelBackgroundColor)
{
	TWeakPtr<IPropertyHandle> WeakPriAxisHandlePtr = PriAxisPropertyHandle;
	TWeakPtr<IPropertyHandle> WeakSecAxisHandlePtr = SecAxisPropertyHandle;

	return
		SNew(SNumericEntryBox<NumericType>)
		.IsEnabled(this, &FConstraintTransformCustomization::IsRotationValueEnabled, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr)
		.Value(this, &FConstraintTransformCustomization::OnGetRotationValue<ProxyType, NumericType>, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr, ProxyValue)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.OnValueCommitted(this, &FConstraintTransformCustomization::OnRotationValueCommitted<ProxyType, NumericType>, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr, ProxyValue)
		.OnValueChanged(this, &FConstraintTransformCustomization::OnRotationValueChanged<ProxyType, NumericType>, WeakPriAxisHandlePtr, WeakSecAxisHandlePtr, ProxyValue)
		.AllowSpin(false)
		.MinValue(TOptional<NumericType>())
		.MaxValue(TOptional<NumericType>())
		.MaxSliderValue(bRotationInDegrees ? 360.0f : TOptional<NumericType>())
		.MinSliderValue(bRotationInDegrees ? 0.0f : TOptional<NumericType>())
		.LabelPadding(FMargin(3))
		.ToolTipText(ToolTipText)
		.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
		.Label()
		[
			SNumericEntryBox<NumericType>::BuildNarrowColorLabel(LabelBackgroundColor)
		];
}

template<typename ProxyType, typename NumericType> TSharedRef<SWidget> FConstraintTransformCustomization::MakeNumericProxyWidget(TSharedRef<IPropertyHandle>& PositionPropertyHandle, TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue, const FText& ToolTipText, const FLinearColor& LabelBackgroundColor)
{
	TWeakPtr<IPropertyHandle> WeakPositionHandlePtr = PositionPropertyHandle;

	return
		SNew(SNumericEntryBox<NumericType>)
		.IsEnabled(this, &FConstraintTransformCustomization::IsPositionValueEnabled, WeakPositionHandlePtr)
		.Value(this, &FConstraintTransformCustomization::OnGetPositionValue<ProxyType, NumericType>, WeakPositionHandlePtr, ProxyValue)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.OnValueCommitted(this, &FConstraintTransformCustomization::OnPositionValueCommitted<ProxyType, NumericType>, WeakPositionHandlePtr, ProxyValue)
		.OnValueChanged(this, &FConstraintTransformCustomization::OnPositionValueChanged<ProxyType, NumericType>, WeakPositionHandlePtr, ProxyValue)
		.AllowSpin(false)
		.LabelPadding(FMargin(3))
		.ToolTipText(ToolTipText)
		.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
		.Label()
		[
			SNumericEntryBox<NumericType>::BuildNarrowColorLabel(LabelBackgroundColor)
		];
}


bool FConstraintTransformCustomization::IsRotationValueEnabled(TWeakPtr<IPropertyHandle> PriAxisWeakHandlePtr, TWeakPtr<IPropertyHandle> SecAxisWeakHandlePtr) const
{
	return (PriAxisWeakHandlePtr.IsValid() && !PriAxisWeakHandlePtr.Pin()->IsEditConst()) && (SecAxisWeakHandlePtr.IsValid() && !SecAxisWeakHandlePtr.Pin()->IsEditConst());
}

template<typename ProxyType, typename NumericType>
TOptional<NumericType> FConstraintTransformCustomization::OnGetRotationValue(TWeakPtr<IPropertyHandle> PriAxisWeakHandlePtr, TWeakPtr<IPropertyHandle> SecAxisWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue) const
{
	if (CacheValues(PriAxisWeakHandlePtr, SecAxisWeakHandlePtr))
	{
		return ProxyValue->Get();
	}
	return TOptional<NumericType>();
}

template<typename ProxyType, typename NumericType>
void FConstraintTransformCustomization::OnRotationValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> PriAxisWeakHandlePtr, TWeakPtr<IPropertyHandle> SecAxisWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue)
{
	if (!bIsUsingSlider && !GIsTransacting)
	{
		ProxyValue->Set(NewValue);
		FlushValues(PriAxisWeakHandlePtr, SecAxisWeakHandlePtr);
	}
}

template<typename ProxyType, typename NumericType>
void FConstraintTransformCustomization::OnRotationValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> PriAxisWeakHandlePtr, TWeakPtr<IPropertyHandle> SecAxisWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue)
{
	if (bIsUsingSlider)
	{
		ProxyValue->Set(NewValue);
		FlushValues(PriAxisWeakHandlePtr, SecAxisWeakHandlePtr);
	}
}

bool FConstraintTransformCustomization::IsPositionValueEnabled(TWeakPtr<IPropertyHandle> PositionWeakHandlePtr) const
{
	return (PositionWeakHandlePtr.IsValid() && !PositionWeakHandlePtr.Pin()->IsEditConst());
}

template<typename ProxyType, typename NumericType> TOptional<NumericType> FConstraintTransformCustomization::OnGetPositionValue(TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue) const
{
	if (CacheValues(PositionWeakHandlePtr))
	{
		return ProxyValue->Get();
	}
	return TOptional<NumericType>();
}

template<typename ProxyType, typename NumericType>
void FConstraintTransformCustomization::OnPositionValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue)
{
	if (!GIsTransacting)
	{
		ProxyValue->Set(NewValue);
		FlushValues(PositionWeakHandlePtr);
	}
}

template<typename ProxyType, typename NumericType>
void FConstraintTransformCustomization::OnPositionValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue)
{
	ProxyValue->Set(NewValue);
	FlushValues(PositionWeakHandlePtr);
}

void FConstraintTransformCustomization::SetFrameLabelText(const FText InText)
{
	FrameLabelText = InText;
}

FText& FConstraintTransformCustomization::GetFrameLabelText()
{
	return FrameLabelText;
}

namespace ConstraintDetails
{
	bool GetBoolProperty(TSharedPtr<IPropertyHandle> Prop)
	{
		bool bIsEnabled = false;

		if (Prop->GetValue(bIsEnabled) == FPropertyAccess::Result::Success)
		{
			return bIsEnabled;
		}
		return false;
	}

	TSharedRef<SWidget> JoinPropertyWidgets(TSharedPtr<IPropertyHandle> TargetProperty, FName TargetChildName, TSharedPtr<IPropertyHandle> ParentProperty, FName CheckPropertyName, TSharedPtr<IPropertyHandle>& StoreCheckProperty)
	{
		StoreCheckProperty = ParentProperty->GetChildHandle(CheckPropertyName);
		StoreCheckProperty->MarkHiddenByCustomization();
		TSharedRef<SWidget> TargetWidget = TargetProperty->GetChildHandle(TargetChildName)->CreatePropertyValueWidget();
		TargetWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([StoreCheckProperty]()
		{
			bool bSet;
			if (StoreCheckProperty->GetValue(bSet) == FPropertyAccess::Result::Success)
			{
				return bSet;
			}

			return false;
		})));

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 5, 0)
			[
				StoreCheckProperty->CreatePropertyValueWidget()
			]
		+ SHorizontalBox::Slot()
			[
				TargetWidget
			];
	}

	TSharedRef<SWidget> CreateTriFloatWidget(TSharedPtr<IPropertyHandle> Prop1, TSharedPtr<IPropertyHandle> Prop2, TSharedPtr<IPropertyHandle> Prop3, const FText& TransactionName)
	{
		auto GetMultipleFloats = [Prop1, Prop2, Prop3]()
		{
			// RerunConstructionScripts gets run when the new value is set (if the component
			// is part of a blueprint). This causes the Objects being edited to be cleared,
			// and will cause GetValue to fail. Skip checking the values in that case.
			if (Prop1->GetNumPerObjectValues())
			{
				float Val1, Val2, Val3;

				ensure(Prop1->GetValue(Val1) != FPropertyAccess::Fail);
				ensure(Prop2->GetValue(Val2) != FPropertyAccess::Fail);
				ensure(Prop3->GetValue(Val3) != FPropertyAccess::Fail);

				if (Val1 == Val2 && Val2 == Val3)
				{
					return TOptional<float>(Val1);
				}
			}

			return TOptional<float>();
		};

		auto SetMultipleFloatsCommitted = [Prop1, TransactionName, GetMultipleFloats](float NewValue, ETextCommit::Type)
		{
			TOptional<float> CommonFloat = GetMultipleFloats();
			if(!CommonFloat.IsSet() || CommonFloat.GetValue() != NewValue)	//don't bother doing it twice
			{
				// Only set the first property. Others should be handled in PostEditChangeChainProperty.
				// This prevents an issue where multiple sets fail when using BlueprintComponents
				// due to RerunConstructionScripts destroying the edit list.
				FScopedTransaction Transaction(TransactionName);
				ensure(Prop1->SetValue(NewValue) == FPropertyAccess::Result::Success);
			}
		};

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SNumericEntryBox<float>)
				.OnValueCommitted_Lambda(SetMultipleFloatsCommitted)
				.Value_Lambda(GetMultipleFloats)
				.MinValue(0.f)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked_Lambda([Prop1, Prop2, Prop3, TransactionName]()
				{
					FScopedTransaction Transaction(TransactionName);
					Prop1->ResetToDefault();
					Prop2->ResetToDefault();
					Prop3->ResetToDefault();
					return FReply::Handled();
				} )
				.Visibility_Lambda([Prop1]() { return Prop1->DiffersFromDefault() ? EVisibility::Visible : EVisibility::Collapsed; })
				.ContentPadding(FMargin(5.f, 0.f))
				.ToolTipText(Prop1->GetResetToDefaultLabel())
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
				]
			];
	}


	bool IsAngularPropertyEqual(TSharedPtr<IPropertyHandle> Prop, EAngularConstraintMotion CheckMotion)
	{
		uint8 Val;
		if (Prop->GetValue(Val) == FPropertyAccess::Result::Success)
		{
			return Val == CheckMotion;
		}
		return false;
	}

}

FPhysicsConstraintComponentDetails::FPhysicsConstraintComponentDetails()
: ChildTransformProxy(StaticCastSharedRef<FConstraintTransformCustomization>(FConstraintTransformCustomization::MakeInstance()))
, ParentTransformProxy(StaticCastSharedRef<FConstraintTransformCustomization>(FConstraintTransformCustomization::MakeInstance()))
{}

TSharedRef<IDetailCustomization> FPhysicsConstraintComponentDetails::MakeInstance()
{
	return MakeShareable(new FPhysicsConstraintComponentDetails());
}

void FPhysicsConstraintComponentDetails::AddConstraintProperties(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TArray<TWeakObjectPtr<UObject>>& Objects)
{
	ChildPositionPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, Pos1));
	ChildPriAxisPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, PriAxis1));
	ChildSecAxisPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, SecAxis1));

	ParentPositionPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, Pos2));
	ParentPriAxisPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, PriAxis2));
	ParentSecAxisPropertyHandle = ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, SecAxis2));

	IDetailCategoryBuilder& ConstraintCategory = DetailBuilder.EditCategory("Constraint");

	if (bInPhat)
	{
		// Add current profile name to the constraint category header.
		ConstraintCategory.HeaderContent(
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.DecoratorStyleSet(&FAppStyle::Get())
				.Text_Lambda([Objects]()
				{
					if (Objects.Num() > 0)
					{
						if (UPhysicsConstraintTemplate* Constraint = Cast<UPhysicsConstraintTemplate>(Objects[0].Get()))
						{
							FName CurrentProfileName = Constraint->GetCurrentConstraintProfileName();
							if (CurrentProfileName != NAME_None)
							{
								if (Constraint->ContainsConstraintProfile(CurrentProfileName))
								{
									return FText::Format(LOCTEXT("ProfileFormatAssigned", "Assigned to Profile: <RichTextBlock.Bold>{0}</>"), FText::FromName(CurrentProfileName));
								}
								else
								{
									return FText::Format(LOCTEXT("ProfileFormatNotAssigned", "Not Assigned to Profile: <RichTextBlock.Bold>{0}</>"), FText::FromName(CurrentProfileName));
								}
							}
							else
							{
								return LOCTEXT("ProfileFormatNone", "Current Profile: <RichTextBlock.Bold>None</>");
							}
						}
					}

					return FText();
				})
			]);

		// Add PhAT specific components to details panel.
		ConstraintCategory.AddProperty(ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, ConstraintBone1))).DisplayName(LOCTEXT("ConstraintChildBoneName", "Child Bone Name"));
		ConstraintCategory.AddProperty(ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, ConstraintBone2))).DisplayName(LOCTEXT("ConstraintParentBoneName", "Parent Bone Name"));

		IDetailCategoryBuilder& ConstraintTransformsCat = DetailBuilder.EditCategory("ConstraintTransforms");

		AddConstraintFrameTransform(EConstraintFrame::Frame1, ConstraintTransformsCat, ConstraintInstance);
		AddConstraintFrameTransform(EConstraintFrame::Frame2, ConstraintTransformsCat, ConstraintInstance);

		UpdateTransformProxyDisplayRelativeToDefault(ConstraintInstance);

		// Hide the constraint positions as they are represented by the proxy values in PhAT.
		ChildPositionPropertyHandle->MarkHiddenByCustomization();
		ParentPositionPropertyHandle->MarkHiddenByCustomization();
	}

	// Always hide the constraint orientation values as they are represented by the proxy values in PhAT and hidden by design everywhere else.
	ChildPriAxisPropertyHandle->MarkHiddenByCustomization();
	ChildSecAxisPropertyHandle->MarkHiddenByCustomization();
	ParentPriAxisPropertyHandle->MarkHiddenByCustomization();
	ParentSecAxisPropertyHandle->MarkHiddenByCustomization();
}	

TSharedRef<SWidget> FPhysicsConstraintComponentDetails::MakeEditSpaceToggleButtonWidget(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags ComponentFlags)
{
	auto ToolTipTextLambda = [this, ComponentFlags]()
	{
		const FText FrameText = EnumHasAnyFlags(ComponentFlags, EConstraintTransformComponentFlags::AllChild) ? LOCTEXT("Child", "Child") : LOCTEXT("Parent", "Parent");
		const FText ComponentText = EnumHasAnyFlags(ComponentFlags, EConstraintTransformComponentFlags::AllPosition) ? LOCTEXT("Position", "Position") : LOCTEXT("Rotation", "Rotation");
		const FText LocalFrameText = FText::Format(LOCTEXT("LocalFrameDescription", "in the frame of the {0} bone"), FrameText);
		const FText SnapFrameText = LOCTEXT("SnapFrameDescription", "relative to the default (snapped) transforms");

		const bool IsDisplayingRelativeToDefault = this->IsDisplayingConstraintTransformComponentRelativeToDefault(ComponentFlags);
		const FText CurrentFrameText = IsDisplayingRelativeToDefault ? SnapFrameText : LocalFrameText;
		const FText AlternativeFrameText = IsDisplayingRelativeToDefault ? LocalFrameText : SnapFrameText;

		return FText::Format(LOCTEXT("ToggleDisplayConstraintTransformComponentRelativeToDefault", "{0} transform {1} component displayed {2}. Click here to switch to displaying {3}. Hold Shift to change all."), FrameText, ComponentText, CurrentFrameText, AlternativeFrameText);
	};

	return
		SNew(SButton)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.OnClicked_Lambda([this, ConstraintInstancePropertyHandle, ComponentFlags]() { this->ToggleDisplayConstraintTransformComponentRelativeToDefault(ConstraintInstancePropertyHandle, ComponentFlags); return FReply::Handled(); })
		.ButtonColorAndOpacity(FSlateColor::UseForeground())
		.Content()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image_Lambda([this, ComponentFlags]() { return (IsDisplayingConstraintTransformComponentRelativeToDefault(ComponentFlags)) ? FAppStyle::GetBrush("Icons.Snap") : FAppStyle::GetBrush("Icons.Transform"); })
			.ToolTipText_Lambda(ToolTipTextLambda)
		];
}

void FPhysicsConstraintComponentDetails::AddConstraintFrameTransform(const EConstraintFrame::Type ConstraintFrameType, IDetailCategoryBuilder& ConstraintCat, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle)
{
	TSharedPtr<FConstraintTransformCustomization> TransformProxy;
	TSharedPtr<IPropertyHandle> PositionPropertyHandle;
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle;
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle;
	EConstraintTransformComponentFlags PositionSnapFlag = EConstraintTransformComponentFlags::None;
	EConstraintTransformComponentFlags RotationSnapFlag = EConstraintTransformComponentFlags::None;
	FText KeyboardShortcutText;

	if (ConstraintFrameType == EConstraintFrame::Frame1) // Child Frame
	{
		TransformProxy = ChildTransformProxy;
		TransformProxy->SetFrameLabelText(LOCTEXT("Child", "Child"));
		KeyboardShortcutText = LOCTEXT("ShiftAlt", "[Shift + Alt]");
		PositionPropertyHandle = ChildPositionPropertyHandle;
		PriAxisPropertyHandle = ChildPriAxisPropertyHandle;
		SecAxisPropertyHandle = ChildSecAxisPropertyHandle;
		PositionSnapFlag = EConstraintTransformComponentFlags::ChildPosition;
		RotationSnapFlag = EConstraintTransformComponentFlags::ChildRotation;
	}
	
	if (ConstraintFrameType == EConstraintFrame::Frame2) // Parent Frame
	{
		TransformProxy = ParentTransformProxy;
		TransformProxy->SetFrameLabelText(LOCTEXT("Parent", "Parent"));
		KeyboardShortcutText = LOCTEXT("Alt", "[Alt]");
		PositionPropertyHandle = ParentPositionPropertyHandle;
		PriAxisPropertyHandle = ParentPriAxisPropertyHandle;
		SecAxisPropertyHandle = ParentSecAxisPropertyHandle;
		PositionSnapFlag = EConstraintTransformComponentFlags::ParentPosition;
		RotationSnapFlag = EConstraintTransformComponentFlags::ParentRotation;
	}

	const EConstraintTransformComponentFlags SnapFlag = PositionSnapFlag | RotationSnapFlag;
		
	if (TransformProxy && PositionPropertyHandle && PriAxisPropertyHandle && SecAxisPropertyHandle)
	{
		// Create list of properties to be associated with the proxy custom rows.
		TArray<TSharedPtr<IPropertyHandle>> TransformProxyPropertyHandleList;
		TransformProxyPropertyHandleList.Add(ConstraintInstancePropertyHandle);

		FDetailWidgetRow& HeaderRow = ConstraintCat.AddCustomRow(TransformProxy->GetFrameLabelText());

		HeaderRow
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(TransformProxy->GetFrameLabelText())
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.ColorAndOpacity(this, &FPhysicsConstraintComponentDetails::GetConstraintTransformColorAndOpacity, SnapFlag)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(KeyboardShortcutText)
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					.ColorAndOpacity(this, &FPhysicsConstraintComponentDetails::GetConstraintTransformColorAndOpacity, SnapFlag)
				]
			];

		HeaderRow.PropertyHandleList(TransformProxyPropertyHandleList);

		HeaderRow.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FPhysicsConstraintComponentDetails::OnCopyConstraintTransform, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle, TransformProxy)));
		HeaderRow.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FPhysicsConstraintComponentDetails::OnPasteConstraintTransform, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle, TransformProxy)));

		if (HeaderRow.IsPasteFromTextBound())
		{
			HeaderRow.OnPasteFromTextDelegate.Pin()->AddSP(this, &FPhysicsConstraintComponentDetails::OnPasteConstraintTransformFromText, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle, TransformProxy);
		}
		
		HeaderRow.OverrideResetToDefault(
			FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateSP(this, &FPhysicsConstraintComponentDetails::IsSnapConstraintTransformComponentVisible, SnapFlag),
				FResetToDefaultHandler::CreateSP(this, &FPhysicsConstraintComponentDetails::SnapConstraintTransformComponentsToDefault, SnapFlag)
			));

		PositionPropertyHandle->MarkHiddenByCustomization();
		PriAxisPropertyHandle->MarkHiddenByCustomization();
		SecAxisPropertyHandle->MarkHiddenByCustomization();

		TSharedRef<IPropertyHandle> PositionPropertyHandleRef = PositionPropertyHandle.ToSharedRef();
		TSharedRef<IPropertyHandle> PriAxisPropertyHandleRef = PriAxisPropertyHandle.ToSharedRef();
		TSharedRef<IPropertyHandle> SecAxisPropertyHandleRef = SecAxisPropertyHandle.ToSharedRef();

		// Create Position Row
		{
			// Create a new, empty row in the details panel.
			FDetailWidgetRow& ProxyRow = ConstraintCat.AddCustomRow(LOCTEXT("ConstraintPositionProxy", "ConstraintPositionProxy")); // TODO - non localised ?

			// Associate the constraint property with the new row.
			ProxyRow.PropertyHandleList(TransformProxyPropertyHandleList);

			// Populate the new row with widgets generated by the rotation proxy object.
			TransformProxy->MakePositionRow(PositionPropertyHandleRef, ProxyRow, MakeEditSpaceToggleButtonWidget(ConstraintInstancePropertyHandle, PositionSnapFlag));

			// Set the behavior of the reset button for the new row, so that it snaps the orientation back to the default.
			ProxyRow.OverrideResetToDefault(
				FResetToDefaultOverride::Create(
					FIsResetToDefaultVisible::CreateSP(this, &FPhysicsConstraintComponentDetails::IsSnapConstraintTransformComponentVisible, PositionSnapFlag),
					FResetToDefaultHandler::CreateSP(this, &FPhysicsConstraintComponentDetails::SnapConstraintTransformComponentsToDefault, PositionSnapFlag)
				));

			ProxyRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsConstraintTransformComponentEnabled, SnapFlag)));
		}

		// Create Rotation Row
		{
			// Create a new, empty row in the details panel.
			FDetailWidgetRow& ProxyRow = ConstraintCat.AddCustomRow(LOCTEXT("ConstraintTransformProxy", "ConstraintTransformProxy"));

			// Associate the constraint property with the new row.
			ProxyRow.PropertyHandleList(TransformProxyPropertyHandleList);

			// Populate the new row with widgets generated by the rotation proxy object.
			TransformProxy->MakeRotationRow(PriAxisPropertyHandleRef, SecAxisPropertyHandleRef, ProxyRow, MakeEditSpaceToggleButtonWidget(ConstraintInstancePropertyHandle, RotationSnapFlag));

			// Set the behavior of the reset button for the new row, so that it snaps the orientation back to the default.
			ProxyRow.OverrideResetToDefault(
				FResetToDefaultOverride::Create(
					FIsResetToDefaultVisible::CreateSP(this, &FPhysicsConstraintComponentDetails::IsSnapConstraintTransformComponentVisible, RotationSnapFlag),
					FResetToDefaultHandler::CreateSP(this, &FPhysicsConstraintComponentDetails::SnapConstraintTransformComponentsToDefault, RotationSnapFlag)
				));

			ProxyRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsConstraintTransformComponentEnabled, SnapFlag)));
		}
	}
}

void FPhysicsConstraintComponentDetails::SnapConstraintTransformComponentsToDefault(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags SnapFlags)
{
	if (ParentPhysicsAsset)
	{
		if (FConstraintInstance* const ConstraintInstance = GetConstraintInstance())
		{
			ConstraintInstance->SnapTransformsToDefault(SnapFlags, ParentPhysicsAsset);
		}
	}
}

bool FPhysicsConstraintComponentDetails::IsSnapConstraintTransformComponentVisible(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags SnapFlags)
{
	bool Result = false;

	const FTransform ParentTransform = ParentTransformProxy->GetDefaultTransform();
	const FTransform ChildTransform = ChildTransformProxy->GetDefaultTransform();

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ChildPosition))
	{
		Result |= !IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, Pos1), ChildTransform.GetLocation());
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ChildRotation))
	{
		Result |= !(IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, PriAxis1), ChildTransform.GetUnitAxis(EAxis::X)) && IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, SecAxis1), ChildTransform.GetUnitAxis(EAxis::Y)));
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ParentPosition))
	{
		Result |= !IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, Pos2), ParentTransform.GetLocation());
	}

	if (EnumHasAnyFlags(SnapFlags, EConstraintTransformComponentFlags::ParentRotation))
	{
		Result |= !(IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, PriAxis2), ParentTransform.GetUnitAxis(EAxis::X)) && IsChildPropertyNearlyEqual< FVector >(ConstraintInstancePropertyHandle, GET_MEMBER_NAME_CHECKED(FConstraintInstance, SecAxis2), ParentTransform.GetUnitAxis(EAxis::Y)));
	}

	return Result;
}

void FPhysicsConstraintComponentDetails::ToggleDisplayConstraintTransformComponentRelativeToDefault(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags ComponentFlags)
{
	if (ParentPhysicsAsset)
	{
		IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());
		const bool bCurrentState = IsDisplayingConstraintTransformComponentRelativeToDefault(ComponentFlags);
		const EConstraintTransformComponentFlags ComponentFlagsToModify = (FSlateApplication::Get().GetModifierKeys().IsShiftDown()) ? EConstraintTransformComponentFlags::All : ComponentFlags;
		PhysicsAssetRenderInterface.SetDisplayConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, ComponentFlagsToModify, !bCurrentState);
		UpdateTransformProxyDisplayRelativeToDefault(ConstraintInstancePropertyHandle);
	}
}

bool FPhysicsConstraintComponentDetails::IsConstraintTransformComponentEnabled(const EConstraintTransformComponentFlags ComponentFlags) const
{
	if (ParentPhysicsAsset)
	{
		IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());
		const EConstraintTransformComponentFlags ComponentManipulationFlags = PhysicsAssetRenderInterface.GetConstraintViewportManipulationFlags(ParentPhysicsAsset);

		return EnumHasAllFlags(ComponentManipulationFlags, ComponentFlags);
	}

	return true;
}

FSlateColor FPhysicsConstraintComponentDetails::GetConstraintTransformColorAndOpacity(const EConstraintTransformComponentFlags ComponentFlags) const
{
	return IsConstraintTransformComponentEnabled(ComponentFlags) ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
}

void FPhysicsConstraintComponentDetails::UpdateTransformProxyDisplayRelativeToDefault(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle)
{
	if (ParentPhysicsAsset)
	{
		IPhysicsAssetRenderInterface& PhysicsAssetRenderInterface = IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName());

		ChildTransformProxy->SetPositionDisplayRelativeToDefault(PhysicsAssetRenderInterface.IsDisplayingConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, EConstraintTransformComponentFlags::ChildPosition));
		ChildTransformProxy->SetRotationDisplayRelativeToDefault(PhysicsAssetRenderInterface.IsDisplayingConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, EConstraintTransformComponentFlags::ChildRotation));
		ParentTransformProxy->SetPositionDisplayRelativeToDefault(PhysicsAssetRenderInterface.IsDisplayingConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, EConstraintTransformComponentFlags::ParentPosition));
		ParentTransformProxy->SetRotationDisplayRelativeToDefault(PhysicsAssetRenderInterface.IsDisplayingConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, EConstraintTransformComponentFlags::ParentRotation));

		if (FConstraintInstance* const ConstraintInstance = GetConstraintInstance())
		{
			ChildTransformProxy->SetDefaultTransform(ConstraintInstance->CalculateDefaultChildTransform(), ChildPositionPropertyHandle, ChildPriAxisPropertyHandle, ChildSecAxisPropertyHandle);
			ParentTransformProxy->SetDefaultTransform(ConstraintInstance->CalculateDefaultParentTransform(ParentPhysicsAsset), ParentPositionPropertyHandle, ParentPriAxisPropertyHandle, ParentSecAxisPropertyHandle);
		}
	}
}

bool FPhysicsConstraintComponentDetails::IsDisplayingConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags)
{
	if (ParentPhysicsAsset)
	{
		return IModularFeatures::Get().GetModularFeature<IPhysicsAssetRenderInterface>(IPhysicsAssetRenderInterface::GetModularFeatureName()).IsDisplayingConstraintTransformComponentRelativeToDefault(ParentPhysicsAsset, ComponentFlags);
	}

	return false;
}

void FPhysicsConstraintComponentDetails::OnCopyConstraintTransform(TSharedPtr<IPropertyHandle> PositionPropertyHandle, TSharedPtr<IPropertyHandle> PriAxisPropertyHandle, TSharedPtr<IPropertyHandle> SecAxisPropertyHandle, TSharedPtr<FConstraintTransformCustomization> TransformProxy)
{
	if (TransformProxy.IsValid())
	{
		TransformProxy->OnCopy(PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle);
	}
}

void FPhysicsConstraintComponentDetails::OnPasteConstraintTransform(TSharedPtr<IPropertyHandle> PositionPropertyHandle, TSharedPtr<IPropertyHandle> PriAxisPropertyHandle, TSharedPtr<IPropertyHandle> SecAxisPropertyHandle, TSharedPtr<FConstraintTransformCustomization> TransformProxy)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	
	PasteConstraintTransformFromText(TEXT(""), PastedText, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle, TransformProxy);
}

void FPhysicsConstraintComponentDetails::OnPasteConstraintTransformFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	TSharedPtr<IPropertyHandle> PositionPropertyHandle,
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle,
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle,
	TSharedPtr<FConstraintTransformCustomization> TransformProxy)
{
	if (TransformProxy.IsValid())
	{
		TransformProxy->OnPasteFromText(InTag, InText, InOperationId, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle);
	}
}

void FPhysicsConstraintComponentDetails::PasteConstraintTransformFromText(
	const FString& InTag,
	const FString& InText,
	TSharedPtr<IPropertyHandle> PositionPropertyHandle,
	TSharedPtr<IPropertyHandle> PriAxisPropertyHandle,
	TSharedPtr<IPropertyHandle> SecAxisPropertyHandle,
	TSharedPtr<FConstraintTransformCustomization> TransformProxy)
{
	OnPasteConstraintTransformFromText(InTag, InText, {}, PositionPropertyHandle, PriAxisPropertyHandle, SecAxisPropertyHandle, TransformProxy);
}

void FPhysicsConstraintComponentDetails::AddConstraintBehaviorProperties(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TSharedPtr<IPropertyHandle> ProfilePropertiesProperty)
{
	IDetailCategoryBuilder& ConstraintCat = DetailBuilder.EditCategory("Constraint Behavior");

	//hide the inner structs that we customize elsewhere
	ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearLimit))->MarkHiddenByCustomization();
	ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ConeLimit))->MarkHiddenByCustomization();
	ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, TwistLimit))->MarkHiddenByCustomization();
	ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearDrive))->MarkHiddenByCustomization();
	ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, AngularDrive))->MarkHiddenByCustomization();
	ProfilePropertiesProperty->MarkHiddenByCustomization();

	//Add properties we want in specific order
	ConstraintCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bDisableCollision)));
	ConstraintCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bParentDominates)));

	IDetailGroup& ProjectionGroup = ConstraintCat.AddGroup("Projection", LOCTEXT("Projection", "Projection"), false, true);
	ProjectionGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bEnableProjection)).ToSharedRef());
	ProjectionGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ProjectionLinearTolerance)).ToSharedRef());
	ProjectionGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ProjectionAngularTolerance)).ToSharedRef());
	ProjectionGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ProjectionLinearAlpha)).ToSharedRef());
	ProjectionGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ProjectionAngularAlpha)).ToSharedRef());

	IDetailGroup& ShockPropGroup = ConstraintCat.AddGroup("ShockPropagation", LOCTEXT("ShockPropagation", "Shock Propagation"), false, false);
	ShockPropGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bEnableShockPropagation)).ToSharedRef());
	ShockPropGroup.AddPropertyRow(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ShockPropagationAlpha)).ToSharedRef());

	//Add the rest
	uint32 NumProfileProperties = 0;
	ProfilePropertiesProperty->GetNumChildren(NumProfileProperties);

	for (uint32 ProfileChildIdx = 0; ProfileChildIdx < NumProfileProperties; ++ProfileChildIdx)
	{
		TSharedPtr<IPropertyHandle> ProfileChildProp = ProfilePropertiesProperty->GetChildHandle(ProfileChildIdx);
		if (!ProfileChildProp->IsCustomized())
		{
			ConstraintCat.AddProperty(ProfileChildProp);
		}
	}
}

void FPhysicsConstraintComponentDetails::AddLinearLimits(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TSharedPtr<IPropertyHandle> ProfilePropertiesProperty)
{
	IDetailCategoryBuilder& LinearLimitCat = DetailBuilder.EditCategory("Linear Limits");
	TSharedPtr<IPropertyHandle> LinearConstraintProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearLimit));

	TSharedPtr<IPropertyHandle> LinearXMotionProperty = LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, XMotion));
	TSharedPtr<IPropertyHandle> LinearYMotionProperty = LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, YMotion));
	TSharedPtr<IPropertyHandle> LinearZMotionProperty = LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, ZMotion));

	TArray<TSharedPtr<FString>> LinearLimitOptionNames;
	TArray<FText> LinearLimitOptionTooltips;
	TArray<bool> LinearLimitOptionRestrictItems;

	const int32 ExpectedLinearLimitOptionCount = 3;
	LinearXMotionProperty->GeneratePossibleValues(LinearLimitOptionNames, LinearLimitOptionTooltips, LinearLimitOptionRestrictItems);
	checkf(LinearLimitOptionNames.Num() == ExpectedLinearLimitOptionCount &&
		LinearLimitOptionTooltips.Num() == ExpectedLinearLimitOptionCount &&
		LinearLimitOptionRestrictItems.Num() == ExpectedLinearLimitOptionCount,
		TEXT("It seems the number of enum entries in ELinearConstraintMotion has changed. This must be handled here as well. "));


	uint8 LinearLimitEnum[LCM_MAX] = { LCM_Free, LCM_Limited, LCM_Locked };
	TSharedPtr<IPropertyHandle> LinearLimitProperties[] = { LinearXMotionProperty, LinearYMotionProperty, LinearZMotionProperty };


	for (int32 PropertyIdx = 0; PropertyIdx < 3; ++PropertyIdx)
	{
		TSharedPtr<IPropertyHandle> CurProperty = LinearLimitProperties[PropertyIdx];

		LinearLimitCat.AddProperty(CurProperty).CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(CurProperty->GetPropertyDisplayName())
				.ToolTipText(CurProperty->GetToolTipText())
			]
		.ValueContent()
			.MinDesiredWidth(125.0f * 3.0f)
			.MaxDesiredWidth(125.0f * 3.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "RadioButton")
					.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, LinearLimitEnum[0])
					.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, LinearLimitEnum[0])
					.ToolTipText(LinearLimitOptionTooltips[0])
					[
						SNew(STextBlock)
						.Text(FText::FromString(*LinearLimitOptionNames[0].Get()))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(5, 0, 0, 0)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "RadioButton")
						.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, LinearLimitEnum[1])
						.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, LinearLimitEnum[1])
						.ToolTipText(LinearLimitOptionTooltips[1])
						[
							SNew(STextBlock)
							.Text(FText::FromString(*LinearLimitOptionNames[1].Get()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(5, 0, 0, 0)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "RadioButton")
						.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, LinearLimitEnum[2])
						.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, LinearLimitEnum[2])
						.ToolTipText(LinearLimitOptionTooltips[2])
						[
							SNew(STextBlock)
							.Text(FText::FromString(*LinearLimitOptionNames[2].Get()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
			];
	}

	auto IsLinearMotionLimited = [LinearXMotionProperty, LinearYMotionProperty, LinearZMotionProperty]()
	{
		uint8 XMotion, YMotion, ZMotion;
		if (LinearXMotionProperty->GetValue(XMotion) == FPropertyAccess::Result::Success && 
			LinearYMotionProperty->GetValue(YMotion) == FPropertyAccess::Result::Success && 
			LinearZMotionProperty->GetValue(ZMotion) == FPropertyAccess::Result::Success)
		{
			return XMotion == LCM_Limited || YMotion == LCM_Limited || ZMotion == LCM_Limited;
		}

		return false;
	};

	TSharedPtr<IPropertyHandle> SoftProperty = LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, bSoftConstraint));

	auto IsRestitutionEnabled = [IsLinearMotionLimited, SoftProperty]()
	{
		return !ConstraintDetails::GetBoolProperty(SoftProperty) && IsLinearMotionLimited();
	};

	LinearLimitCat.AddProperty(LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, Limit))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsLinearMotionLimited)));
	LinearLimitCat.AddProperty(ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, bScaleLinearLimits))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsLinearMotionLimited)));
	LinearLimitCat.AddProperty(LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, bSoftConstraint))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsLinearMotionLimited)));
	LinearLimitCat.AddProperty(LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, Stiffness))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsLinearMotionLimited)));
	LinearLimitCat.AddProperty(LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, Damping))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsLinearMotionLimited)));
	LinearLimitCat.AddProperty(LinearConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearConstraint, Restitution))).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsRestitutionEnabled)));
	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bLinearBreakable)));
	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearBreakThreshold)));

	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bLinearPlasticity)).ToSharedRef());
	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearPlasticityType)).ToSharedRef());
	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearPlasticityThreshold)).ToSharedRef());

	// Mass Scale Properties
	LinearLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ContactTransferScale)).ToSharedRef());
}



void FPhysicsConstraintComponentDetails::AddAngularLimits(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TSharedPtr<IPropertyHandle> ProfilePropertiesProperty)
{
	IDetailCategoryBuilder& AngularLimitCat = DetailBuilder.EditCategory("Angular Limits");

	TSharedPtr<IPropertyHandle> ConeConstraintProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, ConeLimit));
	TSharedPtr<IPropertyHandle> TwistConstraintProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, TwistLimit));

	AngularSwing1MotionProperty = ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Swing1Motion));
	AngularSwing2MotionProperty = ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Swing2Motion));
	AngularTwistMotionProperty = TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, TwistMotion));

	TArray<TSharedPtr<FString>> AngularLimitOptionNames;
	TArray<FText> AngularLimitOptionTooltips;
	TArray<bool> AngularLimitOptionRestrictItems;

	const int32 ExpectedAngularLimitOptionCount = 3;
	AngularSwing1MotionProperty->GeneratePossibleValues(AngularLimitOptionNames, AngularLimitOptionTooltips, AngularLimitOptionRestrictItems);
	checkf(AngularLimitOptionNames.Num() == ExpectedAngularLimitOptionCount &&
		AngularLimitOptionTooltips.Num() == ExpectedAngularLimitOptionCount &&
		AngularLimitOptionRestrictItems.Num() == ExpectedAngularLimitOptionCount,
		TEXT("It seems the number of enum entries in EAngularConstraintMotion has changed. This must be handled here as well. "));


	uint8 AngularLimitEnum[LCM_MAX] = { ACM_Free, LCM_Limited, LCM_Locked };
	TSharedPtr<IPropertyHandle> AngularLimitProperties[] = { AngularSwing1MotionProperty, AngularSwing2MotionProperty, AngularTwistMotionProperty };

	const FName AxisStyleNames[3] =
	{
		"PhysicsAssetEditor.RadioButtons.Red",
		"PhysicsAssetEditor.RadioButtons.Red",
		"PhysicsAssetEditor.RadioButtons.Green"
	};

	for (int32 PropertyIdx = 0; PropertyIdx < 3; ++PropertyIdx)
	{
		TSharedPtr<IPropertyHandle> CurProperty = AngularLimitProperties[PropertyIdx];

		AngularLimitCat.AddProperty(CurProperty).CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(CurProperty->GetPropertyDisplayName())
				.ToolTipText(CurProperty->GetToolTipText())
			]
		.ValueContent()
			.MinDesiredWidth(125.0f * 3.0f)
			.MaxDesiredWidth(125.0f * 3.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), AxisStyleNames[PropertyIdx])
					.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, AngularLimitEnum[0])
					.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, AngularLimitEnum[0])
					.ToolTipText(AngularLimitOptionTooltips[0])
					[
						SNew(STextBlock)
						.Text(FText::FromString(*AngularLimitOptionNames[0].Get()))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(5, 0, 0, 0)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), AxisStyleNames[PropertyIdx])
						.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, AngularLimitEnum[1])
						.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, AngularLimitEnum[1])
						.ToolTipText(AngularLimitOptionTooltips[1])
						[
							SNew(STextBlock)
							.Text(FText::FromString(*AngularLimitOptionNames[1].Get()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(5, 0, 0, 0)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), AxisStyleNames[PropertyIdx])
						.IsChecked(this, &FPhysicsConstraintComponentDetails::IsLimitRadioChecked, CurProperty, AngularLimitEnum[2])
						.OnCheckStateChanged(this, &FPhysicsConstraintComponentDetails::OnLimitRadioChanged, CurProperty, AngularLimitEnum[2])
						.ToolTipText(AngularLimitOptionTooltips[2])
						[
							SNew(STextBlock)
							.Text(FText::FromString(*AngularLimitOptionNames[2].Get()))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
			];
	}

	AngularLimitCat.AddProperty(ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Swing1LimitDegrees)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularSwing1Limit)));
	AngularLimitCat.AddProperty(ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Swing2LimitDegrees)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularSwing2Limit)));
	AngularLimitCat.AddProperty(TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, TwistLimitDegrees)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularTwistLimit)));

	TSharedPtr<IPropertyHandle> SoftSwingProperty = ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, bSoftConstraint));
	auto SwingRestitutionEnabled = [this, SoftSwingProperty]()
	{
		return !ConstraintDetails::GetBoolProperty(SoftSwingProperty) && (IsPropertyEnabled(EPropertyType::AngularSwing1Limit) || IsPropertyEnabled(EPropertyType::AngularSwing2Limit));
	};

	IDetailGroup& SwingGroup = AngularLimitCat.AddGroup("Swing Limits", LOCTEXT("SwingLimits", "Swing Limits"), true, true);

	SwingGroup.AddPropertyRow(SoftSwingProperty.ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularSwingLimit)));
	SwingGroup.AddPropertyRow(ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Stiffness)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularSwingLimit)));
	SwingGroup.AddPropertyRow(ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Damping)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularSwingLimit)));
	SwingGroup.AddPropertyRow(ConeConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConeConstraint, Restitution)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(SwingRestitutionEnabled)));

	TSharedPtr<IPropertyHandle> SoftTwistProperty = TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, bSoftConstraint));
	auto TwistRestitutionEnabled = [this, SoftTwistProperty]()
	{
		return !ConstraintDetails::GetBoolProperty(SoftTwistProperty) && IsPropertyEnabled(EPropertyType::AngularTwistLimit);
	};

	IDetailGroup& TwistGroup = AngularLimitCat.AddGroup("Twist Limits", LOCTEXT("TwistLimits", "Twist Limits"), true, true);

	TwistGroup.AddPropertyRow(SoftTwistProperty.ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularTwistLimit)));
	TwistGroup.AddPropertyRow(TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, Stiffness)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularTwistLimit)));
	TwistGroup.AddPropertyRow(TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, Damping)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::AngularTwistLimit)));
	TwistGroup.AddPropertyRow(TwistConstraintProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTwistConstraint, Restitution)).ToSharedRef()).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(TwistRestitutionEnabled)));

	if (bInPhat == false)
	{
	    AngularLimitCat.AddProperty(ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, AngularRotationOffset)).ToSharedRef());
	}
	else
	{
		AngularLimitCat.AddProperty(ConstraintInstance->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, AngularRotationOffset)).ToSharedRef())
			.Visibility(EVisibility::Collapsed);
	}

	AngularLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bAngularBreakable)).ToSharedRef());
	AngularLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, AngularBreakThreshold)).ToSharedRef());
	AngularLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, bAngularPlasticity)).ToSharedRef());
	AngularLimitCat.AddProperty(ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, AngularPlasticityThreshold)).ToSharedRef());

}

void FPhysicsConstraintComponentDetails::AddLinearDrive(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TSharedPtr<IPropertyHandle> ProfilePropertiesProperty)
{
	IDetailCategoryBuilder& LinearMotorCat = DetailBuilder.EditCategory("LinearMotor");

	TSharedPtr<IPropertyHandle> LinearDriveProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, LinearDrive));

	TSharedPtr<IPropertyHandle> AccelerationModeProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, bAccelerationMode));
	LinearMotorCat.AddProperty(AccelerationModeProperty);

	IDetailGroup& PositionGroup = LinearMotorCat.AddGroup("Linear Position Drive", LOCTEXT("LinearPositionDrive", "Linear Position Drive"), false, true);

	TSharedRef<IPropertyHandle> LinearPositionTargetProperty = LinearDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, PositionTarget)).ToSharedRef();

	TSharedPtr<IPropertyHandle> XDriveProperty = LinearDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, XDrive));
	TSharedPtr<IPropertyHandle> YDriveProperty = LinearDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, YDrive));
	TSharedPtr<IPropertyHandle> ZDriveProperty = LinearDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, ZDrive));

	LinearXPositionDriveProperty = XDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));
	LinearYPositionDriveProperty = YDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));
	LinearZPositionDriveProperty = ZDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));

	TSharedRef<SWidget> LinearPositionXWidget = ConstraintDetails::JoinPropertyWidgets(LinearPositionTargetProperty, FName("X"), XDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive), LinearXPositionDriveProperty);
	TSharedRef<SWidget> LinearPositionYWidget = ConstraintDetails::JoinPropertyWidgets(LinearPositionTargetProperty, FName("Y"), YDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive), LinearYPositionDriveProperty);
	TSharedRef<SWidget> LinearPositionZWidget = ConstraintDetails::JoinPropertyWidgets(LinearPositionTargetProperty, FName("Z"), ZDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive), LinearZPositionDriveProperty);


	FDetailWidgetRow& LinearPositionTargetWidget = PositionGroup.HeaderProperty(LinearPositionTargetProperty).CustomWidget()
	.NameContent()
	[
			LinearPositionTargetProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125 * 3 + 18 * 3)
	.MaxDesiredWidth(125 * 3 + 18 * 3)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			LinearPositionXWidget
		]
		+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			LinearPositionYWidget
		]

		+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			LinearPositionZWidget
		]
	];

	TSharedPtr<IPropertyHandle> StiffnessXProperty = XDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness));
	TSharedRef<SWidget> StiffnessWidget = ConstraintDetails::CreateTriFloatWidget(StiffnessXProperty, YDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness)), ZDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness)), LOCTEXT("EditStrength", "Edit Strength"));
	StiffnessWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::LinearPositionDrive)));

	PositionGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Strength", "Strength"))
		.Font(IDetailLayoutBuilder::GetDetailFont()).ToolTipText(StiffnessXProperty->GetToolTipText())
	]
	.ValueContent()
	[
		StiffnessWidget
	];

	// VELOCITY

	IDetailGroup& VelocityGroup = LinearMotorCat.AddGroup("Linear Velocity Drive", LOCTEXT("LinearVelocityDrive", "Linear Velocity Drive"), false, true);

	TSharedRef<IPropertyHandle> LinearVelocityTargetProperty = LinearDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLinearDriveConstraint, VelocityTarget)).ToSharedRef();

	LinearXVelocityDriveProperty = XDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));
	LinearYVelocityDriveProperty = YDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));
	LinearZVelocityDriveProperty = ZDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));

	TSharedRef<SWidget> LinearVelocityXWidget = ConstraintDetails::JoinPropertyWidgets(LinearVelocityTargetProperty, FName("X"), XDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive), LinearXVelocityDriveProperty);
	TSharedRef<SWidget> LinearVelocityYWidget = ConstraintDetails::JoinPropertyWidgets(LinearVelocityTargetProperty, FName("Y"), YDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive), LinearYVelocityDriveProperty);
	TSharedRef<SWidget> LinearVelocityZWidget = ConstraintDetails::JoinPropertyWidgets(LinearVelocityTargetProperty, FName("Z"), ZDriveProperty, GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive), LinearZVelocityDriveProperty);

	FDetailWidgetRow& LinearVelocityTargetWidget = VelocityGroup.HeaderProperty(LinearVelocityTargetProperty).CustomWidget(true);
	LinearVelocityTargetWidget.NameContent()
		[
			LinearVelocityTargetProperty->CreatePropertyNameWidget()
		];

	LinearVelocityTargetWidget.ValueContent()
		.MinDesiredWidth(125 * 3 + 18 * 3)
		.MaxDesiredWidth(125 * 3 + 18 * 3)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		[
			LinearVelocityXWidget
		]
	+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			LinearVelocityYWidget
		]

	+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			LinearVelocityZWidget
		]
		];

	TSharedPtr<IPropertyHandle> XDampingProperty = XDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping));
	TSharedRef<SWidget> DampingWidget = ConstraintDetails::CreateTriFloatWidget(XDampingProperty, YDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping)), ZDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping)), LOCTEXT("EditStrength", "Edit Strength"));
	DampingWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::LinearVelocityDrive)));

	VelocityGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Damping", "Damping"))
		.Font(IDetailLayoutBuilder::GetDetailFont()).ToolTipText(XDampingProperty->GetToolTipText())
	]
	.ValueContent()
	[
		DampingWidget
	];

	// max force limit
	TSharedPtr<IPropertyHandle> MaxForceProperty = XDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce));
	TSharedRef<SWidget> MaxForceWidget = ConstraintDetails::CreateTriFloatWidget(MaxForceProperty, YDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce)), ZDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce)), LOCTEXT("EditMaxForce", "Edit Max Force"));
	MaxForceWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPhysicsConstraintComponentDetails::IsPropertyEnabled, EPropertyType::LinearDrive)));

	LinearMotorCat.AddCustomRow(LOCTEXT("MaxForce", "Max Force"), true)
	.NameContent()
	[
		MaxForceProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		MaxForceWidget
	];
}

void FPhysicsConstraintComponentDetails::AddAngularDrive(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstance, TSharedPtr<IPropertyHandle> ProfilePropertiesProperty)
{
	IDetailCategoryBuilder& AngularMotorCat = DetailBuilder.EditCategory("AngularMotor");

	TSharedPtr<IPropertyHandle> AngularDriveProperty = ProfilePropertiesProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintProfileProperties, AngularDrive));
	TSharedPtr<IPropertyHandle> AngularDriveModeProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, AngularDriveMode));

	TSharedPtr<IPropertyHandle> AccelerationModeProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, bAccelerationMode));
	TSharedPtr<IPropertyHandle> LimitViolationResponseProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, LimitViolationResponse));
	TSharedPtr<IPropertyHandle> SlerpDriveProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, SlerpDrive));
	TSharedPtr<IPropertyHandle> SwingDriveProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, SwingDrive));
	TSharedPtr<IPropertyHandle> TwistDriveProperty = AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, TwistDrive));

	TSharedPtr<IPropertyHandle> SlerpPositionDriveProperty = SlerpDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));
	TSharedPtr<IPropertyHandle> SlerpVelocityDriveProperty = SlerpDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));
	TSharedPtr<IPropertyHandle> SwingPositionDriveProperty = SwingDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));
	TSharedPtr<IPropertyHandle> SwingVelocityDriveProperty = SwingDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));
	TSharedPtr<IPropertyHandle> TwistPositionDriveProperty = TwistDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnablePositionDrive));
	TSharedPtr<IPropertyHandle> TwistVelocityDriveProperty = TwistDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, bEnableVelocityDrive));

	auto IsAngularMode = [AngularDriveModeProperty](EAngularDriveMode::Type CheckMode)
	{
		uint8 DriveMode;
		if (AngularDriveModeProperty->GetValue(DriveMode) == FPropertyAccess::Result::Success)
		{
			return DriveMode == CheckMode;
		}
		return false;
	};

	auto EligibleForSLERP = [this, IsAngularMode]()
	{
		return IsAngularMode(EAngularDriveMode::SLERP) && !ConstraintDetails::IsAngularPropertyEqual(AngularSwing1MotionProperty, ACM_Locked) && !ConstraintDetails::IsAngularPropertyEqual(AngularSwing2MotionProperty, ACM_Locked) && !ConstraintDetails::IsAngularPropertyEqual(AngularTwistMotionProperty, ACM_Locked);
	};

	auto EligibleForTwistAndSwing = [IsAngularMode]()
	{
		return IsAngularMode(EAngularDriveMode::TwistAndSwing);
	};

	auto OrientationEnabled = [EligibleForSLERP, EligibleForTwistAndSwing, TwistPositionDriveProperty, SwingPositionDriveProperty, SlerpPositionDriveProperty]()
	{
		if(EligibleForSLERP())
		{
			return ConstraintDetails::GetBoolProperty(SlerpPositionDriveProperty);
		} else if(EligibleForTwistAndSwing())
		{
			return ConstraintDetails::GetBoolProperty(TwistPositionDriveProperty) || ConstraintDetails::GetBoolProperty(SwingPositionDriveProperty);
		}

		return false;
	};

	auto VelocityEnabled = [EligibleForSLERP, EligibleForTwistAndSwing, TwistVelocityDriveProperty, SwingVelocityDriveProperty, SlerpVelocityDriveProperty]()
	{
		if (EligibleForSLERP())
		{
			return ConstraintDetails::GetBoolProperty(SlerpVelocityDriveProperty);
		}
		else if (EligibleForTwistAndSwing())
		{
			return ConstraintDetails::GetBoolProperty(TwistVelocityDriveProperty) || ConstraintDetails::GetBoolProperty(SwingVelocityDriveProperty);
		}

		return false;
	};

	auto VelocityOrOrientationEnabled = [VelocityEnabled, OrientationEnabled]()
	{
		return VelocityEnabled() || OrientationEnabled();
	};

	AngularMotorCat.AddProperty(AngularDriveModeProperty);
	AngularMotorCat.AddProperty(AccelerationModeProperty);
	AngularMotorCat.AddProperty(LimitViolationResponseProperty);

	IDetailGroup& OrientationGroup = AngularMotorCat.AddGroup("Orientation Drive", LOCTEXT("OrientrationDrive", "Orientation Drive"), false, true);
	OrientationGroup.HeaderProperty(AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, OrientationTarget)).ToSharedRef()).DisplayName(LOCTEXT("TargetOrientation", "Target Orientation")).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(OrientationEnabled)));


	TSharedRef<SWidget> SlerpPositionWidget = SlerpPositionDriveProperty->CreatePropertyValueWidget();
	TSharedRef<SWidget> SlerpVelocityWidget = SlerpVelocityDriveProperty->CreatePropertyValueWidget();
	SlerpPositionWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForSLERP)));
	SlerpVelocityWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForSLERP)));

	TSharedRef<SWidget> TwistPositionWidget = TwistPositionDriveProperty->CreatePropertyValueWidget();
	TSharedRef<SWidget> TwistVelocityWidget = TwistVelocityDriveProperty->CreatePropertyValueWidget();
	TwistPositionWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForTwistAndSwing)));
	TwistVelocityWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForTwistAndSwing)));

	TSharedRef<SWidget> SwingPositionWidget = SwingPositionDriveProperty->CreatePropertyValueWidget();
	TSharedRef<SWidget> SwingVelocityWidget = SwingVelocityDriveProperty->CreatePropertyValueWidget();
	SwingPositionWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForTwistAndSwing)));
	SwingVelocityWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(EligibleForTwistAndSwing)));

	OrientationGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("TwistSwingSlerpDrive", "Drives"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MinDesiredWidth(125 * 3 + 18 * 3)
	.MaxDesiredWidth(125 * 3 + 18 * 3)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SlerpDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				SlerpPositionWidget
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				TwistDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				TwistPositionWidget
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SwingDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				SwingPositionWidget
			]
		]
	];

	TSharedPtr<IPropertyHandle> StiffnessSlerpProperty = SlerpDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness));
	TSharedRef<SWidget> OrientationStrengthWidget = ConstraintDetails::CreateTriFloatWidget(StiffnessSlerpProperty, TwistDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness)), SwingDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Stiffness)), LOCTEXT("EditStrength", "Edit Strength"));
	OrientationStrengthWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(OrientationEnabled)));

	OrientationGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Strength", "Strength"))
		.Font(IDetailLayoutBuilder::GetDetailFont()).ToolTipText(StiffnessSlerpProperty->GetToolTipText())
	]
	.ValueContent()
	[
		OrientationStrengthWidget
	];

	IDetailGroup& AngularVelocityGroup = AngularMotorCat.AddGroup("Velocity Drive", LOCTEXT("VelocityDrive", "Velocity Drive"), false, true);
	AngularVelocityGroup.HeaderProperty(AngularDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAngularDriveConstraint, AngularVelocityTarget)).ToSharedRef()).DisplayName(LOCTEXT("TargetVelocity", "Target Velocity")).IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(VelocityEnabled)));

	AngularVelocityGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("TwistSwingSlerpDrive", "Drives"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MinDesiredWidth(125 * 3 + 18 * 3)
	.MaxDesiredWidth(125 * 3 + 18 * 3)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SlerpDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				SlerpVelocityWidget
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				TwistDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				TwistVelocityWidget
			]
		]
		+ SHorizontalBox::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SwingDriveProperty->CreatePropertyNameWidget()
			]
			+ SHorizontalBox::Slot()
			[
				SwingVelocityWidget
			]
		]
	];

	TSharedPtr<IPropertyHandle> DampingSlerpProperty = SlerpDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping));
	TSharedRef<SWidget> DampingSlerpWidget = ConstraintDetails::CreateTriFloatWidget(DampingSlerpProperty, TwistDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping)), SwingDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, Damping)), LOCTEXT("EditStrength", "Edit Strength"));
	DampingSlerpWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(VelocityEnabled)));
	AngularVelocityGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Damping", "Damping"))
		.Font(IDetailLayoutBuilder::GetDetailFont()).ToolTipText(DampingSlerpProperty->GetToolTipText())
	]
	.ValueContent()
	[
		DampingSlerpWidget
	];

	// max force limit
	TSharedPtr<IPropertyHandle> MaxForcePropertySlerp = SlerpDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce));
	TSharedRef<SWidget> MaxForceWidget = ConstraintDetails::CreateTriFloatWidget(MaxForcePropertySlerp, TwistDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce)), SwingDriveProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintDrive, MaxForce)), LOCTEXT("EditMaxForce", "Edit Max Force"));
	MaxForceWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(VelocityOrOrientationEnabled)));

	AngularMotorCat.AddCustomRow(LOCTEXT("MaxForce", "Max Force"), true)
	.NameContent()
	[
		MaxForcePropertySlerp->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		MaxForceWidget
	];
}

void FPhysicsConstraintComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	TSharedPtr<IPropertyHandle> ConstraintInstanceProperty;
	APhysicsConstraintActor* OwningConstraintActor = NULL;
	ParentPhysicsAsset = NULL;

	bInPhat = false;

	for (int32 i=0; i < Objects.Num(); ++i)
	{
		if (!Objects[i].IsValid()) { continue; }

		if (Objects[i]->IsA(UPhysicsConstraintTemplate::StaticClass()))
		{
			ConstraintInstanceProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPhysicsConstraintTemplate, DefaultInstance));
			ConstraintTemplate = Cast<UPhysicsConstraintTemplate>(Objects[i].Get());
			ParentPhysicsAsset = Cast<UPhysicsAsset>(Objects[i]->GetOuter());
			bInPhat = true;
			break;
		}
		else if (Objects[i]->IsA(UPhysicsConstraintComponent::StaticClass()))
		{
			ConstraintInstanceProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPhysicsConstraintComponent, ConstraintInstance));
			ConstraintComp = (UPhysicsConstraintComponent*)Objects[i].Get();
			OwningConstraintActor = Cast<APhysicsConstraintActor>(ConstraintComp->GetOwner());
			break;
		}
	}

	AddConstraintProperties(DetailBuilder, ConstraintInstanceProperty, Objects);

	DetailBuilder.EditCategory("Constraint Behavior");	//Create this category first so it's at the top

	TSharedPtr<IPropertyHandle> ProfileInstance = ConstraintInstanceProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FConstraintInstance, ProfileInstance));
	AddLinearLimits(DetailBuilder, ConstraintInstanceProperty, ProfileInstance);
	AddAngularLimits(DetailBuilder, ConstraintInstanceProperty, ProfileInstance);
	AddLinearDrive(DetailBuilder, ConstraintInstanceProperty, ProfileInstance);
	AddAngularDrive(DetailBuilder, ConstraintInstanceProperty, ProfileInstance);

	AddConstraintBehaviorProperties(DetailBuilder, ConstraintInstanceProperty, ProfileInstance);	//Now we've added all the complex UI, just dump the rest into Constraint category
}


bool FPhysicsConstraintComponentDetails::IsPropertyEnabled( EPropertyType::Type Type ) const
{
	bool bIsVisible = false;
	switch (Type)
	{
		case EPropertyType::LinearXPositionDrive:	return ConstraintDetails::GetBoolProperty(LinearXPositionDriveProperty);
		case EPropertyType::LinearYPositionDrive:	return ConstraintDetails::GetBoolProperty(LinearYPositionDriveProperty);
		case EPropertyType::LinearZPositionDrive:	return ConstraintDetails::GetBoolProperty(LinearZPositionDriveProperty);

		case EPropertyType::LinearXVelocityDrive:	return ConstraintDetails::GetBoolProperty(LinearXVelocityDriveProperty);
		case EPropertyType::LinearYVelocityDrive:	return ConstraintDetails::GetBoolProperty(LinearYVelocityDriveProperty);
		case EPropertyType::LinearZVelocityDrive:	return ConstraintDetails::GetBoolProperty(LinearZVelocityDriveProperty);
		case EPropertyType::LinearPositionDrive:	return ConstraintDetails::GetBoolProperty(LinearXPositionDriveProperty) || ConstraintDetails::GetBoolProperty(LinearYPositionDriveProperty) || ConstraintDetails::GetBoolProperty(LinearZPositionDriveProperty);
		case EPropertyType::LinearVelocityDrive:	return ConstraintDetails::GetBoolProperty(LinearXVelocityDriveProperty) || ConstraintDetails::GetBoolProperty(LinearYVelocityDriveProperty) || ConstraintDetails::GetBoolProperty(LinearZVelocityDriveProperty);
		case EPropertyType::LinearDrive:			return ConstraintDetails::GetBoolProperty(LinearXPositionDriveProperty) || ConstraintDetails::GetBoolProperty(LinearYPositionDriveProperty) || ConstraintDetails::GetBoolProperty(LinearZPositionDriveProperty)
															|| ConstraintDetails::GetBoolProperty(LinearXVelocityDriveProperty) || ConstraintDetails::GetBoolProperty(LinearYVelocityDriveProperty) || ConstraintDetails::GetBoolProperty(LinearZVelocityDriveProperty);
		case EPropertyType::AngularSwing1Limit:		return ConstraintDetails::IsAngularPropertyEqual(AngularSwing1MotionProperty, ACM_Limited);
		case EPropertyType::AngularSwing2Limit:		return ConstraintDetails::IsAngularPropertyEqual(AngularSwing2MotionProperty, ACM_Limited);
		case EPropertyType::AngularSwingLimit:		return ConstraintDetails::IsAngularPropertyEqual(AngularSwing1MotionProperty, ACM_Limited) || ConstraintDetails::IsAngularPropertyEqual(AngularSwing2MotionProperty, ACM_Limited);
		case EPropertyType::AngularTwistLimit:		return ConstraintDetails::IsAngularPropertyEqual(AngularTwistMotionProperty, ACM_Limited);
		case EPropertyType::AngularAnyLimit:		return ConstraintDetails::IsAngularPropertyEqual(AngularSwing1MotionProperty, ACM_Limited) || ConstraintDetails::IsAngularPropertyEqual(AngularSwing2MotionProperty, ACM_Limited) || ConstraintDetails::IsAngularPropertyEqual(AngularTwistMotionProperty, ACM_Limited);
	}

	return bIsVisible;
}

ECheckBoxState FPhysicsConstraintComponentDetails::IsLimitRadioChecked( TSharedPtr<IPropertyHandle> Property, uint8 Value ) const
{
	uint8 PropertyEnumValue = 0;
	if (Property.IsValid() && Property->GetValue(PropertyEnumValue) == FPropertyAccess::Result::Success)
	{
		return PropertyEnumValue == Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FPhysicsConstraintComponentDetails::OnLimitRadioChanged( ECheckBoxState CheckType, TSharedPtr<IPropertyHandle> Property, uint8 Value )
{
	if (Property.IsValid() && CheckType == ECheckBoxState::Checked)
	{
		Property->SetValue(Value);
	}
}

FConstraintInstance* FPhysicsConstraintComponentDetails::GetConstraintInstance()
{
	FConstraintInstance* ConstraintInstance = nullptr;

	if (ConstraintTemplate)
	{
		ConstraintInstance = &ConstraintTemplate->DefaultInstance;
	}
	else if (ConstraintComp)
	{
		ConstraintInstance = &ConstraintComp->ConstraintInstance;
	}

	return ConstraintInstance;
}


#undef LOCTEXT_NAMESPACE
