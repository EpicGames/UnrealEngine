// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosEngineInterface.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Customizations/MathStructCustomizations.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDetailWidgetRow;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyTypeCustomization;
class SWidget;
class UObject;
class UPhysicsAsset;
class UPhysicsConstraintComponent;
class UPhysicsConstraintTemplate;
struct FConstraintInstance;
template <typename ObjectType, typename PropertyType> class TProxyProperty;
template <typename ObjectType> class TProxyValue;

enum class ECheckBoxState : uint8;
enum class EConstraintTransformComponentFlags : uint8;

/**
 * Proxy class customization that displays a constraint transform in a user friendly way.
 * 
 * Constraint transforms are described by a position vector and a pair of orthonormal vectors 
 * representing the orientation. This class displays the transform as a position vector and a
 * rotator in the UI details panel in a way that can be easily read and safely edited by users. 
 */
class FConstraintTransformCustomization : public FMathStructCustomization
{
public:

	FConstraintTransformCustomization();

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	void MakeRotationRow(TSharedRef<IPropertyHandle>& InPriAxisPropertyHandle, TSharedRef<IPropertyHandle>& InSecAxisPropertyHandle, FDetailWidgetRow& Row, TSharedRef<SWidget> EditSpaceToggleButtonWidget);
	void MakePositionRow(TSharedRef<IPropertyHandle>& InPositionPropertyHandle, FDetailWidgetRow& Row, TSharedRef<SWidget> EditSpaceToggleButtonWidget);

	template<typename ProxyType, typename NumericType> TSharedRef<SWidget> MakeNumericProxyWidget(TSharedRef<IPropertyHandle>& PriAxisPropertyHandle, TSharedRef<IPropertyHandle>& SecAxisPropertyHandle, TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue, const FText& Label, bool bRotationInDegrees, const FLinearColor& LabelBackgroundColor);
	template<typename ProxyType, typename NumericType> TSharedRef<SWidget> MakeNumericProxyWidget(TSharedRef<IPropertyHandle>& PositionPropertyHandle, TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue, const FText& Label, const FLinearColor& LabelBackgroundColor);

	void OnCopy(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	void OnCopyPosition(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr);
	void OnCopyRotation(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	void OnPaste(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	void OnPastePosition(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr);
	void OnPasteRotation(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);

	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId,
		TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	
	bool IsRotationValueEnabled(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr) const;
	template<typename ProxyType, typename NumericType> TOptional<NumericType> OnGetRotationValue(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue) const;
	template<typename ProxyType, typename NumericType> void OnRotationValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue);
	template<typename ProxyType, typename NumericType> void OnRotationValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue);

	bool IsPositionValueEnabled(TWeakPtr<IPropertyHandle> PositionWeakHandlePtr) const;
	template<typename ProxyType, typename NumericType> TOptional<NumericType> OnGetPositionValue(TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue) const;
	template<typename ProxyType, typename NumericType> void OnPositionValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue);
	template<typename ProxyType, typename NumericType> void OnPositionValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> PositionWeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue);

	void SetFrameLabelText(const FText InText);
	FText& GetFrameLabelText();

	void GetPositionAsFormattedString(FString& OutString);
	void GetRotationAsFormattedString(FString& OutString);
	void GetValueAsFormattedString(FString& OutString);
	bool SetPositionFromFormattedString(const FString& InString);
	bool SetRotationFromFormattedString(const FString& InString);
	bool SetValueFromFormattedString(const FString& InString);

	virtual bool CacheValues(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr) const;
	virtual bool CacheValues(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr) const;
	virtual bool FlushValues(TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr) const;
	virtual bool FlushValues(TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr) const;

	const FTransform& GetDefaultTransform() const { return DefaultTransform; };

	void SetDefaultTransform(const FTransform& InTransform, TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	void SetPositionDisplayRelativeToDefault(const bool bValue) { bPositionDisplayRelativeToDefault = bValue; }
	void SetRotationDisplayRelativeToDefault(const bool bValue) { bRotationDisplayRelativeToDefault = bValue; }

protected:
	void OrthonormalVectorPairToDisplayedRotator(const FVector& PriAxis, const FVector& SecAxis, FRotator& OutRotator) const;
	void DisplayedRotatorToOrthonormalVectorPair(const FRotator& InRotator, FVector& OutPriAxis, FVector& OutSecAxis) const;

	void OnPastePositionFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr);
	void PastePositionFromText(const FString& InTag, const FString& InText, TWeakPtr<IPropertyHandle> PositionPropertyHandlePtr);
	
	void OnPasteRotationFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);
	void PasteRotationFromText(const FString& InTag, const FString& InText, TWeakPtr<IPropertyHandle> PriAxisPropertyHandlePtr, TWeakPtr<IPropertyHandle> SecAxisPropertyHandlePtr);

	mutable TSharedRef< TProxyValue<FRotator> > CachedRotation;
	mutable TSharedRef< TProxyProperty<FRotator, FRotator::FReal> > CachedRotationYaw;
	mutable TSharedRef< TProxyProperty<FRotator, FRotator::FReal> > CachedRotationPitch;
	mutable TSharedRef< TProxyProperty<FRotator, FRotator::FReal> > CachedRotationRoll;

	mutable TSharedRef< TProxyValue<FVector> > CachedPosition;
	mutable TSharedRef< TProxyProperty<FVector, FRotator::FReal> > CachedPositionX;
	mutable TSharedRef< TProxyProperty<FVector, FRotator::FReal> > CachedPositionY;
	mutable TSharedRef< TProxyProperty<FVector, FRotator::FReal> > CachedPositionZ;

	// Constraint transforms are displayed relative to this default transform when the associated ...DisplayRelativeToDefault flag is true. Otherwise they are displayed in the frame of the associated bone.
	FTransform DefaultTransform;
	FTransform InverseDefaultTransform;

	// When true, the Position/Rotation components are displayed as offsets from the default transforms.
	bool bPositionDisplayRelativeToDefault;
	bool bRotationDisplayRelativeToDefault;

	FText FrameLabelText;
};

/** 
 * Detail customizer for PhysicsConstraintComponent and PhysicsConstraintTemplate
 */
class FPhysicsConstraintComponentDetails : public IDetailCustomization
{
public:
	FPhysicsConstraintComponentDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	struct EPropertyType
	{
		enum Type
		{
			LinearXPositionDrive,
			LinearYPositionDrive,
			LinearZPositionDrive,
			LinearPositionDrive,
			LinearXVelocityDrive,
			LinearYVelocityDrive,
			LinearZVelocityDrive,
			LinearVelocityDrive,
			LinearDrive,
		
			AngularSwingLimit,
			AngularSwing1Limit,
			AngularSwing2Limit,
			AngularTwistLimit,
			AngularAnyLimit,
		};
	};

	bool IsPropertyEnabled(EPropertyType::Type Type) const;

	ECheckBoxState IsLimitRadioChecked(TSharedPtr<IPropertyHandle> Property, uint8 Value) const ;
	void OnLimitRadioChanged(ECheckBoxState CheckType, TSharedPtr<IPropertyHandle> Property, uint8 Value);

	void AddConstraintProperties(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TArray<TWeakObjectPtr<UObject>>& Objects);
	void AddConstraintFrameTransform(const EConstraintFrame::Type ConstraintFrameType, IDetailCategoryBuilder& ConstraintCat, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle);
	void AddConstraintBehaviorProperties(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TSharedPtr<IPropertyHandle> ProfileInstance);
	void AddLinearLimits(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TSharedPtr<IPropertyHandle> ProfileInstance);
	void AddAngularLimits(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TSharedPtr<IPropertyHandle> ProfileInstance);
	void AddLinearDrive(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TSharedPtr<IPropertyHandle> ProfileInstance);
	void AddAngularDrive(IDetailLayoutBuilder& DetailBuilder, TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, TSharedPtr<IPropertyHandle> ProfileInstance);

	void OnCopyConstraintTransform(TSharedPtr<IPropertyHandle> PositionPropertyHandle, TSharedPtr<IPropertyHandle> PriAxisPropertyHandle, TSharedPtr<IPropertyHandle> SecAxisPropertyHandle, TSharedPtr<FConstraintTransformCustomization> RotationProxy);
	void OnPasteConstraintTransform(TSharedPtr<IPropertyHandle> PositionPropertyHandle, TSharedPtr<IPropertyHandle> PriAxisPropertyHandle, TSharedPtr<IPropertyHandle> SecAxisPropertyHandle, TSharedPtr<FConstraintTransformCustomization> RotationProxy);

	bool IsSnapConstraintTransformComponentVisible(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags SnapFlags);
	void ToggleDisplayConstraintTransformComponentRelativeToDefault(TSharedPtr<IPropertyHandle> ConstraintInstance, const EConstraintTransformComponentFlags ComponentFlags);
	bool IsDisplayingConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags);
	bool IsConstraintTransformComponentEnabled(const EConstraintTransformComponentFlags ComponentFlags) const;
	FSlateColor GetConstraintTransformColorAndOpacity(const EConstraintTransformComponentFlags ComponentFlags) const;

private:
	void OnPasteConstraintTransformFromText(
		const FString& InTag,
		const FString& InText,
		const TOptional<FGuid>& InOperationId,
		TSharedPtr<IPropertyHandle> PositionPropertyHandle,
		TSharedPtr<IPropertyHandle> PriAxisPropertyHandle,
		TSharedPtr<IPropertyHandle> SecAxisPropertyHandle,
		TSharedPtr<FConstraintTransformCustomization> TransformProxy);

	void PasteConstraintTransformFromText(
		const FString& InTag,
		const FString& InText,
		TSharedPtr<IPropertyHandle> PositionPropertyHandle,
		TSharedPtr<IPropertyHandle> PriAxisPropertyHandle,
		TSharedPtr<IPropertyHandle> SecAxisPropertyHandle,
		TSharedPtr<FConstraintTransformCustomization> TransformProxy);
	
	TSharedRef<SWidget> MakeEditSpaceToggleButtonWidget(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags ComponentFlags); // TODO - get this in the right place in src
	void SnapConstraintTransformComponentsToDefault(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle, const EConstraintTransformComponentFlags SnapFlags);
	void UpdateTransformProxyDisplayRelativeToDefault(TSharedPtr<IPropertyHandle> ConstraintInstancePropertyHandle);
	FConstraintInstance* GetConstraintInstance();

	TSharedRef<FConstraintTransformCustomization> ChildTransformProxy; // Creates a user friendly representation of the constraint's transform in the frame of the child bone in the UI details panel.
	TSharedRef<FConstraintTransformCustomization> ParentTransformProxy;// Creates a user friendly representation of the constraint's transform in the frame of the parent bone in the UI details panel.

	TSharedPtr<IPropertyHandle> LinearXPositionDriveProperty;
	TSharedPtr<IPropertyHandle> LinearYPositionDriveProperty;
	TSharedPtr<IPropertyHandle> LinearZPositionDriveProperty;

	TSharedPtr<IPropertyHandle> LinearXVelocityDriveProperty;
	TSharedPtr<IPropertyHandle> LinearYVelocityDriveProperty;
	TSharedPtr<IPropertyHandle> LinearZVelocityDriveProperty;

	TSharedPtr<IPropertyHandle> AngularSwing1MotionProperty;
	TSharedPtr<IPropertyHandle> AngularSwing2MotionProperty;
	TSharedPtr<IPropertyHandle> AngularTwistMotionProperty;

	TSharedPtr<IPropertyHandle> ChildPositionPropertyHandle;
	TSharedPtr<IPropertyHandle> ChildPriAxisPropertyHandle;
	TSharedPtr<IPropertyHandle> ChildSecAxisPropertyHandle;
	
	TSharedPtr<IPropertyHandle> ParentPositionPropertyHandle;
	TSharedPtr<IPropertyHandle> ParentPriAxisPropertyHandle;
	TSharedPtr<IPropertyHandle> ParentSecAxisPropertyHandle;

	UPhysicsConstraintComponent* ConstraintComp;
	UPhysicsConstraintTemplate* ConstraintTemplate;

	UPhysicsAsset* ParentPhysicsAsset;

	bool bInPhat;
};
