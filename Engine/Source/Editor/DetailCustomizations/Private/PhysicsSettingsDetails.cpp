// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsSettingsDetails.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Containers/EnumAsByte.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PhysicalSurfaceDetails"

void SPhysicalSurfaceEditBox::Construct(const FArguments& InArgs)
{
	PhysicalSurface = InArgs._PhysicalSurface;
	PhysicalSurfaceEnum = InArgs._PhysicalSurfaceEnum;
	OnCommitChange = InArgs._OnCommitChange;
	check(PhysicalSurface.IsValid() && PhysicalSurfaceEnum);

	ChildSlot
	[
		SAssignNew(NameEditBox, SEditableTextBox)
		.Text(this, &SPhysicalSurfaceEditBox::GetName)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OnTextCommitted(this, &SPhysicalSurfaceEditBox::NewNameEntered)
		.OnTextChanged(this, &SPhysicalSurfaceEditBox::OnTextChanged)
		.IsReadOnly(PhysicalSurface->Type == SurfaceType_Default)
		.SelectAllTextWhenFocused(true)
	];
}



void SPhysicalSurfaceEditBox::OnTextChanged(const FText& NewText)
{
	FString NewName = NewText.ToString();

	if(NewName.Find(TEXT(" "))!=INDEX_NONE)
	{
		// no white space
		NameEditBox->SetError(TEXT("No white space is allowed"));
	}
	else
	{
		NameEditBox->SetError(TEXT(""));
	}
}

void SPhysicalSurfaceEditBox::NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo)
{
	// Don't digest the number if we just clicked away from the pop-up
	if((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus))
	{
		FString NewName = NewText.ToString();
		if(NewName.Find(TEXT(" "))==INDEX_NONE)
		{
			FName NewSurfaceName(*NewName);
			if(PhysicalSurface->Name!=NAME_None && NewSurfaceName == NAME_None)
			{
				if(FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("SPhysicalSurfaceListItem_DeleteConfirm", "Would you like to delete the name? If this type is used, it will invalidate the usage.")) == EAppReturnType::No)
				{
					return;
				}
			}
			if(NewSurfaceName != PhysicalSurface->Name)
			{
				PhysicalSurface->Name = NewSurfaceName;
				OnCommitChange.ExecuteIfBound();
			}
		}
		else
		{
			// clear error
			NameEditBox->SetError(TEXT(""));
		}
	}
}

FText SPhysicalSurfaceEditBox::GetName() const
{
	return FText::FromName(PhysicalSurface->Name);
}

class FPhysicalSurfaceList : public IDetailCustomNodeBuilder, public TSharedFromThis<FPhysicalSurfaceList>
{
public:
	FPhysicalSurfaceList(UPhysicsSettings* InPhysicsSettings, UEnum* InPhysicalSurfaceEnum, TSharedPtr<IPropertyHandle>& InPhysicalSurfacesProperty )
		: PhysicsSettings(InPhysicsSettings)
		, PhysicalSurfaceEnum(InPhysicalSurfaceEnum)
		, PhysicalSurfacesProperty(InPhysicalSurfacesProperty)
	{
		PhysicalSurfacesProperty->MarkHiddenByCustomization();
	}

	void RefreshPhysicalSurfaceList()
	{
		// make sure no duplicate exists
		// if exists, use the last one
		for(auto Iter = PhysicsSettings->PhysicalSurfaces.CreateIterator(); Iter; ++Iter)
		{
			for(auto InnerIter = Iter+1; InnerIter; ++InnerIter)
			{
				// see if same type
				if(Iter->Type == InnerIter->Type)
				{
					// remove the current one
					PhysicsSettings->PhysicalSurfaces.RemoveAt(Iter.GetIndex());
					--Iter;
					break;
				}
			}
		}

		bool bCreatedItem[SurfaceType_Max];
		FGenericPlatformMemory::Memzero(bCreatedItem, sizeof(bCreatedItem));

		PhysicalSurfaceList.Empty();

		// I'm listing all of these because it is easier for users to understand how does this work. 
		// I can't just link behind of scene because internally it will only save the enum
		// for example if you name SurfaceType5 to be Water and later changed to Sand, everything that used
		// SurfaceType5 will changed to Sand
		// I think what might be better is to show what options they have, and it's for them to choose how to name

		// add the first one by default
		{
			bCreatedItem[0] = true;
			PhysicalSurfaceList.Add(MakeShareable(new FPhysicalSurfaceListItem(MakeShareable(new FPhysicalSurfaceName(SurfaceType_Default, TEXT("Default"))))));
		}

		// we don't create the first one. First one is always default. 
		for(auto Iter = PhysicsSettings->PhysicalSurfaces.CreateIterator(); Iter; ++Iter)
		{
			bCreatedItem[Iter->Type] = true;
			PhysicalSurfaceList.Add(MakeShareable(new FPhysicalSurfaceListItem(MakeShareable(new FPhysicalSurfaceName(*Iter)))));
		}

		for(int32 Index = (int32)SurfaceType1; Index < SurfaceType_Max; ++Index)
		{
			if(bCreatedItem[Index] == false)
			{
				FPhysicalSurfaceName NeweElement((EPhysicalSurface)Index, TEXT(""));
				PhysicalSurfaceList.Add(MakeShareable(new FPhysicalSurfaceListItem(MakeShareable(new FPhysicalSurfaceName(NeweElement)))));
			}
		}

		// sort PhysicalSurfaceList by Type

		struct FComparePhysicalSurface
		{
			FORCEINLINE bool operator()(const TSharedPtr<FPhysicalSurfaceListItem> A, const TSharedPtr<FPhysicalSurfaceListItem> B) const
			{
				check(A.IsValid());
				check(B.IsValid());
				return A->PhysicalSurface->Type < B->PhysicalSurface->Type;
			}
		};

		PhysicalSurfaceList.Sort(FComparePhysicalSurface());

		PhysicsSettings->LoadSurfaceType();

		RegenerateChildren.ExecuteIfBound();
	}

	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override
	{
		RegenerateChildren = InOnRegenerateChildren;
	}

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override
	{
		// no header row
	}

	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override
	{
		FText SearchString = LOCTEXT("FPhysicsSettingsDetails_PhysicalSurface", "Physical Surface");

		for(TSharedPtr<FPhysicalSurfaceListItem>& Item : PhysicalSurfaceList)
		{
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(SearchString);

			FString TypeString = PhysicalSurfaceEnum->GetNameStringByValue((int64)Item->PhysicalSurface->Type);

			Row.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TypeString))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];

			Row.ValueContent()
			[
				SNew(SPhysicalSurfaceEditBox)
				.PhysicalSurface(Item->PhysicalSurface)
				.PhysicalSurfaceEnum(PhysicalSurfaceEnum)
				.OnCommitChange(this, &FPhysicalSurfaceList::OnCommitChange)
			];
		}
	}

	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const { return false; }
	virtual FName GetName() const override
	{
		static const FName Name(TEXT("PhysicalSurfaceList"));
		return Name;
	}
private:

	void OnCommitChange()
	{
		bool bDoCommit=true;
		// make sure it verifies all data is correct
		// skip the first one
		for(auto Iter = PhysicalSurfaceList.CreateConstIterator()+1; Iter; ++Iter)
		{
			TSharedPtr<FPhysicalSurfaceListItem> ListItem = *Iter;
			if(ListItem->PhysicalSurface->Name != NAME_None)
			{
				// make sure no same name exists
				for(auto InnerIter= Iter+1; InnerIter; ++InnerIter)
				{
					TSharedPtr<FPhysicalSurfaceListItem> InnerItem = *InnerIter;
					if(ListItem->PhysicalSurface->Name == InnerItem->PhysicalSurface->Name)
					{
						// duplicate name, warn user and get out
						FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FPhysicsSettingsDetails_InvalidName", "Duplicate name found."));
						bDoCommit = false;
						break;
					}
				}
			}
		}

		if(bDoCommit)
		{
			PhysicalSurfacesProperty->NotifyPreChange();

			PhysicsSettings->PhysicalSurfaces.Empty();
			for(auto Iter = PhysicalSurfaceList.CreateConstIterator()+1; Iter; ++Iter)
			{
				TSharedPtr<FPhysicalSurfaceListItem> ListItem = *Iter;
				if(ListItem->PhysicalSurface->Name != NAME_None)
				{
					PhysicsSettings->PhysicalSurfaces.Add(FPhysicalSurfaceName(ListItem->PhysicalSurface->Type, ListItem->PhysicalSurface->Name));
				}
			}

			PhysicsSettings->TryUpdateDefaultConfigFile();
			PhysicsSettings->LoadSurfaceType();

			PhysicalSurfacesProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
private:
	FSimpleDelegate RegenerateChildren;
	TArray< TSharedPtr< FPhysicalSurfaceListItem > >	PhysicalSurfaceList;
	UPhysicsSettings* PhysicsSettings;
	UEnum* PhysicalSurfaceEnum;
	TSharedPtr<IPropertyHandle> PhysicalSurfacesProperty;
};


//====================================================================================
// FPhysicsSettingsDetails
//=====================================================================================
TSharedRef<IDetailCustomization> FPhysicsSettingsDetails::MakeInstance()
{
	return MakeShareable( new FPhysicsSettingsDetails );
}

void FPhysicsSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
  	IDetailCategoryBuilder& PhysicalSurfaceCategory = DetailBuilder.EditCategory("Physical Surface", FText::GetEmpty(), ECategoryPriority::Uncommon);

	PhysicsSettings= UPhysicsSettings::Get();
	check(PhysicsSettings);

	PhysicalSurfaceEnum = StaticEnum<EPhysicalSurface>();
	check(PhysicalSurfaceEnum);

	TSharedPtr<IPropertyHandle> PhysicalSurfacesProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPhysicsSettings, PhysicalSurfaces));

	TSharedRef<FPhysicalSurfaceList> PhysicalSurfaceListCustomization = MakeShareable(new FPhysicalSurfaceList(PhysicsSettings, PhysicalSurfaceEnum, PhysicalSurfacesProperty) );
	PhysicalSurfaceListCustomization->RefreshPhysicalSurfaceList();

	const FString PhysicalSurfaceDocLink = TEXT("Shared/Physics");
	TSharedPtr<SToolTip> PhysicalSurfaceTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("PhysicalSurface", "Edit physical surface."), NULL, PhysicalSurfaceDocLink, TEXT("PhysicalSurface"));


	// Customize collision section
	PhysicalSurfaceCategory.AddCustomRow(LOCTEXT("FPhysicsSettingsDetails_PhysicalSurface", "Physical Surface"))
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.ToolTip(PhysicalSurfaceTooltip)
		.AutoWrapText(true)
		.Text(LOCTEXT("PhysicalSurface_Menu_Description", " You can have up to 62 custom surface types for your project. \nOnce you name each type, they will show up as surface type in the physical material."))
	];


	PhysicalSurfaceCategory.AddCustomBuilder(PhysicalSurfaceListCustomization);
}



#undef LOCTEXT_NAMESPACE

