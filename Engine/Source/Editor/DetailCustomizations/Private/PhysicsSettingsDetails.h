// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IDetailLayoutBuilder;
class ITableRow;
class SEditableTextBox;
class STableViewBase;
class UEnum;
class UPhysicsSettings;
struct FPhysicalSurfaceName;
template <typename ItemType> class SListView;

DECLARE_DELEGATE(FOnCommitChange)

// Class containing the friend information - used to build the list view
class FPhysicalSurfaceListItem
{
public:

	/**
	* Constructor takes the required details
	*/
	FPhysicalSurfaceListItem(TSharedPtr<FPhysicalSurfaceName> InPhysicalSurface)
		: PhysicalSurface(InPhysicalSurface)
	{}

	TSharedPtr<FPhysicalSurfaceName> PhysicalSurface;
};

/**
* Implements the FriendsList
*/
class SPhysicalSurfaceEditBox : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPhysicalSurfaceEditBox) { }
		SLATE_ARGUMENT(TSharedPtr<FPhysicalSurfaceName>, PhysicalSurface)
		SLATE_ARGUMENT(UEnum*, PhysicalSurfaceEnum)
		SLATE_EVENT(FOnCommitChange, OnCommitChange)
	SLATE_END_ARGS()

public:

	/**
	* Constructs the application.
	*
	* @param InArgs - The Slate argument list.
	*/
	void Construct(const FArguments& InArgs);
	
	FString GetTypeString() const;

	FText GetName() const;
	void NewNameEntered(const FText& NewText, ETextCommit::Type CommitInfo);
	void OnTextChanged(const FText& NewText);

private:
	TSharedPtr<FPhysicalSurfaceName>	PhysicalSurface;
	UEnum *								PhysicalSurfaceEnum;
	FOnCommitChange						OnCommitChange;
	TSharedPtr<SEditableTextBox>		NameEditBox;
};

typedef  SListView< TSharedPtr< FPhysicalSurfaceListItem > > SPhysicalSurfaceListView;

class FPhysicsSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/**
	* Generates a widget for a channel item.
	* @param InItem - the ChannelListItem
	* @param OwnerTable - the owning table
	* @return The table row widget
	*/
	TSharedRef<ITableRow> HandleGenerateListWidget(TSharedPtr< FPhysicalSurfaceListItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);


private:
	TArray< TSharedPtr< FPhysicalSurfaceListItem > >	PhysicalSurfaceList;

	UPhysicsSettings *							PhysicsSettings;
	UEnum*										PhysicalSurfaceEnum;
	// functions
	void OnCommitChange();
};

