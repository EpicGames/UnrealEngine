// Copyright Epic Games, Inc. All Rights Reservekd.

#include "PoseAssetDetails.h"

#include "Animation/AnimSequence.h"
#include "Animation/SmartName.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE	"PoseAssetDetails"

// default name for retarget source
#define DEFAULT_RETARGET_SOURCE_NAME	TEXT("Default")
#define REFERENCE_BASE_POSE_NAME		TEXT("Reference Pose")

TSharedRef<IDetailCustomization> FPoseAssetDetails::MakeInstance()
{
	return MakeShareable(new FPoseAssetDetails);
}

void FPoseAssetDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjectsList = DetailBuilder.GetSelectedObjects();
	TArray< TWeakObjectPtr<UPoseAsset> > SelectedPoseAssets;

	for (auto SelectionIt = SelectedObjectsList.CreateConstIterator(); SelectionIt; ++SelectionIt)
	{
		if (UPoseAsset* TestPoseAsset = Cast<UPoseAsset>(SelectionIt->Get()))
		{
			SelectedPoseAssets.Add(TestPoseAsset);
		}
	}

	// we only support 1 asset for now
	if (SelectedPoseAssets.Num() > 1)
	{
		return;
	}

	PoseAsset = SelectedPoseAssets[0];

	// now additive/pose selector

	/////////////////////////////////////////////////////////////////////////////////
	// retarget source handler in Animation
	/////////////////////////////////////////////////////////////////////////////////
	IDetailCategoryBuilder& AnimationCategory = DetailBuilder.EditCategory("Animation");
	RetargetSourceNameHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPoseAsset, RetargetSource));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RetargetSourceAssetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimSequence, RetargetSourceAsset));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// first create profile combo list
	RetargetSourceComboList.Empty();
	// first one is default one
	RetargetSourceComboList.Add( MakeShareable( new FString (DEFAULT_RETARGET_SOURCE_NAME) ));

	// find skeleton
	TSharedPtr<IPropertyHandle> SkeletonHanlder = DetailBuilder.GetProperty(TEXT("Skeleton"));
	FName CurrentPoseName;
	ensure (RetargetSourceNameHandler->GetValue(CurrentPoseName) != FPropertyAccess::Fail);
	
	// Check if we use only one skeleton
	TargetSkeleton = PoseAsset->GetSkeleton();

	// find what is initial selection is
	TSharedPtr<FString> InitialSelected;
	if (TargetSkeleton.IsValid())
	{
		RegisterRetargetSourceChanged();
		// go through profile and see if it has mine
		for (auto Iter = TargetSkeleton->AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
		{
			RetargetSourceComboList.Add( MakeShareable( new FString ( Iter.Key().ToString() )));

			if (Iter.Key() == CurrentPoseName) 
			{
				InitialSelected = RetargetSourceComboList.Last();
			}
		}
	}

	// add widget for editing retarget source
	AnimationCategory.AddCustomRow(RetargetSourceNameHandler->GetPropertyDisplayName())
	.NameContent()
	[
		RetargetSourceNameHandler->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(RetargetSourceComboBox, SSearchableComboBox)
		.OptionsSource(&RetargetSourceComboList)
		.OnGenerateWidget(this, &FPoseAssetDetails::MakeRetargetSourceComboWidget)
		.OnSelectionChanged(this, &FPoseAssetDetails::OnRetargetSourceChanged)
		.OnComboBoxOpening(this, &FPoseAssetDetails::OnRetargetSourceComboOpening)
		.InitiallySelectedItem(InitialSelected)
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.ContentPadding(0.f)
		.Content()
		[
			SNew( STextBlock )
			.Text(this, &FPoseAssetDetails::GetRetargetSourceComboBoxContent)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.ToolTipText(this, &FPoseAssetDetails::GetRetargetSourceComboBoxToolTip)
		]	
	];

	AnimationCategory.AddCustomRow(RetargetSourceAssetHandle->GetPropertyDisplayName())
	.NameContent()
	[
		RetargetSourceAssetHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			RetargetSourceAssetHandle->CreatePropertyValueWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(SButton)
			.Text(LOCTEXT("UpdateRetargetSourceAssetDataButton", "Update"))
			.ToolTipText(LOCTEXT("UpdateRetargetSourceAssetDataButtonToolTip", "Updates retargeting data for RetargetSourceAsset. This is updated automatically at save, but you can click here to update without saving."))
			.Visibility(this, &FPoseAssetDetails::UpdateRetargetSourceAssetDataVisibility)
			.OnClicked(this, &FPoseAssetDetails::UpdateRetargetSourceAssetData)
		]
	];

	DetailBuilder.HideProperty(RetargetSourceNameHandler);
	DetailBuilder.HideProperty(RetargetSourceAssetHandle);
	/////////////////////////////////////////////////////////////////////////////
	// Additive settings category
	/////////////////////////////////////////////////////////////////////////////
	// now customize to combo box
	// cache for UI
	CachePoseAssetData();
	// get list of poses list
	// first create profile combo list
	BasePoseComboList.Empty();
	// add ref pose
	BasePoseComboList.Add(MakeShareable(new FString(REFERENCE_BASE_POSE_NAME)));

	TArray<FName> PoseNames = PoseAsset->GetPoseFNames();
	FName BasePoseName;

	if (PoseNames.IsValidIndex(CachedBasePoseIndex))
	{
		BasePoseName = PoseNames[CachedBasePoseIndex];
	}

	TSharedPtr<FString> InitialSelectedPose;
	if (PoseNames.Num() > 0)
	{
		RegisterBasePoseChanged();

		// go through profile and see if it has mine
		for (const auto& PoseName : PoseNames)
		{
			BasePoseComboList.Add(MakeShareable(new FString(PoseName.ToString())));

			if (PoseName == BasePoseName)
			{
				InitialSelectedPose = BasePoseComboList.Last();
			}
		}
	}
	else
	{
		InitialSelectedPose = BasePoseComboList.Last();
	}

	IDetailCategoryBuilder& AdditiveCategory = DetailBuilder.EditCategory("Additive");

	TSharedPtr<IPropertyHandle> AdditivePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPoseAsset, bAdditivePose));
	IDetailPropertyRow& AdditivePropertyRow = AdditiveCategory.AddProperty(AdditivePropertyHandle);
	AdditivePropertyRow.CustomWidget()
	.NameContent()
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &FPoseAssetDetails::OnAdditiveToggled)
		.IsChecked(this, &FPoseAssetDetails::IsAdditiveChecked)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AdditiveLabel", "Additive"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	]
	.ValueContent()
	.MinDesiredWidth(200.f)
	[
		SNew(SHorizontalBox)

		// if additive, show base pose
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(4.0f,5.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AdditiveBasePoseLabel", "Base Pose"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]

		// if additive, let them choose base pose
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.0f,5.0f))
		[
			SAssignNew(BasePoseComboBox, SSearchableComboBox)
			.OptionsSource(&BasePoseComboList)
			.OnGenerateWidget(this, &FPoseAssetDetails::MakeBasePoseComboWidget)
			.OnSelectionChanged(this, &FPoseAssetDetails::OnBasePoseChanged)
			.OnComboBoxOpening(this, &FPoseAssetDetails::OnBasePoseComboOpening)
			.InitiallySelectedItem(InitialSelectedPose)
			.IsEnabled(this, &FPoseAssetDetails::CanSelectBasePose)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FPoseAssetDetails::GetBasePoseComboBoxContent)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTipText(this, &FPoseAssetDetails::GetBasePoseComboBoxToolTip)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(4.0f,5.0f))
		[
			// apply button 
			SNew(SButton)
			.Text(this, &FPoseAssetDetails::GetButtonText)
			.ToolTipText(LOCTEXT("ApplySettingButton_Tooltip", "Apply Additive Setting changes"))
			.OnClicked(this, &FPoseAssetDetails::OnApplyAdditiveSettings)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Visibility(this, &FPoseAssetDetails::CanApplySettings)
		]
	];

	/////////////////////////////////
	// Source Animation filter for skeleton
	IDetailCategoryBuilder& SourceCategory = DetailBuilder.EditCategory("Source");
	SourceAnimationPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPoseAsset, SourceAnimation));

	DetailBuilder.HideProperty(SourceAnimationPropertyHandle);

	TSharedPtr<IPropertyHandle> SourceAnimationProperty = DetailBuilder.GetProperty(TEXT("SourceAnimation"));
	IDetailPropertyRow& SourceAnimationRow = SourceCategory.AddProperty(SourceAnimationProperty);
	SourceAnimationRow.CustomWidget()
	.NameContent()
	[
		SourceAnimationPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.AutoHeight()
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UAnimSequence::StaticClass())
			.OnShouldFilterAsset(this, &FPoseAssetDetails::ShouldFilterAsset)
			.PropertyHandle(SourceAnimationPropertyHandle)
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.AutoHeight()
		[
			SNew(SButton)
			.Text(this, &FPoseAssetDetails::GetAnimationUpdateButtonText)
			.ToolTipText(LOCTEXT("UpdateSource_Tooltip", "Update Poses From Source Animation"))
			.OnClicked(this, &FPoseAssetDetails::OnUpdatePoseSourceAnimation)
			.IsEnabled(this, &FPoseAssetDetails::IsUpdateSourceEnabled)
			.HAlign(HAlign_Center)
		]
	];
}

void FPoseAssetDetails::OnSourceAnimationChanged(const FAssetData& AssetData)
{
	ensureAlways(SourceAnimationPropertyHandle->SetValue(AssetData) == FPropertyAccess::Result::Success);;
}

bool FPoseAssetDetails::ShouldFilterAsset(const FAssetData& AssetData)
{
	if (TargetSkeleton.IsValid())
	{
		return !TargetSkeleton->IsCompatibleForEditor(AssetData);
	}

	return true;
}

FText FPoseAssetDetails::GetButtonText() const
{
	if (PoseAsset.IsValid())
	{
		bool bIsAdditiveAsset = PoseAsset->IsValidAdditive();
		if (bCachedAdditive != bIsAdditiveAsset)
		{
			if (bCachedAdditive)
			{
				return LOCTEXT("ApplyPose_ConvertToAdditive_Label", "Convert To Additive Pose");
			}
			else
			{
				return LOCTEXT("ApplyPose_ConvertToFull_Label", "Convert To Full Pose");
			}
		}
		
		if (bIsAdditiveAsset && CachedBasePoseIndex != PoseAsset->GetBasePoseIndex())
		{
			return LOCTEXT("ApplyPose_RecalculateAdditive_Label", "Apply New Base Pose");
		}

	}

	return LOCTEXT("ApplyPose_Apply_Label", "Apply");
}

bool FPoseAssetDetails::CanSelectBasePose() const
{
	return (bCachedAdditive);
}

TSharedRef<SWidget> FPoseAssetDetails::MakeBasePoseComboWidget(TSharedPtr<FString> InItem)
{
	return 	SNew(STextBlock).Text(FText::FromString(*InItem)).Font(IDetailLayoutBuilder::GetDetailFont());
}

void FPoseAssetDetails::RefreshBasePoseChanged()
{
	if (PoseAsset.IsValid())
	{
		BasePoseComboList.Reset();
		// cache current value again 
		TSharedPtr<FString> SelectedItem = BasePoseComboBox->GetSelectedItem();
		if (SelectedItem.IsValid())
		{
			FString SelectedString = *SelectedItem;
			CachedBasePoseIndex = PoseAsset->GetPoseIndexByName(FName(*SelectedString));
		}
		else
		{
			CachedBasePoseIndex = -1;
		}

		TArray<FName> PoseNames = PoseAsset->GetPoseFNames();
		// add ref pose
		BasePoseComboList.Add(MakeShareable(new FString(REFERENCE_BASE_POSE_NAME)));

		if (PoseNames.Num() > 0)
		{
			// go through profile and see if it has mine
			for (const auto& PoseName : PoseNames)
			{
				BasePoseComboList.Add(MakeShareable(new FString(PoseName.ToString())));
			}
		}

		BasePoseComboBox->RefreshOptions();
	}
}

void FPoseAssetDetails::RegisterBasePoseChanged()
{
	if (PoseAsset.IsValid() && !OnDelegatePoseListChanged.IsBound())
	{
		OnDelegatePoseListChanged = UPoseAsset::FOnPoseListChanged::CreateSP(this, &FPoseAssetDetails::RefreshBasePoseChanged);
		OnDelegatePoseListChangedDelegateHandle = PoseAsset->RegisterOnPoseListChanged(OnDelegatePoseListChanged);
	}
}

void FPoseAssetDetails::OnBasePoseComboOpening()
{
	TSharedPtr<FString> ComboStringPtr = GetBasePoseString(CachedBasePoseIndex);
	if (ComboStringPtr.IsValid())
	{
		BasePoseComboBox->SetSelectedItem(ComboStringPtr);
	}
}

void FPoseAssetDetails::OnBasePoseChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();

		if (NewValue == REFERENCE_BASE_POSE_NAME)
		{
			CachedBasePoseIndex = -1;
		}
		else
		{
			CachedBasePoseIndex = PoseAsset->GetPoseIndexByName(FName(*NewValue));
		}
	}
}

FText FPoseAssetDetails::GetBasePoseComboBoxContent() const
{
	return FText::FromString(*GetBasePoseString(CachedBasePoseIndex).Get());
}

FText FPoseAssetDetails::GetBasePoseComboBoxToolTip() const
{
	return LOCTEXT("BasePoseComboToolTip", "Select Base Pose for the ");
}

TSharedPtr<FString> FPoseAssetDetails::GetBasePoseString(int32 InBasePoseIndex) const
{
	if (PoseAsset.IsValid())
	{
		FName BasePoseName = PoseAsset->GetPoseNameByIndex(InBasePoseIndex);
		if (BasePoseName != NAME_None)
		{
			FString BasePoseNameString = BasePoseName.ToString();
			// go through profile and see if it has mine
			for (int32 Index = 1; Index < BasePoseComboList.Num(); ++Index)
			{
				if (BasePoseNameString == *BasePoseComboList[Index])
				{
					return BasePoseComboList[Index];
				}
			}

		}
	}

	return BasePoseComboList[0];
}
void FPoseAssetDetails::CachePoseAssetData()
{
	bCachedAdditive = PoseAsset->IsValidAdditive();
	CachedBasePoseIndex = PoseAsset->GetBasePoseIndex();
}

 TSharedRef<SWidget> FPoseAssetDetails::MakeRetargetSourceComboWidget(TSharedPtr<FString> InItem)
 {
	 return SNew(STextBlock).Text(FText::FromString(*InItem)).Font(IDetailLayoutBuilder::GetDetailFont());
 }

 void FPoseAssetDetails::DelegateRetargetSourceChanged()
 {
	 if (TargetSkeleton.IsValid())
	 {
		 // first create profile combo list
		 RetargetSourceComboList.Empty();
		 // first one is default one
		 RetargetSourceComboList.Add(MakeShareable(new FString(DEFAULT_RETARGET_SOURCE_NAME)));

		 // go through profile and see if it has mine
		 for (auto Iter = TargetSkeleton->AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
		 {
			 RetargetSourceComboList.Add(MakeShareable(new FString(Iter.Key().ToString())));
		 }

		 RetargetSourceComboBox->RefreshOptions();
	 }
 }

 void FPoseAssetDetails::RegisterRetargetSourceChanged()
 {
	 if (TargetSkeleton.IsValid() && !OnDelegateRetargetSourceChanged.IsBound())
	 {
		 OnDelegateRetargetSourceChanged = USkeleton::FOnRetargetSourceChanged::CreateSP(this, &FPoseAssetDetails::DelegateRetargetSourceChanged);
		 OnDelegateRetargetSourceChangedDelegateHandle = TargetSkeleton->RegisterOnRetargetSourceChanged(OnDelegateRetargetSourceChanged);
	 }
 }

 void FPoseAssetDetails::OnRetargetSourceComboOpening()
 {
	 FName RetargetSourceName;
	 if (RetargetSourceNameHandler->GetValue(RetargetSourceName) != FPropertyAccess::Result::MultipleValues)
	 {
		 TSharedPtr<FString> ComboStringPtr = GetRetargetSourceString(RetargetSourceName);
		 if (ComboStringPtr.IsValid())
		 {
			 RetargetSourceComboBox->SetSelectedItem(ComboStringPtr);
		 }
	 }
 }

 void FPoseAssetDetails::OnRetargetSourceChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
 {
	 // if it's set from code, we did that on purpose
	 if (SelectInfo != ESelectInfo::Direct)
	 {
		 FString NewValue = *NewSelection.Get();

		 if (NewValue == DEFAULT_RETARGET_SOURCE_NAME)
		 {
			 NewValue = TEXT("");
		 }
		 // set profile set up
		 ensure(RetargetSourceNameHandler->SetValue(NewValue) == FPropertyAccess::Result::Success);
	 }
 }

 FText FPoseAssetDetails::GetRetargetSourceComboBoxContent() const
 {
	 FName RetargetSourceName;
	 if (RetargetSourceNameHandler->GetValue(RetargetSourceName) == FPropertyAccess::Result::MultipleValues)
	 {
		 return LOCTEXT("MultipleValues", "Multiple Values");
	 }

	 return FText::FromString(*GetRetargetSourceString(RetargetSourceName).Get());
 }

 FText FPoseAssetDetails::GetRetargetSourceComboBoxToolTip() const
 {
	 return LOCTEXT("RetargetSourceComboToolTip", "When retargeting, this pose will be used as a base of animation");
 }

 TSharedPtr<FString> FPoseAssetDetails::GetRetargetSourceString(FName RetargetSourceName) const
 {
	 FString RetargetSourceString = RetargetSourceName.ToString();

	 // go through profile and see if it has mine
	 for (int32 Index = 1; Index < RetargetSourceComboList.Num(); ++Index)
	 {
		 if (RetargetSourceString == *RetargetSourceComboList[Index])
		 {
			 return RetargetSourceComboList[Index];
		 }
	 }

	 return RetargetSourceComboList[0];
 }

 EVisibility FPoseAssetDetails::UpdateRetargetSourceAssetDataVisibility() const
 {
	 if (UPoseAsset* Pose = PoseAsset.Get())
	 {
		 PRAGMA_DISABLE_DEPRECATION_WARNINGS
		 if (!Pose->RetargetSourceAsset.IsNull())
		 {
			 return EVisibility::Visible;
		 }
		 PRAGMA_ENABLE_DEPRECATION_WARNINGS
	 }

	 return EVisibility::Collapsed;
 }

 FReply FPoseAssetDetails::UpdateRetargetSourceAssetData()
 {
	 RetargetSourceAssetHandle->NotifyPostChange(EPropertyChangeType::Unspecified);
	 return FReply::Handled();
 }

 EVisibility FPoseAssetDetails::CanApplySettings() const
 {
	 if (PoseAsset.IsValid())
	 {
		 bool bIsAdditiveAsset = PoseAsset->IsValidAdditive();
		 return (bCachedAdditive != bIsAdditiveAsset || (bIsAdditiveAsset && CachedBasePoseIndex != PoseAsset->GetBasePoseIndex())) ? EVisibility::Visible : EVisibility::Collapsed;
	 }

	 return EVisibility::Collapsed;
 }

 FReply FPoseAssetDetails::OnApplyAdditiveSettings()
 {
	 if (PoseAsset.IsValid())
	 {
		 FScopedTransaction Transaction(LOCTEXT("ApplyAdditiveSetting_Transaction", "Apply Additive Setting"));
		 PoseAsset->Modify();

		 PoseAsset->ConvertSpace(bCachedAdditive, CachedBasePoseIndex);
	 }

	 return FReply::Handled();
 }

 FPoseAssetDetails::~FPoseAssetDetails()
 {
	 if (TargetSkeleton.IsValid() && OnDelegateRetargetSourceChanged.IsBound())
	 {
		 TargetSkeleton->UnregisterOnRetargetSourceChanged(OnDelegateRetargetSourceChangedDelegateHandle);
	 }

	 if (PoseAsset.IsValid() && OnDelegatePoseListChanged.IsBound())
	 {
		 PoseAsset->UnregisterOnPoseListChanged(OnDelegatePoseListChangedDelegateHandle);
	 }
 }

 void FPoseAssetDetails::OnAdditiveToggled(ECheckBoxState NewCheckedState)
 {
	 bCachedAdditive = NewCheckedState == ECheckBoxState::Checked;
 }

 ECheckBoxState FPoseAssetDetails::IsAdditiveChecked() const
 {
	 return (bCachedAdditive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
 }


 FReply FPoseAssetDetails::OnUpdatePoseSourceAnimation()
 {
	 if (PoseAsset.IsValid())
	 {
		 UObject* ObjectSet;
		 SourceAnimationPropertyHandle->GetValue(ObjectSet);

		 UAnimSequence* AnimSequenceSelected = Cast<UAnimSequence>(ObjectSet);
		 if (AnimSequenceSelected && PoseAsset->GetSkeleton()->IsCompatibleForEditor(AnimSequenceSelected->GetSkeleton()))
		 {
			 FScopedTransaction Transaction(LOCTEXT("UpdatePoseSourceAnimation_Transaction", "Update Pose"));
			 PoseAsset->Modify();
			 PoseAsset->UpdatePoseFromAnimation(AnimSequenceSelected);

			 FFormatNamedArguments Args;
			 Args.Add(TEXT("AssetName"), FText::FromString(PoseAsset->GetName()));
			 Args.Add(TEXT("SourceAsset"), FText::FromString(AnimSequenceSelected->GetName()));
			 FText ResultText = FText::Format(LOCTEXT("PoseAssetSourceHasBeenUpdated", "PoseAsset {AssetName} has been updated from {SourceAsset}"), Args);

			 FNotificationInfo Info(ResultText);
			 Info.bFireAndForget = true;
			 Info.bUseLargeFont = false;

			 TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
			 if (Item.IsValid())
			 {
				 Item->SetCompletionState(SNotificationItem::CS_Success);
			 }
		 }
		 else
		 {
			 // invalid source asset message
			 FFormatNamedArguments Args;
			 Args.Add(TEXT("AssetName"), FText::FromString(PoseAsset->GetName()));
			 Args.Add(TEXT("SourceAsset"), FText::FromString(GetNameSafe(AnimSequenceSelected)));
			 Args.Add(TEXT("SkeletonName"), FText::FromString(GetNameSafe(PoseAsset->GetSkeleton())));
			 FText ResultText = FText::Format(LOCTEXT("UpdatePoseWithInvalidSkeleton", "Source Asset {SourceAsset} is invalid or does not have a compatible skeleton {SkeletonName} with {AssetName}"), Args);

			 FNotificationInfo Info(ResultText);
			 Info.bFireAndForget = true;
			 Info.bUseLargeFont = true;

			 TSharedPtr<SNotificationItem> Item = FSlateNotificationManager::Get().AddNotification(Info);
			 if (Item.IsValid())
			 {
				 Item->SetCompletionState(SNotificationItem::CS_Fail);
			 }
		 }
	 }
	 return FReply::Handled();
 }

bool FPoseAssetDetails::IsUpdateSourceEnabled() const
{
	const UObject* ObjectSet;
	SourceAnimationPropertyHandle->GetValue(ObjectSet);

	const UAnimSequence* AnimSequenceSelected = Cast<UAnimSequence>(ObjectSet);
	if (AnimSequenceSelected && PoseAsset.IsValid())
	{
		return (PoseAsset->SourceAnimation && PoseAsset->GetSourceAnimationGuid() != PoseAsset->SourceAnimationRawDataGUID);		
	}

	return false;
}

FText FPoseAssetDetails::GetAnimationUpdateButtonText() const
{
	if (PoseAsset.IsValid() && !PoseAsset->SourceAnimationRawDataGUID.IsValid())
	{
		return LOCTEXT("RestoreSource_Label", "Restore Source");
	}

	return LOCTEXT("UpdateSource_Label", "Update Source");
}

#undef LOCTEXT_NAMESPACE
