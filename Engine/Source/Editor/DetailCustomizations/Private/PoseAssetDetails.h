// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/PoseAsset.h"
#include "Animation/Skeleton.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SSearchableComboBox;
class SWidget;
struct FAssetData;

class FPoseAssetDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	virtual ~FPoseAssetDetails();

private:
 	TWeakObjectPtr<UPoseAsset> PoseAsset;
	TWeakObjectPtr<USkeleton> TargetSkeleton;

	// property handler
	TSharedPtr<IPropertyHandle> RetargetSourceNameHandler;
	TSharedPtr<IPropertyHandle> RetargetSourceAssetHandle;

	// retarget source related
	TSharedPtr< SSearchableComboBox > RetargetSourceComboBox;
	TArray< TSharedPtr< FString > >						RetargetSourceComboList;

	TSharedRef<SWidget> MakeRetargetSourceComboWidget( TSharedPtr<FString> InItem );
	void OnRetargetSourceChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo  );
	FText GetRetargetSourceComboBoxContent() const;
	FText GetRetargetSourceComboBoxToolTip() const;
	void OnRetargetSourceComboOpening();
	TSharedPtr<FString> GetRetargetSourceString(FName RetargetSourceName) const;
	EVisibility UpdateRetargetSourceAssetDataVisibility() const;
	FReply UpdateRetargetSourceAssetData();

	USkeleton::FOnRetargetSourceChanged OnDelegateRetargetSourceChanged;
	FDelegateHandle OnDelegateRetargetSourceChangedDelegateHandle;
	void RegisterRetargetSourceChanged();
	void DelegateRetargetSourceChanged();

	// additive setting
	void OnAdditiveToggled(ECheckBoxState NewCheckedState);
	ECheckBoxState IsAdditiveChecked() const;

	// base pose
	TSharedPtr< SSearchableComboBox > BasePoseComboBox;
	TArray< TSharedPtr< FString > >						BasePoseComboList;

	TSharedRef<SWidget> MakeBasePoseComboWidget(TSharedPtr<FString> InItem);
	void OnBasePoseChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetBasePoseComboBoxContent() const;
	FText GetBasePoseComboBoxToolTip() const;
	void OnBasePoseComboOpening();
	TSharedPtr<FString> GetBasePoseString(int32 InBasePoseIndex) const;
	bool CanSelectBasePose() const;
	UPoseAsset::FOnPoseListChanged OnDelegatePoseListChanged;
	FDelegateHandle OnDelegatePoseListChangedDelegateHandle;
	void RegisterBasePoseChanged();
	void RefreshBasePoseChanged();

	bool bCachedAdditive;
	int32 CachedBasePoseIndex;
	void CachePoseAssetData();

	EVisibility CanApplySettings() const;
	FReply OnApplyAdditiveSettings();

	TSharedPtr<IPropertyHandle> SourceAnimationPropertyHandle;
	// replacing source animation
	void OnSourceAnimationChanged(const FAssetData& AssetData);
	bool ShouldFilterAsset(const FAssetData& AssetData);
	FReply OnUpdatePoseSourceAnimation();
	bool IsUpdateSourceEnabled() const;
	FText GetAnimationUpdateButtonText() const;

	FText GetButtonText() const;
};

