// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcessSettingsCustomization.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "LevelEditorViewport.h"
#include "Engine/BlendableInterface.h"
#include "Factories/Factory.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "ISettingsEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "ObjectEditorUtils.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Engine/RendererSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"
#include "SWarningOrErrorBox.h"
#include "RenderViewportFeedback.h"

class IDetailPropertyRow;
class SWidget;
class UPackage;

#define LOCTEXT_NAMESPACE "PostProcessSettingsCustomization"

const FName ShowPostProcessCategoriesName("ShowPostProcessCategories");
const FName ShowOnlyInnerPropertiesName("ShowOnlyInnerProperties");

struct FCategoryOrGroup
{
	IDetailCategoryBuilder* Category;
	IDetailGroup* Group;

	FCategoryOrGroup(IDetailCategoryBuilder& NewCategory)
		: Category(&NewCategory)
		, Group(nullptr)
	{}

	FCategoryOrGroup(IDetailGroup& NewGroup)
		: Category(nullptr)
		, Group(&NewGroup)
	{}

	FCategoryOrGroup()
		: Category(nullptr)
		, Group(nullptr)
	{}

	IDetailPropertyRow& AddProperty(TSharedRef<IPropertyHandle> PropertyHandle)
	{
		if (Category)
		{
			return Category->AddProperty(PropertyHandle);
		}
		else
		{
			return Group->AddPropertyRow(PropertyHandle);
		}
	}

	IDetailGroup& AddGroup(FName GroupName, const FText& DisplayName)
	{
		if (Category)
		{
			return Category->AddGroup(GroupName, DisplayName);
		}
		else
		{
			return Group->AddGroup(GroupName, DisplayName);
		}
	}

	bool IsValid() const
	{
		return Group || Category;
	}
};

struct FAlert
{
	enum class EType
	{
		Warning,
		Error
	};

	EType Type;
	FText Text;
	TAttribute<EVisibility> Visibility;
};

struct FPostProcessGroup
{
	FString RawGroupName;
	FString DisplayName;
	FString ParentPath;
	FCategoryOrGroup RootCategory;
	FCategoryOrGroup Detail;
	TOptional<FAlert> Alert;
	TArray<TSharedPtr<IPropertyHandle>> SimplePropertyHandles;
	TArray<TSharedPtr<IPropertyHandle>> AdvancedPropertyHandles;

	bool IsValid() const
	{
		return !RawGroupName.IsEmpty() && !DisplayName.IsEmpty() && RootCategory.IsValid();
	}

	FPostProcessGroup()
		: RootCategory()
		, Detail()
	{}
};

class FPostProcessRayTracingDisabledWarning
{
	static inline TWeakPtr<SNotificationItem> NotificationPtr = nullptr;

public:
	static void DisplayRayTracingDisabledWarning()
	{
		bool bSuppressNotification = false;
		GConfig->GetBool(TEXT("PostProcess"), TEXT("SuppressEnableRayTracingNotification"), bSuppressNotification, GEditorPerProjectIni);

		if (bSuppressNotification)
		{
			return;
		}
		
		static IConsoleVariable* RayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
		static const auto LumenUseHardwareRayTracingCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Lumen.HardwareRayTracing"));
		const bool bRayTracingEnabled = RayTracingCVar->GetInt() != 0;
		const bool bLumenRayTracingEnabled = LumenUseHardwareRayTracingCVar->GetValueOnAnyThread() != 0;
									
		if (bRayTracingEnabled && bLumenRayTracingEnabled)
		{
			return;
		}

		if (NotificationPtr.IsValid())
		{
			return;
		}
		
		const FText WarningText =LOCTEXT("RayTracedTranslucencyWarning",
			"The following Project Settings must be enabled for Ray Traced Translucency:\n"
			"- Hardware Ray Tracing: Support HWRT\n"
			"- Lumen: Use HWRT When Available");
										
		FNotificationInfo Warning(WarningText);
		Warning.bFireAndForget = false;
		Warning.bUseLargeFont = false;
		Warning.bUseThrobber = false;
		Warning.bUseSuccessFailIcons = false;
		Warning.WidthOverride = 400.0f;
		
		Warning.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("RawTracedTranslucencyWarning_Enable", "Enable"),
			LOCTEXT("RawTracedTranslucencyWarning_EnableToolTip", "Enable Support Hardware Ray Tracing and Use Hardware Ray Tracing When Available in your project settings"),
			FSimpleDelegate::CreateStatic(&EnableRayTracingSettings)));

		Warning.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("RawTracedTranslucencyWarning_NotNow", "Not Now"),
			LOCTEXT("RawTracedTranslucencyWarning_NotNowToolTip", "Don't enable Support Hardware Ray Tracing and Use Hardware Ray Tracing When Available"),
			FSimpleDelegate::CreateStatic(&CloseNotification)));

		Warning.CheckBoxState = TAttribute<ECheckBoxState>::Create(&GetSuppressNotificationCheckboxState);
		Warning.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(&SetSuppressNotification);
		Warning.CheckBoxText = LOCTEXT("RayTracedTranslucencyWarning_SuppressCheckBox", "Don't show this again");
										
		NotificationPtr = FSlateNotificationManager::Get().AddNotification(Warning);
		NotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}

private:
	static ECheckBoxState GetSuppressNotificationCheckboxState()
	{
		bool bSuppressNotification = false;
		GConfig->GetBool(TEXT("PostProcess"), TEXT("SuppressEnableRayTracingNotification"), bSuppressNotification, GEditorPerProjectIni);
		return bSuppressNotification ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	static void SetSuppressNotification(ECheckBoxState NewState)
	{
		// If the user selects to not show this again, set that in the config so we know about it in between sessions
		const bool bSuppressNotification = (NewState == ECheckBoxState::Checked);
		GConfig->SetBool(TEXT("PostProcess"), TEXT("SuppressEnableRayTracingNotification"), bSuppressNotification, GEditorPerProjectIni);
	}

	static void EnableRayTracingSettings()
	{
		static IConsoleVariable* RayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
		RayTracingCVar->Set(true);
		
		static IConsoleVariable* LumenUseHardwareRayTracingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Lumen.HardwareRayTracing"));
		LumenUseHardwareRayTracingCVar->Set(true);
		
		URendererSettings* const RendererSettings = GetMutableDefault<URendererSettings>();
		RendererSettings->bEnableRayTracing = true;
		RendererSettings->bUseHardwareRayTracingForLumen = true;

		auto UpdatePropertyValue = [RendererSettings](const FName& PropertyName)
		{
			FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(PropertyName);

			FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, {RendererSettings});
			RendererSettings->PostEditChangeProperty(PropertyChangedEvent);
			RendererSettings->UpdateSinglePropertyInConfigFile(Property, RendererSettings->GetDefaultConfigFilename());
		};

		UpdatePropertyValue(GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableRayTracing));
		UpdatePropertyValue(GET_MEMBER_NAME_CHECKED(URendererSettings, bUseHardwareRayTracingForLumen));
		
		FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
		
		CloseNotification();
	}
	
	static void CloseNotification()
	{
		if (NotificationPtr.IsValid())
		{
			NotificationPtr.Pin()->SetCompletionState(SNotificationItem::CS_Success);
			NotificationPtr.Pin()->ExpireAndFadeout();
		}

		NotificationPtr.Reset();
	}
};

// See FDetailGroup::MakeNameWidget()
static TSharedRef<SWidget> MakeNameWidget(FText LocalizedDisplayName, TOptional<FText> LocalizedToolTip)
{
	TSharedRef<STextBlock> TextBlock = 
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LocalizedDisplayName);

	if (LocalizedToolTip.IsSet() && !LocalizedToolTip->IsEmpty())
	{
		TextBlock->SetToolTipText(LocalizedToolTip.GetValue());
	}

	return
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ContentPadding(FMargin(0, 2, 0, 2))
		//.OnClicked(this, &FDetailGroup::OnNameClicked)
		.ForegroundColor(FSlateColor::UseForeground())
		.Content()
		[
			TextBlock
		];
};

static bool GetRenderFeedbackForActiveViewport(FRenderViewportFeedback& OutValue)
{
	static FLevelEditorModule* LevelEditorModule = nullptr;
	if (!LevelEditorModule && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		LevelEditorModule = &FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	}

	if (!LevelEditorModule)
	{
		return false;
	}

	const TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule->GetFirstActiveLevelViewport();
	if (!ActiveLevelViewport)
	{
		return false;
	}

	FLevelEditorViewportClient& ViewportClient = ActiveLevelViewport->GetLevelViewportClient();
	if (ViewportClient.RenderViewportFeedback)
	{
		OutValue = ViewportClient.RenderViewportFeedback->Data();
		return true;
	}
	else
	{
		// Request data collection for next frame
		ViewportClient.RenderViewportFeedback = MakeShared<UE::RenderViewportFeedback::FReceiver>();
		return false;
	}
}

static void CustomizeGroup(FPostProcessGroup& Owner, TSharedRef<IPropertyHandle>& StructPropertyHandle);

static void MakeDetail(FPostProcessGroup& Elem, TMap<FString, FPostProcessGroup>& NameToGroupMap, TSharedRef<IPropertyHandle>& StructPropertyHandle)
{
	if (Elem.Detail.IsValid())
	{
		return;
	}

	FCategoryOrGroup* ParentDetail;
	int _;
	bool bIsRoot = !Elem.ParentPath.FindChar('|', _);
	if (bIsRoot)
	{
		ParentDetail = &Elem.RootCategory;
	}
	else
	{
		ParentDetail = &NameToGroupMap.Find(Elem.ParentPath)->Detail;
	}
	Elem.Detail = ParentDetail->AddGroup(*Elem.RawGroupName, FText::FromString(Elem.DisplayName));
	CustomizeGroup(Elem, StructPropertyHandle);
}

static FPostProcessGroup* FindOrCreateGroup(TMap<FString, FPostProcessGroup>& NameToGroupMap, FCategoryOrGroup* Root, FStringView GroupPath, TSharedRef<IPropertyHandle>& StructPropertyHandle)
{
	FPostProcessGroup* PPGroup = NameToGroupMap.Find(FString(GroupPath));

	if (!PPGroup)
	{
		int32 LastSeperator;
		bool bIsRoot = !GroupPath.FindLastChar('|', LastSeperator);
		if (bIsRoot)
		{
			return nullptr;
		}
		else
		{
			PPGroup = &NameToGroupMap.Add(FString(GroupPath));
		}
	}

	// Is this a new group?
	if (!PPGroup->IsValid())
	{
		PPGroup->RootCategory = *Root;
		PPGroup->RawGroupName = GroupPath;

		int32 LastSeparatorIndex;
		if (GroupPath.FindLastChar('|', LastSeparatorIndex))
		{
			PPGroup->ParentPath = GroupPath.Left(LastSeparatorIndex);
			FPostProcessGroup* Parent = FindOrCreateGroup(NameToGroupMap, Root, PPGroup->ParentPath, StructPropertyHandle);
			
			// Parent is now certainly non-empty, so build its IDetailGroup that the child will go into
			if (Parent)
			{
				MakeDetail(*Parent, NameToGroupMap, StructPropertyHandle);
			}

			PPGroup->DisplayName = GroupPath.RightChop(LastSeparatorIndex+1);
		}
		else
		{
			// Path is root
			checkNoEntry();
		}
	}

	return PPGroup;
}

static void CustomizeGroup(FPostProcessGroup& Group, TSharedRef<IPropertyHandle>& StructPropertyHandle)
{
	if (Group.RawGroupName == TEXT("Lens|Bloom|Gaussian"))
	{
		Group.Alert = FAlert {
			.Type = FAlert::EType::Warning,
			.Text = LOCTEXT("PostProcessBloomGaussianWarning", "Primary viewport currently shows convolution bloom"),
			.Visibility = MakeAttributeLambda([=, CurVisibility = EVisibility::Collapsed]() mutable
			{
				// Show if Gaussian bloom is selected, but not currently used in the viewport.
				// Visibility is updated only if we can access all the required data.
				FRenderViewportFeedback RenderViewportFeedback;
				if (GetRenderFeedbackForActiveViewport(RenderViewportFeedback))
				{
					FPostProcessSettings* PostProcessSettings;
					if (StructPropertyHandle->GetValueData((void*&)PostProcessSettings) == FPropertyAccess::Success)
					{
						bool bBloomFFTSelected = PostProcessSettings->bOverride_BloomMethod && PostProcessSettings->BloomMethod == EBloomMethod::BM_FFT;
						CurVisibility = 
							bBloomFFTSelected && RenderViewportFeedback.BloomMethod == EBloomMethod::BM_SOG 
							? EVisibility::Visible : EVisibility::Collapsed;
					}
				}

				return CurVisibility;
			})
		};
	}
	else if (Group.RawGroupName == TEXT("Lens|Bloom|Convolution"))
	{
		Group.Alert = FAlert {
			.Type = FAlert::EType::Warning,
			.Text = LOCTEXT("PostProcessBloomConvolutionWarning", "Primary viewport currently shows Gaussian bloom"),
			.Visibility = MakeAttributeLambda([=, CurVisibility = EVisibility::Collapsed]() mutable
			{
				// Show if convolution bloom is selected, but not currently used in the viewport.
				// Visibility is updated only if we can access all the required data.
				FRenderViewportFeedback RenderViewportFeedback;
				if (GetRenderFeedbackForActiveViewport(RenderViewportFeedback))
				{
					FPostProcessSettings* PostProcessSettings;
					if (StructPropertyHandle->GetValueData((void*&)PostProcessSettings) == FPropertyAccess::Success)
					{
						bool bBloomSOGSelected = PostProcessSettings->bOverride_BloomMethod && PostProcessSettings->BloomMethod == EBloomMethod::BM_SOG;
						CurVisibility = 
							bBloomSOGSelected && RenderViewportFeedback.BloomMethod == EBloomMethod::BM_FFT 
							? EVisibility::Visible : EVisibility::Collapsed;
					}
				}

				return CurVisibility;
			})
		};
	}
}

// Return false to hide the property
static bool CustomizeProperty(TSharedPtr<IPropertyHandle>& PropertyHandle, FProperty* Property, const FName& CategoryFName, TSharedRef<IPropertyHandle>& StructHandle, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	static const FName ExposureCategory("Lens|Exposure");
	static const FName TranslucencyCategory("Rendering Features|Translucency");

	if (CategoryFName == ExposureCategory)
	{
		static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));
		const bool bExtendedLuminanceRange = VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnGameThread() == 1;

		if (bExtendedLuminanceRange)
		{
			if (Property->GetName() == GET_MEMBER_NAME_CHECKED(FPostProcessSettings, AutoExposureMinBrightness))
			{
				Property->SetMetaData(TEXT("DisplayName"), TEXT("Min EV100"));
			}
			else if (Property->GetName() == GET_MEMBER_NAME_CHECKED(FPostProcessSettings, AutoExposureMaxBrightness))
			{
				Property->SetMetaData(TEXT("DisplayName"), TEXT("Max EV100"));
			}
			else if (Property->GetName() == GET_MEMBER_NAME_CHECKED(FPostProcessSettings, HistogramLogMin))
			{
				Property->SetMetaData(TEXT("DisplayName"), TEXT("Histogram Min EV100"));
			}
			else if (Property->GetName() == GET_MEMBER_NAME_CHECKED(FPostProcessSettings, HistogramLogMax))
			{
				Property->SetMetaData(TEXT("DisplayName"), TEXT("Histogram Max EV100"));
			}
		}
	}
	// Special handling for the ray tracing translucency section, which needs to show or hide specific properties based on the translucency type
	else if (CategoryFName == TranslucencyCategory)
	{
		TSharedPtr<IPropertyHandle> TranslucencyTypeHandle = StructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, TranslucencyType));
		if (PropertyHandle->GetProperty()->GetFName() != TranslucencyTypeHandle->GetProperty()->GetName())
		{
			// Based on the translucency type property value, hide any properties that don't match the current translucency type
			uint8 TranslucencyType = 0;
			TranslucencyTypeHandle->GetValue(TranslucencyType);

			const FString& TranslucencyTypeMetaData = PropertyHandle->GetMetaData(TEXT("TranslucencyType"));
			const FString& TranslucencyTypeName = StaticEnum<ETranslucencyType>()->GetNameStringByValue(TranslucencyType);
			if (!TranslucencyTypeMetaData.Equals(TranslucencyTypeName, ESearchCase::IgnoreCase))
			{
				PropertyHandle->MarkHiddenByCustomization();
				return false;
			}
		}
		else
		{
			// Add a property changed event on the translucency type property to refresh the details panel so that the necessary properties
			// are made visible or hidden, as well as notifying the user if the project settings don't have hardware ray tracing enabled,
			// which is required for ray traced translucency
			PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([StructHandle, &StructCustomizationUtils]()
			{
				TSharedPtr<IPropertyHandle> TranslucencyTypeHandle = StructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FPostProcessSettings, TranslucencyType));
								
				uint8 TranslucencyType = 0;
				TranslucencyTypeHandle->GetValue(TranslucencyType);

				if (static_cast<ETranslucencyType>(TranslucencyType) == ETranslucencyType::RayTraced)
				{
					FPostProcessRayTracingDisabledWarning::DisplayRayTracingDisabledWarning();
				}
								
				StructCustomizationUtils.GetPropertyUtilities()->ForceRefresh();
			}));
		}
	}

	return true;
}

void FPostProcessSettingsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren = 0;
	FPropertyAccess::Result Result = StructPropertyHandle->GetNumChildren(NumChildren);

	// Create new categories in the parent layout rather than adding all post process settings to one category
	IDetailLayoutBuilder& LayoutBuilder = StructBuilder.GetParentCategory().GetParentLayout();

	TMap<FString, FCategoryOrGroup> NameToCategoryBuilderMap;
	TMap<FString, FPostProcessGroup> NameToGroupMap;

	TArray<FString> CategoryAndGroups;

	bool bShowPostProcessCategories = StructPropertyHandle->HasMetaData(ShowPostProcessCategoriesName);

	if (Result == FPropertyAccess::Success && NumChildren > 0)
	{
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex);

			if (ChildHandle.IsValid() && ChildHandle->GetProperty())
			{
				FProperty* Property = ChildHandle->GetProperty();

				FName CategoryFName = FObjectEditorUtils::GetCategoryFName(Property);
				FString RawCategoryName = CategoryFName.ToString();

				CategoryAndGroups.Reset();
				RawCategoryName.ParseIntoArray(CategoryAndGroups, TEXT("|"), 1);

				FString RootCategoryName = CategoryAndGroups.Num() > 0 ? CategoryAndGroups[0] : RawCategoryName;

				FCategoryOrGroup* Category = NameToCategoryBuilderMap.Find(RootCategoryName);
				if (!Category)
				{
					if (bShowPostProcessCategories)
					{
						IDetailCategoryBuilder& NewCategory = LayoutBuilder.EditCategory(*RootCategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);
						Category = &NameToCategoryBuilderMap.Emplace(RootCategoryName, NewCategory);
					}
					else
					{
						IDetailGroup& NewGroup = StructBuilder.AddGroup(*RootCategoryName, FText::FromString(RootCategoryName));
						Category = &NameToCategoryBuilderMap.Emplace(RootCategoryName, NewGroup);
					}
				}

				bool bVisible = CustomizeProperty(ChildHandle, Property, CategoryFName, StructPropertyHandle, StructCustomizationUtils);
				if (!bVisible)
				{
					continue;
				}

				if (CategoryAndGroups.Num() > 1)
				{
					FPostProcessGroup* PPGroup = FindOrCreateGroup(NameToGroupMap, Category, RawCategoryName, StructPropertyHandle);
					check(PPGroup);
					
					bool bIsSimple = !ChildHandle->GetProperty()->HasAnyPropertyFlags(CPF_AdvancedDisplay);
					if (bIsSimple)
					{
						PPGroup->SimplePropertyHandles.Add(ChildHandle);
					}
					else
					{
						PPGroup->AdvancedPropertyHandles.Add(ChildHandle);
					}
				}
				else
				{
					Category->AddProperty(ChildHandle.ToSharedRef());
				}
			}
		}

		for (auto& NameAndGroup : NameToGroupMap)
		{
			FPostProcessGroup& PPGroup = NameAndGroup.Value;

			if (PPGroup.SimplePropertyHandles.Num() > 0 || PPGroup.AdvancedPropertyHandles.Num() > 0)
			{
				MakeDetail(PPGroup, NameToGroupMap, StructPropertyHandle);
				IDetailGroup& SimpleGroup = *PPGroup.Detail.Group;

				if (PPGroup.Alert.IsSet())
				{
					FDetailWidgetRow Header = SimpleGroup.HeaderRow()
					.NameContent()
					[
						MakeNameWidget(FText::FromString(PPGroup.DisplayName), {})
					]
					.ExtensionContent()
					[
						SNew(SBorder)
						.Visibility(PPGroup.Alert->Visibility)
						.ToolTip(
							SNew(SToolTip)
							.Text(PPGroup.Alert->Text)
						)
						.Padding(3, 3, 3, 3)
						.BorderBackgroundColor(FColor::Transparent)
						[
							SNew(SImage)
							.DesiredSizeOverride(FVector2D(16,16))
							.Image_Lambda([Type=PPGroup.Alert->Type]()
							{ 
								return (Type == FAlert::EType::Warning) ? FAppStyle::Get().GetBrush("Icons.WarningWithColor") : FAppStyle::Get().GetBrush("Icons.ErrorWithColor"); 
							})
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
				}

				static const FString ColorGradingName = TEXT("Color Grading");

				// Only enable group reset on color grading category groups
				if (PPGroup.RawGroupName.Contains(ColorGradingName))
				{
					SimpleGroup.EnableReset(true);
				}

				for (auto& SimpleProperty : PPGroup.SimplePropertyHandles)
				{
					SimpleGroup.AddPropertyRow(SimpleProperty.ToSharedRef());
				}

				if (PPGroup.AdvancedPropertyHandles.Num() > 0)
				{
					IDetailGroup& AdvancedGroup = SimpleGroup.AddGroup(*(PPGroup.RawGroupName+TEXT("Advanced")), LOCTEXT("PostProcessAdvancedGroup", "Advanced"));
					
					for (auto& AdvancedProperty : PPGroup.AdvancedPropertyHandles)
					{
						AdvancedGroup.AddPropertyRow(AdvancedProperty.ToSharedRef());
					}
				}
			}
		}
	}
}

void FPostProcessSettingsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	bool bShowHeader = !StructPropertyHandle->HasMetaData(ShowPostProcessCategoriesName) && !StructPropertyHandle->HasMetaData(ShowOnlyInnerPropertiesName);
	if (bShowHeader)
	{
		HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];

		HeaderRow.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
	}
}

void FWeightedBlendableCustomization::AddDirectAsset(TSharedRef<IPropertyHandle> StructPropertyHandle, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value, UClass* Class)
{
	Weight->SetValue(1.0f);

	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);

		TArray<FString> Values;

		for (TArray<UObject*>::TConstIterator It = Objects.CreateConstIterator(); It; It++)
		{
			UObject* Obj = *It;

			const UObject* NewObj = NewObject<UObject>(Obj, Class);

			FString Str = NewObj->GetPathName();

			Values.Add(Str);
		}

		Value->SetPerObjectValues(Values);
	}
}

void FWeightedBlendableCustomization::AddIndirectAsset(TSharedPtr<IPropertyHandle> Weight)
{
	Weight->SetValue(1.0f);
}

EVisibility FWeightedBlendableCustomization::IsWeightVisible(TSharedPtr<IPropertyHandle> Weight) const
{
	float WeightValue = 1.0f;
	
	Weight->GetValue(WeightValue);

	return (WeightValue >= 0) ? EVisibility::Visible : EVisibility::Hidden;
}

FText FWeightedBlendableCustomization::GetDirectAssetName(TSharedPtr<IPropertyHandle> Value) const
{
	UObject* RefObject = 0;
	
	Value->GetValue(RefObject);

	check(RefObject);

	return FText::FromString(RefObject->GetFullName());
}

FReply FWeightedBlendableCustomization::JumpToDirectAsset(TSharedPtr<IPropertyHandle> Value)
{
	UObject* RefObject = 0;
	
	Value->GetValue(RefObject);

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RefObject);

	return FReply::Handled();
}

TSharedRef<SWidget> FWeightedBlendableCustomization::GenerateContentWidget(TSharedRef<IPropertyHandle> StructPropertyHandle, UPackage* Package, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value)
{
	bool bSeparatorIsNeeded = false; 

	FMenuBuilder MenuBuilder(true, NULL);
	{
		for (TObjectIterator<UClass> It; It; ++It)
		{
			if (It->IsChildOf(UFactory::StaticClass()))
			{
				UFactory* Factory = It->GetDefaultObject<UFactory>();

				check(Factory);

				UClass* SupportedClass = Factory->GetSupportedClass();

				if (SupportedClass)
				{
					if (SupportedClass->ImplementsInterface(UBlendableInterface::StaticClass()))
					{
						// At the moment we know about 3 Blendables: Material, UMaterialInstanceConstant, LightPropagationVolumeBlendable
						// The materials are not that useful to have here (hard to reference) so we suppress them here
						if (!(
							SupportedClass == UMaterial::StaticClass() ||
							SupportedClass == UMaterialInstanceConstant::StaticClass()
							))
						{
							FUIAction Direct2(FExecuteAction::CreateSP(this, &FWeightedBlendableCustomization::AddDirectAsset, StructPropertyHandle, Weight, Value, SupportedClass));

							FName ClassName = SupportedClass->GetFName();
						
							MenuBuilder.AddMenuEntry(FText::FromString(ClassName.GetPlainNameString()),
								LOCTEXT("Blendable_DirectAsset2h", "Creates an asset that is owned by the containing object"), FSlateIcon(), Direct2);

							bSeparatorIsNeeded = true;
						}
					}
				}
			}
		}

		if (bSeparatorIsNeeded)
		{
			MenuBuilder.AddMenuSeparator();
		}

		FUIAction Indirect(FExecuteAction::CreateSP(this, &FWeightedBlendableCustomization::AddIndirectAsset, Weight));
		MenuBuilder.AddMenuEntry(LOCTEXT("Blendable_IndirectAsset", "Asset reference"), 
			LOCTEXT("Blendable_IndirectAsseth", "reference a Blendable asset (owned by a content package), e.g. material with Post Process domain"), FSlateIcon(), Indirect);
	}
	

	TSharedRef<SWidgetSwitcher> Switcher = SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FWeightedBlendableCustomization::ComputeSwitcherIndex, StructPropertyHandle, Package, Weight, Value);

	Switcher->AddSlot()
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Blendable_ChooseElement", "Choose"))
			]
			.ContentPadding(FMargin(6.0, 2.0))
			.MenuContent()
			[
				MenuBuilder.MakeWidget()
			]
		];

	Switcher->AddSlot()
		[
			SNew(SButton)
			.ContentPadding(FMargin(0,0))
			.Text(this, &FWeightedBlendableCustomization::GetDirectAssetName, Value)
			.OnClicked(this, &FWeightedBlendableCustomization::JumpToDirectAsset, Value)
		];

	Switcher->AddSlot()
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(Value)
		];

	return Switcher;
}


int32 FWeightedBlendableCustomization::ComputeSwitcherIndex(TSharedRef<IPropertyHandle> StructPropertyHandle, UPackage* Package, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value) const
{
	float WeightValue = 1.0f;
	UObject* RefObject = 0;
	
	Weight->GetValue(WeightValue);
	Value->GetValue(RefObject);

	if (RefObject)
	{
		UPackage* PropPackage = RefObject->GetOutermost();

		return (PropPackage == Package) ? 1 : 2;
	}
	else
	{
		return (WeightValue < 0.0f) ? 0 : 2;
	}
}

void FWeightedBlendableCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// we don't have children but this is a pure virtual so we need to override
}

void FWeightedBlendableCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> SharedWeightProp;
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(FName(TEXT("Weight")));
		if (ChildHandle.IsValid() && ChildHandle->GetProperty())
		{
			SharedWeightProp = ChildHandle;
		}
	}
		
	TSharedPtr<IPropertyHandle> SharedValueProp;
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(FName(TEXT("Object")));
		if (ChildHandle.IsValid() && ChildHandle->GetProperty())
		{
			SharedValueProp = ChildHandle;
		}
	}

	float WeightValue = 1.0f;
	UObject* RefObject = 0;
	
	SharedWeightProp->GetValue(WeightValue);
	SharedValueProp->GetValue(RefObject);

	UPackage* StructPackage = 0;
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = StructPropertyHandle->GetParentHandle();
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);

		for (TArray<UObject*>::TConstIterator It = Objects.CreateConstIterator(); It; It++)
		{
			UObject* ref = *It;

			if (StructPackage)
			{
				// Differing outermost package values indicate that the current RefObject refers to post-process 
				// volumes selected within different levels, e.g. persistent and a sub-level. 
				// In this case, do not store a package name. It is only used by ComputeSwitcherIndex() to determine direct
				// vs. indirect assets in the post process materials/blendables array. When more than one volume is selected, the direct
				// asset entries will simple read 'Multiple values' since each belongs to separate post-process volumes.
				if (StructPackage != ref->GetOutermost())
				{
					StructPackage = NULL;
					break;
				}
			}
			else
			{
				StructPackage = ref->GetOutermost();
			}
		}
	}

	HeaderRow.NameContent()
	[
		SNew(SHorizontalBox)
		.Visibility(this, &FWeightedBlendableCustomization::IsWeightVisible, SharedWeightProp)
		+SHorizontalBox::Slot()
		[
			SNew(SBox)
			.MinDesiredWidth(60.0f)
			.MaxDesiredWidth(60.0f)		
			[
				SharedWeightProp->CreatePropertyValueWidget()
			]
		]
	];

	HeaderRow.ValueContent()
	.MaxDesiredWidth(0.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			GenerateContentWidget(StructPropertyHandle, StructPackage, SharedWeightProp, SharedValueProp)
		]
	];
}





#undef LOCTEXT_NAMESPACE
