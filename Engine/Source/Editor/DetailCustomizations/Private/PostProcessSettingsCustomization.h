// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class SWidget;
class UClass;
class UPackage;

/**
 * Implements a details view customization for the FPostProcessSettings structure.
 */
class FPostProcessSettingsCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance( )
	{
		return MakeShareable(new FPostProcessSettingsCustomization());
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
};


/**
 * Implements a details view customization for the FBlendablesWrapper structure.
 */
class FWeightedBlendableCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance( )
	{
		return MakeShareable(new FWeightedBlendableCustomization());
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	TSharedRef<SWidget> GenerateContentWidget(TSharedRef<IPropertyHandle> StructPropertyHandle, UPackage* Package, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value);

	// "Outer" is the object that has the blendables container
	void AddDirectAsset(TSharedRef<IPropertyHandle> StructPropertyHandle, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value, UClass* Class);
	// "Outer" is a package
	void AddIndirectAsset(TSharedPtr<IPropertyHandle> Weight);

	// open the details panel for this object
	FReply JumpToDirectAsset(TSharedPtr<IPropertyHandle> Value);
	
	// to get the blendable type as string
	FText GetDirectAssetName(TSharedPtr<IPropertyHandle> Value) const;

	// @return 0:choose 1:direct, 2:indirect
	int32 ComputeSwitcherIndex(TSharedRef<IPropertyHandle> StructPropertyHandle, UPackage* Package, TSharedPtr<IPropertyHandle> Weight, TSharedPtr<IPropertyHandle> Value) const;

	// the weight is only visible if the user choose the type (or the reference)
	EVisibility IsWeightVisible(TSharedPtr<IPropertyHandle> Weight) const;
};
