// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimitiveComponentDetails.h"

#include "BodyInstanceCustomization.h"
#include "ComponentMaterialCategory.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/UnrealString.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "EditorCategoryUtils.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailGroup.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "PrimitiveComponentDetails"

//////////////////////////////////////////////////////////////
// This class customizes collision setting in primitive component
//////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FPrimitiveComponentDetails::MakeInstance()
{
	return MakeShareable( new FPrimitiveComponentDetails );
}

void FPrimitiveComponentDetails::AddPhysicsCategory(IDetailLayoutBuilder& DetailBuilder)
{
	BodyInstanceCustomizationHelper = MakeShareable(new FBodyInstanceCustomizationHelper(ObjectsCustomized));
	BodyInstanceCustomizationHelper->CustomizeDetails(DetailBuilder, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, BodyInstance)));
}

void FPrimitiveComponentDetails::AddCollisionCategory(IDetailLayoutBuilder& DetailBuilder)
{
	if (DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, BodyInstance))->IsValidHandle())
	{
		// Collision
		TSharedPtr<IPropertyHandle> BodyInstanceHandler = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, BodyInstance));
		uint32 NumChildren = 0;
		BodyInstanceHandler->GetNumChildren(NumChildren);

		IDetailCategoryBuilder& CollisionCategory = DetailBuilder.EditCategory("Collision");

		// add all collision properties
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = BodyInstanceHandler->GetChildHandle(ChildIndex);
			static const FName CollisionCategoryName(TEXT("Collision"));

			FName CategoryName = FObjectEditorUtils::GetCategoryFName(ChildProperty->GetProperty());
			if (CategoryName == CollisionCategoryName)
			{
				CollisionCategory.AddProperty(ChildProperty);
			}
		}
	}
}

void FPrimitiveComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Get the objects being customized so we can enable/disable editing of 'Simulate Physics'
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	// See if we are hiding Physics category
	TArray<FString> HideCategories;
	FEditorCategoryUtils::GetClassHideCategories(DetailBuilder.GetBaseClass(), HideCategories);

	if(!HideCategories.Contains("Materials"))
	{
		AddMaterialCategory(DetailBuilder);
	}

	TSharedRef<IPropertyHandle> MobilityHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UPrimitiveComponent, Mobility), USceneComponent::StaticClass());
	MobilityHandle->SetToolTipText(LOCTEXT("PrimitiveMobilityTooltip", "Mobility for primitive components controls how they can be modified in game and therefore how they interact with lighting and physics.\n* A movable primitive component can be changed in game, but requires dynamic lighting and shadowing from lights which have a large performance cost.\n* A static primitive component can't be changed in game, but can have its lighting baked, which allows rendering to be very efficient."));


	if(!HideCategories.Contains("Physics"))
	{
		AddPhysicsCategory(DetailBuilder);
	}

	if (!HideCategories.Contains("Collision"))
	{
		AddCollisionCategory(DetailBuilder);
	}

	if(!HideCategories.Contains("Lighting"))
	{
		AddLightingCategory(DetailBuilder);
	}
}

void FPrimitiveComponentDetails::AddMaterialCategory( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<USceneComponent> > Components;

	for( TWeakObjectPtr<UObject> Object : ObjectsCustomized )
	{
		USceneComponent*  Component = Cast<USceneComponent>(Object.Get());
		if( Component )
		{
			Components.Add( Component );
		}
	}

	MaterialCategory = MakeShareable(new FComponentMaterialCategory(Components));
	MaterialCategory->Create(DetailBuilder);
}

void FPrimitiveComponentDetails::AddLightingCategory(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& LightingCategory = DetailBuilder.EditCategory("Lighting");
}

void FPrimitiveComponentDetails::AddAdvancedSubCategory( IDetailLayoutBuilder& DetailBuilder, FName MainCategoryName, FName SubCategoryName)
{
	TArray<TSharedRef<IPropertyHandle> > SubCategoryProperties;
	IDetailCategoryBuilder& SubCategory = DetailBuilder.EditCategory(SubCategoryName);

	const bool bSimpleProperties = false;
	const bool bAdvancedProperties = true;
	SubCategory.GetDefaultProperties( SubCategoryProperties, bSimpleProperties, bAdvancedProperties );

	if( SubCategoryProperties.Num() > 0 )
	{
		IDetailCategoryBuilder& MainCategory = DetailBuilder.EditCategory(MainCategoryName);

		const bool bForAdvanced = true;
		IDetailGroup& Group = MainCategory.AddGroup( SubCategoryName, SubCategory.GetDisplayName(), bForAdvanced );

		for( int32 PropertyIndex = 0; PropertyIndex < SubCategoryProperties.Num(); ++PropertyIndex )
		{
			TSharedRef<IPropertyHandle>& PropertyHandle = SubCategoryProperties[PropertyIndex];

			// Ignore customized properties
			if( !PropertyHandle->IsCustomized() )
			{
				Group.AddPropertyRow( SubCategoryProperties[PropertyIndex] );
			}
		}
	}

}


#undef LOCTEXT_NAMESPACE

