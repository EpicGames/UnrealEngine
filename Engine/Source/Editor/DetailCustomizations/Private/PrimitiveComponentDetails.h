// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UObject;

class FPrimitiveComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	void AddMaterialCategory( IDetailLayoutBuilder& DetailBuilder);
	void AddLightingCategory(IDetailLayoutBuilder& DetailBuilder);
	void AddPhysicsCategory(IDetailLayoutBuilder& DetailBuilder);
	void AddCollisionCategory(IDetailLayoutBuilder& DetailBuilder);

	void AddAdvancedSubCategory(IDetailLayoutBuilder& DetailBuilder, FName MainCategory, FName SubCategory);

	FReply OnMobilityResetClicked(TSharedRef<IPropertyHandle> MobilityHandle);

	EVisibility GetMobilityResetVisibility(TSharedRef<IPropertyHandle> MobilityHandle) const;
	
	/** Returns whether to enable editing the 'Simulate Physics' checkbox based on the selected objects physics geometry */
	bool IsSimulatePhysicsEditable() const;
	/** Returns whether to enable editing the 'Use Async Scene' checkbox based on the selected objects' mobility and if the project uses an AsyncScene */
	bool IsUseAsyncEditable() const;

	TOptional<float> OnGetBodyMass() const;
	bool IsBodyMassReadOnly() const;
	bool IsBodyMassEnabled() const { return !IsBodyMassReadOnly(); }

private:
	/** Objects being customized so we can update the 'Simulate Physics' state if physics geometry is added/removed */
	TArray< TWeakObjectPtr<UObject> > ObjectsCustomized;

	TSharedPtr<class FComponentMaterialCategory> MaterialCategory;
	TSharedPtr<class FBodyInstanceCustomizationHelper> BodyInstanceCustomizationHelper;

};

