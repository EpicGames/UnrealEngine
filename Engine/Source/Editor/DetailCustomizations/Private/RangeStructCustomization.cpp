// Copyright Epic Games, Inc. All Rights Reserved.

#include "RangeStructCustomization.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/RangeBound.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;


#define LOCTEXT_NAMESPACE "RangeStructCustomization"


/* Helper traits for getting a metadata property based on the template parameter type
 *****************************************************************************/

namespace
{
	template <typename NumericType>
	struct FGetMetaDataHelper
	{ };

	template <>
	struct FGetMetaDataHelper<float>
	{
		static float GetMetaData(const TSharedRef<IPropertyHandle>& Property, const TCHAR* Key)
		{
			return Property->GetFloatMetaData(Key);
		}
	};

	template <>
	struct FGetMetaDataHelper<double>
	{
		static double GetMetaData(const TSharedRef<IPropertyHandle>& Property, const TCHAR* Key)
		{
			return Property->GetDoubleMetaData(Key);
		}
	};

	template <>
	struct FGetMetaDataHelper<int32>
	{
		static int32 GetMetaData(const TSharedRef<IPropertyHandle>& Property, const TCHAR* Key)
		{
			return Property->GetIntMetaData(Key);
		}
	};
}


/* FRangeStructCustomization static interface
 *****************************************************************************/

template <typename NumericType>
TSharedRef<IPropertyTypeCustomization> FRangeStructCustomization<NumericType>::MakeInstance()
{
	return MakeShareable(new FRangeStructCustomization<NumericType>);
}


/* IPropertyTypeCustomization interface
 *****************************************************************************/

template <typename NumericType>
void FRangeStructCustomization<NumericType>::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Get handles to the properties we're interested in
	LowerBoundStructHandle = StructPropertyHandle->GetChildHandle(TEXT("LowerBound"));
	UpperBoundStructHandle = StructPropertyHandle->GetChildHandle(TEXT("UpperBound"));
	check(LowerBoundStructHandle.IsValid());
	check(UpperBoundStructHandle.IsValid());

	LowerBoundValueHandle = LowerBoundStructHandle->GetChildHandle(TEXT("Value"));
	UpperBoundValueHandle = UpperBoundStructHandle->GetChildHandle(TEXT("Value"));
	LowerBoundTypeHandle = LowerBoundStructHandle->GetChildHandle(TEXT("Type"));
	UpperBoundTypeHandle = UpperBoundStructHandle->GetChildHandle(TEXT("Type"));
	check(LowerBoundValueHandle.IsValid());
	check(UpperBoundValueHandle.IsValid());
	check(LowerBoundTypeHandle.IsValid());
	check(UpperBoundTypeHandle.IsValid());

	// Get min/max metadata values if defined
	if (StructPropertyHandle->HasMetaData(TEXT("UIMin")))
	{
		MinAllowedValue = TOptional<NumericType>(FGetMetaDataHelper<NumericType>::GetMetaData(StructPropertyHandle, TEXT("UIMin")));
	}

	if (StructPropertyHandle->HasMetaData(TEXT("UIMax")))
	{
		MaxAllowedValue = TOptional<NumericType>(FGetMetaDataHelper<NumericType>::GetMetaData(StructPropertyHandle, TEXT("UIMax")));
	}

	// Make weak pointers to be passed as payloads to the widgets
	TWeakPtr<IPropertyHandle> LowerBoundValueWeakPtr = LowerBoundValueHandle;
	TWeakPtr<IPropertyHandle> UpperBoundValueWeakPtr = UpperBoundValueHandle;
	TWeakPtr<IPropertyHandle> LowerBoundTypeWeakPtr = LowerBoundTypeHandle;
	TWeakPtr<IPropertyHandle> UpperBoundTypeWeakPtr = UpperBoundTypeHandle;

	// Generate a list of enum values for the combo box from the LowerBound.Type property
	TArray<bool> RestrictedList;
	LowerBoundTypeHandle->GeneratePossibleValues(ComboBoxList, ComboBoxToolTips, RestrictedList);

	// Get initial values for the combo box
	uint8 LowerBoundType;
	TSharedPtr<FString> LowerBoundTypeSelectedItem;
	if (ensure(LowerBoundTypeHandle->GetValue(LowerBoundType) == FPropertyAccess::Success))
	{
		check(LowerBoundType < ComboBoxList.Num());
		LowerBoundTypeSelectedItem = ComboBoxList[LowerBoundType];
	}

	uint8 UpperBoundType;
	TSharedPtr<FString> UpperBoundTypeSelectedItem;
	if (ensure(UpperBoundTypeHandle->GetValue(UpperBoundType) == FPropertyAccess::Success))
	{
		check(UpperBoundType < ComboBoxList.Num());
		UpperBoundTypeSelectedItem = ComboBoxList[UpperBoundType];
	}

	// Build the widgets
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	.MaxDesiredWidth(200.0f)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 3.0f, 0.0f, 2.0f))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 6.0f, 0.0f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MinimumBoundLabel", "Min"))
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<NumericType>)
				.Value(this, &FRangeStructCustomization<NumericType>::OnGetValue, LowerBoundValueWeakPtr, LowerBoundTypeWeakPtr)
				.MinValue(MinAllowedValue)
				.MinSliderValue(MinAllowedValue)
				.MaxValue(this, &FRangeStructCustomization<NumericType>::OnGetValue, UpperBoundValueWeakPtr, UpperBoundTypeWeakPtr)
				.MaxSliderValue(this, &FRangeStructCustomization<NumericType>::OnGetValue, UpperBoundValueWeakPtr, UpperBoundTypeWeakPtr)
				.OnValueCommitted(this, &FRangeStructCustomization<NumericType>::OnValueCommitted, LowerBoundValueWeakPtr)
				.OnValueChanged(this, &FRangeStructCustomization<NumericType>::OnValueChanged, LowerBoundValueWeakPtr)
				.OnBeginSliderMovement(this, &FRangeStructCustomization<NumericType>::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &FRangeStructCustomization<NumericType>::OnEndSliderMovement)
				.IsEnabled(this, &FRangeStructCustomization<NumericType>::OnQueryIfEnabled, LowerBoundTypeWeakPtr)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AllowSpin(true)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&ComboBoxList)
				.OnGenerateWidget(this, &FRangeStructCustomization<NumericType>::OnGenerateComboWidget)
				.OnSelectionChanged(this, &FRangeStructCustomization<NumericType>::OnComboSelectionChanged, LowerBoundTypeWeakPtr)
				.InitiallySelectedItem(LowerBoundTypeSelectedItem)
				[
					// combo box button intentionally blank to avoid displaying excessive details
					SNew(SSpacer)
				]
			]
		]

		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 0.0f, 3.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaximumBoundLabel", "Max"))
			]
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<NumericType>)
				.Value(this, &FRangeStructCustomization<NumericType>::OnGetValue, UpperBoundValueWeakPtr, UpperBoundTypeWeakPtr)
				.MinValue(this, &FRangeStructCustomization<NumericType>::OnGetValue, LowerBoundValueWeakPtr, LowerBoundTypeWeakPtr)
				.MinSliderValue(this, &FRangeStructCustomization<NumericType>::OnGetValue, LowerBoundValueWeakPtr, LowerBoundTypeWeakPtr)
				.MaxValue(MaxAllowedValue)
				.MaxSliderValue(MaxAllowedValue)
				.OnValueCommitted(this, &FRangeStructCustomization<NumericType>::OnValueCommitted, UpperBoundValueWeakPtr)
				.OnValueChanged(this, &FRangeStructCustomization<NumericType>::OnValueChanged, UpperBoundValueWeakPtr)
				.OnBeginSliderMovement(this, &FRangeStructCustomization<NumericType>::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &FRangeStructCustomization<NumericType>::OnEndSliderMovement)
				.IsEnabled(this, &FRangeStructCustomization<NumericType>::OnQueryIfEnabled, UpperBoundTypeWeakPtr)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AllowSpin(true)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox< TSharedPtr<FString> >)
				.OptionsSource(&ComboBoxList)
				.OnGenerateWidget(this, &FRangeStructCustomization<NumericType>::OnGenerateComboWidget)
				.OnSelectionChanged(this, &FRangeStructCustomization<NumericType>::OnComboSelectionChanged, UpperBoundTypeWeakPtr)
				.InitiallySelectedItem(UpperBoundTypeSelectedItem)
				[
					// combo box button intentionally blank to avoid displaying excessive details
					SNew(SSpacer)
				]
			]
		]
	];
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Don't display children, as editing them directly can break the constraints
}


/* FRangeStructCustomization callbacks
 *****************************************************************************/

template <typename NumericType>
TOptional<NumericType> FRangeStructCustomization<NumericType>::OnGetValue(TWeakPtr<IPropertyHandle> ValueWeakPtr, TWeakPtr<IPropertyHandle> TypeWeakPtr) const
{
	auto ValueSharedPtr = ValueWeakPtr.Pin();
	auto TypeSharedPtr = TypeWeakPtr.Pin();

	if (TypeSharedPtr.IsValid())
	{
		uint8 Type;
		if (ensure(TypeSharedPtr->GetValue(Type) == FPropertyAccess::Success))
		{
			if (Type != ERangeBoundTypes::Open && ValueSharedPtr.IsValid())
			{
				NumericType Value;
				if (ensure(ValueSharedPtr->GetValue(Value) == FPropertyAccess::Success))
				{
					return TOptional<NumericType>(Value);
				}
			}
		}
	}

	// Value couldn't be accessed, or was bound type 'open'.  Return an unset value
	return TOptional<NumericType>();
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> HandleWeakPtr)
{
	auto HandleSharedPtr = HandleWeakPtr.Pin();
	if (HandleSharedPtr.IsValid() && (!bIsUsingSlider || (bIsUsingSlider && ShouldAllowSpin())))
	{
		ensure(HandleSharedPtr->SetValue(NewValue) == FPropertyAccess::Success);
	}
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> HandleWeakPtr)
{
	if (bIsUsingSlider && ShouldAllowSpin())
	{
		auto HandleSharedPtr = HandleWeakPtr.Pin();
		if (HandleSharedPtr.IsValid())
		{
			ensure(HandleSharedPtr->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success);
		}
	}
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::OnBeginSliderMovement()
{
	bIsUsingSlider = true;

	if (ShouldAllowSpin())
	{
		GEditor->BeginTransaction(LOCTEXT("SetRangeProperty", "Set Range Property"));
	}
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::OnEndSliderMovement(NumericType /*NewValue*/)
{
	bIsUsingSlider = false;

	if (ShouldAllowSpin())
	{
		GEditor->EndTransaction();
	}
}


template <typename NumericType>
bool FRangeStructCustomization<NumericType>::OnQueryIfEnabled(TWeakPtr<IPropertyHandle> HandleWeakPtr) const
{
	auto PropertyHandle = HandleWeakPtr.Pin();
	if (PropertyHandle.IsValid())
	{
		uint8 BoundType;
		if (ensure(PropertyHandle->GetValue(BoundType) == FPropertyAccess::Success))
		{
			return (BoundType != ERangeBoundTypes::Open);
		}
	}

	return false;
}


template <typename NumericType>
bool FRangeStructCustomization<NumericType>::ShouldAllowSpin() const
{
	uint8 LowerBoundType;
	uint8 UpperBoundType;
	if (ensure(LowerBoundTypeHandle->GetValue(LowerBoundType) == FPropertyAccess::Success) &&
		ensure(UpperBoundTypeHandle->GetValue(UpperBoundType) == FPropertyAccess::Success))
	{
		return (LowerBoundType != ERangeBoundTypes::Open && UpperBoundType != ERangeBoundTypes::Open);
	}

	return false;
}


template <typename NumericType>
TSharedRef<SWidget> FRangeStructCustomization<NumericType>::OnGenerateComboWidget(TSharedPtr<FString> InComboString)
{
	FText ToolTip;

	// A list of tool tips should have been populated in a 1 to 1 correspondence
	check(ComboBoxList.Num() == ComboBoxToolTips.Num());

	if (ComboBoxToolTips.Num() > 0)
	{
		int32 Index = ComboBoxList.IndexOfByKey(InComboString);
		if (ensure(Index >= 0))
		{
			ToolTip = ComboBoxToolTips[Index];
		}
	}

	return
		SNew(SBox)
		.WidthOverride(150.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*InComboString))
			.ToolTipText(ToolTip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(true)
		];
}


template <typename NumericType>
void FRangeStructCustomization<NumericType>::OnComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandleWeakPtr)
{
	auto PropertyHandle = HandleWeakPtr.Pin();
	if (PropertyHandle.IsValid())
	{
		int32 Index = ComboBoxList.IndexOfByKey(InSelectedItem);
		if (ensure(Index >= 0))
		{
			ensure(PropertyHandle->SetValue(static_cast<uint8>(Index)) == FPropertyAccess::Success);
		}
	}
}


/* Only explicitly instantiate the types which are supported
 *****************************************************************************/

template class FRangeStructCustomization<float>;
template class FRangeStructCustomization<double>;
template class FRangeStructCustomization<int32>;


#undef LOCTEXT_NAMESPACE
