// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"

class FDetailWidgetRow;
class FString;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;

/**
* Implements a details panel customization for FFloatRange structures.
*/
template <typename NumericType>
class FRangeStructCustomization : public IPropertyTypeCustomization
{
public:

	FRangeStructCustomization()
		: bIsUsingSlider(false) {}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

public:

	/**
	* Creates a new instance.
	*
	* @return A new struct customization for Guids.
	*/
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:

	/**
	 * Gets the value for the provided property handle
	 *
	 * @param	ValueWeakPtr	Handle to the property to get the value from
	 * @param	TypeWeakPtr		Handle to the property to get the type from
	 * @return	The value or unset if it could not be accessed
	 */
	TOptional<NumericType> OnGetValue(TWeakPtr<IPropertyHandle> ValueWeakPtr, TWeakPtr<IPropertyHandle> TypeWeakPtr) const;

	/**
	 * Called when the value is committed from the property editor
	 *
	 * @param	NewValue		The new value of the property
	 * @param	CommitType		How the value was committed (unused)
	 * @param	HandleWeakPtr	Handle to the property that the new value is for
	 */
	void OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> HandleWeakPtr);

	/**
	 * Called when the value is changed in the property editor
	 *
	 * @param	NewValue		The new value of the property
	 * @param	HandleWeakPtr	Handle to the property that the new value is for
	 */
	void OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> HandleWeakPtr);

	/**
	 * Called when a value starts to be changed by a slider
	 */
	void OnBeginSliderMovement();

	/**
	 * Called when a value stops being changed by a slider
	 *
	 * @param	NewValue		The new value of the property
	 */
	void OnEndSliderMovement(NumericType NewValue);

	/**
	 * Determines if the value is valid from the handle of the range type passed in
	 *
	 * @param	HandleWeakPtr	Handle to the property to get the type from
	 * @return	Whether the value is valid or not.
	 */
	bool OnQueryIfEnabled(TWeakPtr<IPropertyHandle> HandleWeakPtr) const;

	/**
	 * Determines if the spin box is enabled on a numeric value widget
	 *
	 * @return Whether the spin box should be enabled.
	 */
	bool ShouldAllowSpin() const;

	/**
	 * Generates a row of the combo widget
	 *
	 * @param	InComboString	Item string to be displayed in the combo item
	 * @return	An SWidget representing the combo row.
	 */
	TSharedRef<SWidget> OnGenerateComboWidget(TSharedPtr<FString> InComboString);

	/**
	 * Called when an item is selected in the combo box
	 *
	 * @param	InSelectedItem	String of the item selected
	 * @param	SelectInfo		Type of selection
	 * @param	HandleWeakPtr	Handle to the type property which will be changed by the combo selection
	 */
	void OnComboSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo, TWeakPtr<IPropertyHandle> HandleWeakPtr);

	// Cached shared pointers to properties that we are managing
	TSharedPtr<IPropertyHandle> LowerBoundStructHandle;
	TSharedPtr<IPropertyHandle> UpperBoundStructHandle;
	TSharedPtr<IPropertyHandle> LowerBoundValueHandle;
	TSharedPtr<IPropertyHandle> LowerBoundTypeHandle;
	TSharedPtr<IPropertyHandle> UpperBoundValueHandle;
	TSharedPtr<IPropertyHandle> UpperBoundTypeHandle;

	// Min and max allowed values from the metadata
	TOptional<NumericType> MinAllowedValue;
	TOptional<NumericType> MaxAllowedValue;

	// Arrays of combo box data: list items, and tooltips
	TArray< TSharedPtr<FString> > ComboBoxList;
	TArray< FText > ComboBoxToolTips;

	// Flags whether the slider is being moved at the moment on any of our widgets
	bool bIsUsingSlider;
};
