// Copyright Epic Games, Inc. All Rights Reserved.

#include "RawDistributionVectorStructCustomization.h"

#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomNodeBuilder.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "RawDistributionVectorStructCustomization"


class FReplaceVectorWithLinearColorBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FReplaceVectorWithLinearColorBuilder>
{
public:

	/**
	 * Constructor
	 *
	 * @param The property handle representing this item
	 */
	explicit FReplaceVectorWithLinearColorBuilder(TSharedRef<IPropertyHandle> InPropertyHandle);

	/**
	 * Sets a delegate that should be used when the custom node needs to rebuild children
	 *
	 * @param A delegate to invoke when the tree should rebuild this nodes children
	 */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}


	/**
	 * Called to generate content in the header of this node ( the actual node content ).
	 * Only called if HasHeaderRow is true
	 *
	 * @param NodeRow The row to put content in
	 */
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

	/**
	 * Called to generate child content of this node
	 *
	 * @param OutChildRows An array of rows to add children to
	 */
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

	/**
	 * Called each tick if ReqiresTick is true
	 */
	virtual void Tick(float DeltaTime) override {}

	/**
	 * @return true if this node requires tick, false otherwise
	 */
	virtual bool RequiresTick() const override { return false; }

	/**
	 * @return true if this node should be collapsed in the tree
	 */
	virtual bool InitiallyCollapsed() const override { return false; }

	/**
	 * @return The name of this custom builder.  This is used as an identifier to save expansion state if needed
	 */
	virtual FName GetName() const override;


protected:

	/**
	 * Creates a widget representing the FVector being pointed to as a FLinearColor
	 */
	TSharedRef<SWidget> CreateColorWidget(const TSharedPtr<IPropertyHandle>& StructHandle);

	/**
	 * Adds a child widget representing a component of an FLinearColor
	 *
	 * @param	StructHandle	Handle of the FVector being represented as a FLinearColor
	 * @param	Text			Label text
	 * @param	ElementIndex	Element of the FVector it represents (0-2)
	 * @param	ChildrenBuilder	The builder object
	 */
	void AddColorChildProperty(const TSharedPtr<IPropertyHandle>& StructHandle, const FText& Text, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/**
	 * Generates a child property, handling nested arrays and structs by creating a child property builder.
	 *
	 * @param	Handle			Property handle of the property being generated
	 * @param	ChildrenBuilder	The current builder object
	 */
	void GeneratePropertyContent(const TSharedRef<IPropertyHandle>& Handle, IDetailChildrenBuilder& ChildrenBuilder);

	/**
	 * Called by the array builder when it needs to generate a new child widget
	 *
	 * @param	ElementProperty	Property handle of the array element
	 * @param	ElementIndex	Index of the element in the array
	 * @param	ChildrenBuilder	The current builder object
	 */
	void OnGenerateArrayElementWidget(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/**
	 * Called by the color element widgets to determine the current color element value
	 *
	 * @param	StructHandle	Property handle of the FVector representing the color
	 * @param	ElementIndex	Element of the FVector it represents (0-2)
	 */
	TOptional<float> OnGetColorElementValue(TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex) const;

	/**
	 * Called when a numeric component is set.
	 *
	 * @param	NewValue		New element value
	 * @param	CommitType		Specifies how the result was entered
	 * @param	StructHandle	Property handle of the FVector representing the color
	 * @param	ElementIndex	Element of the FVector it represents (0-2)
	 */
	void OnColorElementValueCommitted(float NewValue, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex);

	/**
	 * Called when a numeric component is changed.
	 *
	 * @param	NewValue		New element value
	 * @param	StructHandle	Property handle of the FVector representing the color
	 * @param	ElementIndex	Element of the FVector it represents (0-2)
	 */
	void OnColorElementValueChanged(float NewValue, TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex);

	/** Called when the slider is moved on a numeric entry box */
	void OnBeginSliderMovement();

	/** Called when the slider is released on a numeric entry box */
	void OnEndSliderMovement(float NewValue);

	/** Returns the FLinearColor represented by the FVector pointed to by the property handle */
	FLinearColor OnGetColorForColorBlock(TSharedPtr<IPropertyHandle> StructHandle) const;

	/**
	 * Called when mouse is clicked on the color widget
	 */
	FReply OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<IPropertyHandle> StructHandle);

	/** Creates a color picker window */
	void CreateColorPicker(const TSharedPtr<IPropertyHandle>& StructHandle);

	/**
	 * Called when a color is chosen from the color picker
	 *
	 * @param	NewColor		Color that was chosen
	 * @param	StructHandle	Property handle of the FVector representing the color
	 */
	void OnSetColorFromColorPicker(FLinearColor NewColor, TSharedPtr<IPropertyHandle> StructHandle);

	/**
	 * Called when the color picker is cancelled
	 *
	 * @param	OriginalColor	Original value of the property
	 * @param	StructHandle	Property handle of the FVector representing the color
	 */
	void OnColorPickerCancelled(FLinearColor OriginalColor, TSharedPtr<IPropertyHandle> StructHandle);

	/** Called when a drag starts in the color picker */
	void OnColorPickerInteractiveBegin();

	/** Called when a drag ends in the color picker */
	void OnColorPickerInteractiveEnd();

private:

	/** Holds the property handle being referenced by this builder object */
	TSharedRef<IPropertyHandle> PropertyHandle;

	/** true if the property is an FVector, and hence needs customization */
	bool bIsVectorProperty;

	/** true if the slider is being dragged in a numeric entry box */
	bool bIsUsingSlider;

	/** Original value of the property, prior to using the color picker */
	FLinearColor OldColorValue;

	/** Widget the Color Picker is parented to */
	TSharedPtr<SWidget> ColorPickerParentWidget;
};


// Some helper functions for obtaining values from the FVector property, and converting them to FLinearColor
namespace
{
	float GetColorElementValue(const TSharedPtr<IPropertyHandle>& StructHandle, int32 ElementIndex)
	{
		FVector Result;
		ensure(StructHandle->GetValue(Result) == FPropertyAccess::Success);
		return static_cast<float>(Result[ElementIndex]);
	}

	void SetColorElementValue(const TSharedPtr<IPropertyHandle>& StructHandle, int32 ElementIndex, float Value)
	{
		FVector Result;
		ensure(StructHandle->GetValue(Result) == FPropertyAccess::Success);
		Result[ElementIndex] = Value;
		ensure(StructHandle->SetValue(Result) == FPropertyAccess::Success);
	}

	FLinearColor GetColorValue(const TSharedPtr<IPropertyHandle>& StructHandle)
	{
		FVector Result;
		ensure(StructHandle->GetValue(Result) == FPropertyAccess::Success);
		return FLinearColor(static_cast<float>(Result.X), static_cast<float>(Result.Y), static_cast<float>(Result.Z), 1.0f);
	}

	void SetColorValue(const TSharedPtr<IPropertyHandle>& StructHandle, FLinearColor Value)
	{
		FVector ValueAsVector(Value.R, Value.G, Value.B);
		ensure(StructHandle->SetValue(ValueAsVector) == FPropertyAccess::Success);
	}
}


FReplaceVectorWithLinearColorBuilder::FReplaceVectorWithLinearColorBuilder(TSharedRef<IPropertyHandle> InPropertyHandle)
	: PropertyHandle(InPropertyHandle)
	, bIsVectorProperty(false)
	, bIsUsingSlider(false)
{
	// Determine if this is an FVector - if so it will be specialized
	FProperty* Property = InPropertyHandle->GetProperty();

	if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		FName StructType = StructProperty->Struct->GetFName();
		bIsVectorProperty = (StructType == NAME_Vector);
	}
}


void FReplaceVectorWithLinearColorBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	// Only generate a header row if the handle has a valid FProperty.
	// Note that it's possible for the Property to be NULL if the property node is an FObjectPropertyNode - however we still want to create children in this case.
	if (PropertyHandle->GetProperty() != nullptr)
	{
		NodeRow.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			];

		if (bIsVectorProperty)
		{
			// Customization - make FVector look like an FLinearColor
			NodeRow.ValueContent()
				.MinDesiredWidth(250.0f)
				.MaxDesiredWidth(250.0f)
				[
					CreateColorWidget(PropertyHandle)
				];
		}
		else
		{
			// Otherwise, use the default property widget
			NodeRow.ValueContent()
				.MinDesiredWidth(1)
				.MaxDesiredWidth(4096)
				[
					PropertyHandle->CreatePropertyValueWidget()
				];
		}
	}
}


void FReplaceVectorWithLinearColorBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (bIsVectorProperty)
	{
		// Customization - children of an FVector are made to look like color components of an FLinearColor
		AddColorChildProperty(PropertyHandle, LOCTEXT("RedComponent", "R"), 0, ChildrenBuilder);
		AddColorChildProperty(PropertyHandle, LOCTEXT("GreenComponent", "G"), 1, ChildrenBuilder);
		AddColorChildProperty(PropertyHandle, LOCTEXT("BlueComponent", "B"), 2, ChildrenBuilder);
	}
	else
	{
		// Otherwise, go through the child properties and render them as normal
		uint32 NumChildren;
		ensure(PropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex);
			check(ChildHandle.IsValid());
			GeneratePropertyContent(ChildHandle.ToSharedRef(), ChildrenBuilder);
		}
	}
}


FName FReplaceVectorWithLinearColorBuilder::GetName() const
{
	if (PropertyHandle->GetProperty() != nullptr)
	{
		return PropertyHandle->GetProperty()->GetFName();
	}

	return NAME_None;
}



TSharedRef<SWidget> FReplaceVectorWithLinearColorBuilder::CreateColorWidget(const TSharedPtr<IPropertyHandle>& StructHandle)
{
	FSlateFontInfo NormalText = IDetailLayoutBuilder::GetDetailFont();

	return SNew(SBox)
		.VAlign(VAlign_Center)
		[
			SAssignNew(ColorPickerParentWidget, SColorBlock)
			.Color(this, &FReplaceVectorWithLinearColorBuilder::OnGetColorForColorBlock, StructHandle)
			.ShowBackgroundForAlpha(false)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.OnMouseButtonDown(this, &FReplaceVectorWithLinearColorBuilder::OnMouseButtonDownColorBlock, StructHandle)
			.Size(FVector2D(70.0f, 12.0f))
		];
}


void FReplaceVectorWithLinearColorBuilder::AddColorChildProperty(const TSharedPtr<IPropertyHandle>& StructHandle, const FText& Text, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	ChildrenBuilder.AddCustomRow(LOCTEXT("Color", "Color"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(Text)
			.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
		]
	.ValueContent()
		.MinDesiredWidth(100.0f)
		.MaxDesiredWidth(100.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
			.Value(this, &FReplaceVectorWithLinearColorBuilder::OnGetColorElementValue, StructHandle, ElementIndex)
			.OnValueCommitted(this, &FReplaceVectorWithLinearColorBuilder::OnColorElementValueCommitted, StructHandle, ElementIndex)
			.OnValueChanged(this, &FReplaceVectorWithLinearColorBuilder::OnColorElementValueChanged, StructHandle, ElementIndex)
			.OnBeginSliderMovement(this, &FReplaceVectorWithLinearColorBuilder::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &FReplaceVectorWithLinearColorBuilder::OnEndSliderMovement)
			.AllowSpin(true)
			.MinSliderValue(0.0f)
			.MaxSliderValue(1.0f)
		];
}


void FReplaceVectorWithLinearColorBuilder::GeneratePropertyContent(const TSharedRef<IPropertyHandle>& Handle, IDetailChildrenBuilder& ChildrenBuilder)
{
	// Add to the current builder, depending on the property type.
	uint32 NumChildren = 0;
	ensure(Handle->GetNumChildren(NumChildren) == FPropertyAccess::Success);
	bool bHasChildren = (NumChildren > 0);
	bool bIsArray = Handle->AsArray().IsValid();

	if (bIsArray)
	{
		// Arrays need special handling and will create an array builder
		TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShareable(new FDetailArrayBuilder(Handle));
		ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FReplaceVectorWithLinearColorBuilder::OnGenerateArrayElementWidget));
		ChildrenBuilder.AddCustomBuilder(ArrayBuilder);
	}
	else if (bHasChildren)
	{
		// If there are children, we invoke a new instance of our custom builder for recursive handling
		// Note, if this is an FVector, it will be handled specially by the implementation of the IDetailCustomNodeBuilder interface.
		TSharedRef<FReplaceVectorWithLinearColorBuilder> StructBuilder = MakeShareable(new FReplaceVectorWithLinearColorBuilder(Handle));
		ChildrenBuilder.AddCustomBuilder(StructBuilder);
	}
	else
	{
		// No children - just add the property.
		ChildrenBuilder.AddProperty(Handle);
	}
}


void FReplaceVectorWithLinearColorBuilder::OnGenerateArrayElementWidget(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	GeneratePropertyContent(ElementProperty, ChildrenBuilder);
}


TOptional<float> FReplaceVectorWithLinearColorBuilder::OnGetColorElementValue(TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex) const
{
	return GetColorElementValue(StructHandle, ElementIndex);
}


void FReplaceVectorWithLinearColorBuilder::OnColorElementValueCommitted(float NewValue, ETextCommit::Type CommitType, TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex)
{
	SetColorElementValue(StructHandle, ElementIndex, NewValue);
}


void FReplaceVectorWithLinearColorBuilder::OnColorElementValueChanged(float NewValue, TSharedPtr<IPropertyHandle> StructHandle, int32 ElementIndex)
{
	if (bIsUsingSlider)
	{
		SetColorElementValue(StructHandle, ElementIndex, NewValue);
	}
}


FLinearColor FReplaceVectorWithLinearColorBuilder::OnGetColorForColorBlock(TSharedPtr<IPropertyHandle> StructHandle) const
{
	return GetColorValue(StructHandle);
}


void FReplaceVectorWithLinearColorBuilder::OnBeginSliderMovement()
{
	GEditor->BeginTransaction(LOCTEXT("SetColorProperty", "Set Color Property"));
	bIsUsingSlider = true;
}


void FReplaceVectorWithLinearColorBuilder::OnEndSliderMovement(float NewValue)
{
	bIsUsingSlider = false;
	GEditor->EndTransaction();
}


FReply FReplaceVectorWithLinearColorBuilder::OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TSharedPtr<IPropertyHandle> StructHandle)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	CreateColorPicker(StructHandle);
	return FReply::Handled();
}


void FReplaceVectorWithLinearColorBuilder::CreateColorPicker(const TSharedPtr<IPropertyHandle>& StructHandle)
{
	OldColorValue = GetColorValue(StructHandle);

	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.bOnlyRefreshOnMouseUp = false;
	PickerArgs.bOnlyRefreshOnOk = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FReplaceVectorWithLinearColorBuilder::OnSetColorFromColorPicker, StructHandle);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FReplaceVectorWithLinearColorBuilder::OnColorPickerCancelled, StructHandle);
	PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &FReplaceVectorWithLinearColorBuilder::OnColorPickerInteractiveBegin);
	PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &FReplaceVectorWithLinearColorBuilder::OnColorPickerInteractiveEnd);
	PickerArgs.InitialColor = OldColorValue;
	PickerArgs.ParentWidget = ColorPickerParentWidget;

	OpenColorPicker(PickerArgs);
}


void FReplaceVectorWithLinearColorBuilder::OnSetColorFromColorPicker(FLinearColor NewColor, TSharedPtr<IPropertyHandle> StructHandle)
{
	SetColorValue(StructHandle, NewColor);
}


void FReplaceVectorWithLinearColorBuilder::OnColorPickerCancelled(FLinearColor OriginalColor, TSharedPtr<IPropertyHandle> StructHandle)
{
	SetColorValue(StructHandle, OldColorValue);
}


void FReplaceVectorWithLinearColorBuilder::OnColorPickerInteractiveBegin()
{
	GEditor->BeginTransaction(LOCTEXT("SetColorProperty", "Set Color Property"));
}


void FReplaceVectorWithLinearColorBuilder::OnColorPickerInteractiveEnd()
{
	GEditor->EndTransaction();
}


TSharedRef<IPropertyTypeCustomization> FRawDistributionVectorStructCustomization::MakeInstance()
{
	return MakeShareable(new FRawDistributionVectorStructCustomization);
}


void FRawDistributionVectorStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(1)
	.MaxDesiredWidth(4096)
	[
		StructPropertyHandle->CreatePropertyValueWidget()
	];
}


void FRawDistributionVectorStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Determine from the metadata whether we should treat vectors as FLinearColors or not
	bool bTreatAsColor = StructPropertyHandle->HasMetaData("TreatAsColor");

	uint32 NumChildren;
	ensure(StructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success);

	// Now recurse through all children, creating a custom builder for each which will either add the default property row, or
	// a property row exposing a FLinearColor type customization which maps directly to the elements of the original FVector.
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex);
		check(ChildHandle.IsValid());
		if (bTreatAsColor)
		{
			TSharedRef<FReplaceVectorWithLinearColorBuilder> CustomBuilder = MakeShareable(new FReplaceVectorWithLinearColorBuilder(ChildHandle.ToSharedRef()));
			StructBuilder.AddCustomBuilder(CustomBuilder);
		}
		else
		{
			StructBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}


#undef LOCTEXT_NAMESPACE
