// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReflectionCaptureDetails.h"

#include "Components/ReflectionCaptureComponent.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/IConsoleManager.h"
#include "Materials/Material.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "ReflectionCaptureDetails"



TSharedRef<IDetailCustomization> FReflectionCaptureDetails::MakeInstance()
{
	return MakeShareable( new FReflectionCaptureDetails );
}

void FReflectionCaptureDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	TSharedPtr<IPropertyHandle> PostProcessMaterialHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, PostProcessMaterial));
	if (!PostProcessMaterialHandle->IsValidHandle())
	{
		return;
	}

	IDetailCategoryBuilder& RuntimeCaptureCategory = DetailLayout.EditCategory("RuntimeCapture");

	// Explicitly add bRuntimeCapture first to preserve the declaration order when PostProcessMaterial is also added explicitly below.
	// Customize the value widget to append a warning icon when bRuntimeCapture mismatches r.ReflectionCapture.Runtime: a baked
	// capture won't render in runtime-only mode and a runtime capture won't render in baked-only mode.
	TSharedPtr<IPropertyHandle> RuntimeCaptureHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UReflectionCaptureComponent, bRuntimeCapture));
	IDetailPropertyRow& RuntimeCaptureRow = RuntimeCaptureCategory.AddProperty(RuntimeCaptureHandle.ToSharedRef());

	auto IsRuntimeModeMismatched = [RuntimeCaptureHandle]() -> bool
	{
		bool bRuntime = false;
		if (RuntimeCaptureHandle->GetValue(bRuntime) != FPropertyAccess::Success)
		{
			return false;  // multi-edit with mixed values, etc.
		}
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCapture.Runtime"));
		const bool bRuntimeMode = CVar && CVar->GetValueOnGameThread() != 0;
		return bRuntime != bRuntimeMode;
	};

	RuntimeCaptureRow.CustomWidget()
		.NameContent()
		[
			RuntimeCaptureHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					RuntimeCaptureHandle->CreatePropertyValueWidget()
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					PropertyCustomizationHelpers::MakeCustomButton(
						FAppStyle::Get().GetBrush("Icons.WarningWithColor"),
						FSimpleDelegate(),
						LOCTEXT("RuntimeCaptureCVarMismatch", "This capture won't render in the current reflection capture mode.\nr.ReflectionCapture.Runtime selects baked (0) or runtime (1) captures.\nRuntimeCapture must match the CVar for this capture to be visible."),
						false,
						TAttribute<EVisibility>::CreateLambda([IsRuntimeModeMismatched]()
						{
							return IsRuntimeModeMismatched() ? EVisibility::Visible : EVisibility::Collapsed;
						}))
				]
		];
	IDetailPropertyRow& PropertyRow = RuntimeCaptureCategory.AddProperty(PostProcessMaterialHandle.ToSharedRef());

	TSharedRef<SObjectPropertyEntryBox> ObjectEntryBox =
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(PostProcessMaterialHandle)
		.AllowedClass(UMaterialInterface::StaticClass())
		.ThumbnailPool(DetailLayout.GetThumbnailPool())
		.CustomButtonSlot()
		[
			PropertyCustomizationHelpers::MakeCustomButton(
				FAppStyle::Get().GetBrush("Icons.ErrorWithColor"),
				FSimpleDelegate(),
				LOCTEXT("InvalidPostProcessMaterial", "Invalid reflection capture post process material.\nConfirm that Blendable Location is Scene Color Before Bloom."),
				false,
				TAttribute<EVisibility>::CreateLambda([PostProcessMaterialHandle]()
				{
					UObject* MaterialObj = nullptr;
					PostProcessMaterialHandle->GetValue(MaterialObj);
					UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialObj);
					if (!MaterialInterface)
					{
						return EVisibility::Collapsed;
					}

					UMaterial* BaseMaterial = MaterialInterface->GetMaterial();
					if (BaseMaterial &&
						BaseMaterial->IsPostProcessMaterial() &&
						MaterialInterface->GetUserSceneTextureOutput(BaseMaterial).IsNone() &&
						MaterialInterface->GetBlendableLocation(BaseMaterial) == BL_SceneColorBeforeBloom)
					{
						return EVisibility::Collapsed;
					}

					return EVisibility::Visible;
				}))
		];

	float MinWidth, MaxWidth;
	ObjectEntryBox->GetDesiredWidth(MinWidth, MaxWidth);

	PropertyRow.CustomWidget()
		.NameContent()
		[
			PostProcessMaterialHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(MinWidth)
		.MaxDesiredWidth(MaxWidth)
		[
			ObjectEntryBox
		];
}

#undef LOCTEXT_NAMESPACE
