// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderPassesCustomization.h"

#include "BufferVisualizationData.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Protocols/CompositionGraphCaptureProtocol.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UMaterialInterface;

#define LOCTEXT_NAMESPACE "RenderPassesCustomization"


class SRenderPassesCustomization : public SCompoundWidget
{
public:
	struct FRenderPassInfo
	{
		FString Name;
		FText Text;
	};

	SLATE_BEGIN_ARGS(SRenderPassesCustomization){}
		SLATE_ARGUMENT(FCompositionGraphCapturePasses*, Property)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Property = InArgs._Property;

		auto OnGenerateWidget = [](TSharedPtr<FRenderPassInfo> RenderPass){
			return SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(RenderPass->Text);
		};

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ComboBox, SComboBox<TSharedPtr<FRenderPassInfo>>)
				.OptionsSource(&ComboEntries)
				.OnSelectionChanged(this, &SRenderPassesCustomization::OnAddElement)
				.OnGenerateWidget_Lambda(OnGenerateWidget)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("ComboText", "Add Render Pass..."))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(EnabledPassesContainer, SVerticalBox)
			]
		];

		Update();
	}

	void Update()
	{
		TMap<FString, FText> AllAvailablePasses;

		ComboEntries.Reset();

		EnabledPassesContainer->ClearChildren();

		struct FIterator
		{
			TMap<FString, FText>& RenderPasses;

			FIterator(TMap<FString, FText>& InRenderPasses) : RenderPasses(InRenderPasses) {}

			void ProcessValue(const FString& InName, UMaterialInterface* Material, const FText& InText)
			{
				RenderPasses.Add(InName, InText);
			}
		} Iterator(AllAvailablePasses);

		GetBufferVisualizationData().IterateOverAvailableMaterials(Iterator);

		for (auto& Pair : AllAvailablePasses)
		{
			const int32 PropertyIndex = Property->Value.IndexOfByKey(Pair.Key);
			if (PropertyIndex == INDEX_NONE)
			{
				ComboEntries.Add(MakeShareable( new FRenderPassInfo{ Pair.Key, Pair.Value } ));
			}
			else
			{
				TSharedRef<SWidget> RemoveButton = PropertyCustomizationHelpers::MakeRemoveButton(
					FSimpleDelegate::CreateLambda([this, PropertyIndex] 
					{
						if (PropertyIndex < Property->Value.Num()) 
						{
							Property->Value.RemoveAt(PropertyIndex);
							Update();
						}
					}));

				EnabledPassesContainer->AddSlot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(FMargin(5.f, 0))
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(Pair.Value)
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(5.f, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						RemoveButton
					]
				];
			}
		}

		ComboEntries.Sort([](const TSharedPtr<FRenderPassInfo>& A, const TSharedPtr<FRenderPassInfo>& B){
			return A->Text.CompareToCaseIgnored(B->Text) < 0;
		});

		ComboBox->ClearSelection();
		ComboBox->SetVisibility(ComboEntries.Num() == 0 ? EVisibility::Collapsed : EVisibility::Visible);
	}
	
private:

	void OnAddElement(TSharedPtr<FRenderPassInfo> RenderPass, ESelectInfo::Type)
	{
		if (RenderPass.IsValid())
		{
			Property->Value.Add(RenderPass->Name);
			Update();
		}
	}

private:
	TArray<TSharedPtr<FRenderPassInfo>> ComboEntries;
	/** The property that is an array of FStrings */
	FCompositionGraphCapturePasses* Property;

	TSharedPtr<SComboBox<TSharedPtr<FRenderPassInfo>>> ComboBox;
	TSharedPtr<SVerticalBox> EnabledPassesContainer;
};

TSharedRef<IPropertyTypeCustomization> FRenderPassesCustomization::MakeInstance()
{
	return MakeShareable( new FRenderPassesCustomization );
}

void FRenderPassesCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils )
{
	FCompositionGraphCapturePasses* Settings = nullptr;

	if (InPropertyHandle->IsValidHandle())
	{
		TArray<void*> StructPtrs;
		InPropertyHandle->AccessRawData(StructPtrs);
		for (void* RawPtr : StructPtrs)
		{
			if (RawPtr)
			{
				Settings = reinterpret_cast<FCompositionGraphCapturePasses*>(RawPtr);
				break;
			}
		}
	}

	HeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(TOptional<float>())
	.MinDesiredWidth(200.f)
	[
		SNew(SRenderPassesCustomization)
		.Property(Settings)
	];
}

#undef LOCTEXT_NAMESPACE
