// Copyright Epic Games, Inc. All Rights Reserved.

#include "RotatorStructCustomization.h"

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AxisDisplayInfo.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

class IPropertyTypeCustomization;

#define LOCTEXT_NAMESPACE "FRotatorStructCustomization"

namespace UE::Editor::DetailCustomizations::Private
{
	static TAutoConsoleVariable<bool> ShowRotator3Children(
		TEXT("Editor.DetailCustomizations.ShowRotator3Children"),
		true,
		TEXT("When true, the detail customizations for Rotator3 variants expand to show children")
	);
}

TSharedRef<IPropertyTypeCustomization> FRotatorStructCustomization::MakeInstance() 
{
	return MakeShareable(new FRotatorStructCustomization);
}

void FRotatorStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (UE::Editor::DetailCustomizations::Private::ShowRotator3Children.GetValueOnGameThread())
	{
		FMathStructCustomization::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
	}
}

void FRotatorStructCustomization::GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren)
{
	static const FName Roll("Roll");
	static const FName Pitch("Pitch");
	static const FName Yaw("Yaw");

	const bool bUseForwardRightUpDisplayNames = AxisDisplayInfo::UseForwardRightUpDisplayNames();

	TSharedPtr< IPropertyHandle > RotatorChildren[3];

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == Roll)
		{
			if (bUseForwardRightUpDisplayNames)
			{
				ChildHandle->SetPropertyDisplayName(LOCTEXT("RollDisplayName", "Roll"));
				ChildHandle->SetToolTipText(LOCTEXT("RollToolTip", "Roll (degrees) around Forward (was X) axis"));
			}

			RotatorChildren[0] = ChildHandle;
		}
		else if (PropertyName == Pitch)
		{
			if (bUseForwardRightUpDisplayNames)
			{
				ChildHandle->SetPropertyDisplayName(LOCTEXT("PitchDisplayName", "Pitch"));
				ChildHandle->SetToolTipText(LOCTEXT("PitchToolTip", "Pitch (degrees) around Right (was Y) axis"));
			}

			RotatorChildren[1] = ChildHandle;
		}
		else
		{
			check(PropertyName == Yaw);

			if (bUseForwardRightUpDisplayNames)
			{
				ChildHandle->SetPropertyDisplayName(LOCTEXT("YawDisplayName", "Yaw"));
				ChildHandle->SetToolTipText(LOCTEXT("YawToolTip", "Yaw (degrees) around Up (was Z) axis"));
			}

			RotatorChildren[2] = ChildHandle;
		}
	}

	OutChildren.Add(RotatorChildren[0].ToSharedRef());
	OutChildren.Add(RotatorChildren[1].ToSharedRef());
	OutChildren.Add(RotatorChildren[2].ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
