// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Customizations/MathStructCustomizations.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IPropertyTypeCustomization;

/**
 * Customizes FRotator structs.
 */
class FRotatorStructCustomization
	: public FMathStructCustomization
{
public:

	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** FMathStructCustomization interface */
	virtual void GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray<TSharedRef<IPropertyHandle>>& OutChildren) override;
};
