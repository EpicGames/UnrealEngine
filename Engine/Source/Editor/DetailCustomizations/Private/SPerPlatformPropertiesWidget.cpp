// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPerPlatformPropertiesWidget.h"

#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

void SPerPlatformPropertiesRow::Construct(const FArguments& InArgs, FName PlatformName)
{
	this->OnGenerateWidget = InArgs._OnGenerateWidget;
	this->OnRemovePlatform = InArgs._OnRemovePlatform;

	ChildSlot
	[
		MakePerPlatformWidget(PlatformName)
	];

}


FReply SPerPlatformPropertiesRow::RemovePlatform(FName PlatformName)
{
	if (OnRemovePlatform.IsBound())
	{
		OnRemovePlatform.Execute(PlatformName);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SPerPlatformPropertiesRow::MakePerPlatformWidget(FName InName)
{
	TSharedPtr<SHorizontalBox> HorizontalBox;

	TSharedRef<SWidget> Widget = 
		SNew(SBox)
		.Padding(FMargin(0.0f, 2.0f, 4.0f, 2.0f))
		.MinDesiredWidth(50.0f)
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			.ToolTipText((InName != NAME_None)
				? FText::Format(NSLOCTEXT("SPerPlatformPropertiesWidget", "PerPlatformDesc", "Override for {0}"), FText::AsCultureInvariant(InName.ToString()))
				: NSLOCTEXT("SPerPlatformPropertiesWidget", "DefaultDesc", "Default value for properties without an override"))
			+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				OnGenerateWidget.Execute(InName)
			]
		];

	if(InName != NAME_None)
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SPerPlatformPropertiesRow::RemovePlatform, InName)
				.ToolTipText(FText::Format(NSLOCTEXT("SPerPlatformPropertiesWidget", "RemoveOverrideFor", "Remove Override for {0}"), FText::AsCultureInvariant(InName.ToString())))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}


	return Widget;
}