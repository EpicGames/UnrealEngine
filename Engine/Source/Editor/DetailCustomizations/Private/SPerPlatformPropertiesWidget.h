// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PerPlatformPropertyCustomization.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class SWidget;


/**
* SPerPlatformPropertiesWidget
*/
class SPerPlatformPropertiesRow : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPerPlatformPropertiesRow)
		{}
		
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)
		SLATE_EVENT(FOnPlatformOverrideAction, OnRemovePlatform)

	SLATE_END_ARGS()

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	void Construct(const FArguments& InArgs, FName PlatformName);

protected:
	TSharedRef<SWidget> MakePerPlatformWidget(FName InName);

	FReply RemovePlatform(FName PlatformName);

	EActiveTimerReturnType CheckPlatformCount(double InCurrentTime, float InDeltaSeconds);

	void AddPlatformToMenu(const FName& PlatformName, const FTextFormat Format, FMenuBuilder& AddPlatformMenuBuilder);

	FOnGenerateWidget OnGenerateWidget;
	FOnPlatformOverrideAction OnRemovePlatform;
	TAttribute<TArray<FName>> PlatformOverrideNames;
	int32 LastPlatformOverrideNames;
	bool bAddedMenuItem;
};

