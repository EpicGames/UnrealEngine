// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneCaptureDetails.h"

#include "Components/SceneCaptureComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Components/SceneCaptureComponentCube.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailGroup.h"
#include "IRenderCaptureProvider.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "RenderCaptureInterface.h"
#include "ScopedTransaction.h"
#include "ShowFlags.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneCaptureDetails"

TSharedRef<IDetailCustomization> FSceneCaptureDetails::MakeInstance()
{
	return MakeShareable( new FSceneCaptureDetails );
}

namespace UE
{
	namespace DetailsCustomization
	{
		namespace Private
		{
			// Used to sort the show flags inside of their categories in the order of the text that is actually displayed
			inline static bool SortAlphabeticallyByLocalizedText(const FString& ip1, const FString& ip2)
			{
				FText LocalizedText1;
				FEngineShowFlags::FindShowFlagDisplayName(ip1, LocalizedText1);

				FText LocalizedText2;
				FEngineShowFlags::FindShowFlagDisplayName(ip2, LocalizedText2);

				return LocalizedText1.ToString() < LocalizedText2.ToString();
			}

			// Utility function to get a show flag state by name.
			static bool GetShowFlagState(const USceneCaptureComponent* SceneCaptureComponent, FStringView ShowFlagName)
			{
				if (!IsValid(SceneCaptureComponent))
				{
					return false;
				}

				const int32 SettingIndex = SceneCaptureComponent->ShowFlags.FindIndexByName(ShowFlagName.GetData());
				return SettingIndex != INDEX_NONE ? SceneCaptureComponent->ShowFlags.GetSingleFlag(SettingIndex) : false;
			}

			// Update archetype instances so that once the value matches the default, the instance override will be removed. This correctly mimics the behavior of other checkbox states.
			static void UpdateArchetypeChainShowFlagSettings(USceneCaptureComponent* SceneCaptureComponent, FStringView ShowFlagName, bool bEnabledState)
			{
				if (!IsValid(SceneCaptureComponent))
				{
					return;
				}

				FProperty* ShowFlagsSettingsPropertyVariable = FindFieldChecked<FProperty>(USceneCaptureComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USceneCaptureComponent, ShowFlagSettings));
				FPropertyChangedEvent PropertyChangedEvent(ShowFlagsSettingsPropertyVariable);

				// Update archetype instances so that once the value matches the default, the instance override will be removed.
				// This correctly mimics the behavior of other checkbox states.
				TArray<UObject*> Instances;
				SceneCaptureComponent->GetArchetypeInstances(Instances);
				for (UObject* Instance : Instances)
				{
					if (USceneCaptureComponent* InstanceComponent = Cast<USceneCaptureComponent>(Instance))
					{
						TArray<FEngineShowFlagsSetting> InstanceShowFlagSettings = InstanceComponent->GetShowFlagSettings();
						const int32 RemovedElements = InstanceShowFlagSettings.RemoveAll([ShowFlagName, bEnabledState](const FEngineShowFlagsSetting& InSetting)
							{
								return ShowFlagName.Equals(InSetting.ShowFlagName) && (bEnabledState == InSetting.Enabled);
							});
						if (RemovedElements > 0)
						{
							InstanceComponent->PreEditChange(ShowFlagsSettingsPropertyVariable);
							InstanceComponent->SetShowFlagSettings(InstanceShowFlagSettings);
							InstanceComponent->PostEditChangeProperty(PropertyChangedEvent);
						}
					}
				}
			}
		}
	}
}


void FSceneCaptureDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	IDetailCategoryBuilder& SceneCaptureCategoryBuilder = DetailLayout.EditCategory("SceneCapture");

	ShowFlagSettingsProperty = DetailLayout.GetProperty("ShowFlagSettings", USceneCaptureComponent::StaticClass());
	check(ShowFlagSettingsProperty->IsValidHandle());
	ShowFlagSettingsProperty->MarkHiddenByCustomization();
	
	// Add all the properties that are there by default
	// (These would get added by default anyway, but we want to add them first so what we add next comes later in the list)
	TArray<TSharedRef<IPropertyHandle>> SceneCaptureCategoryDefaultProperties;
	SceneCaptureCategoryBuilder.GetDefaultProperties(SceneCaptureCategoryDefaultProperties);
	for (TSharedRef<IPropertyHandle> Handle : SceneCaptureCategoryDefaultProperties)
	{
		if (Handle->GetProperty() != ShowFlagSettingsProperty->GetProperty())
		{
			SceneCaptureCategoryBuilder.AddProperty(Handle);
		}
	}

	// Show flags that should be exposed for Scene Captures
	TArray<FEngineShowFlags::EShowFlag> ShowFlagsToAllowForCaptures;

	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Atmosphere);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_DeferredAtmospherePass);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Cloud);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_BSP);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Decals);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Fog);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Landscape);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Particles);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_SkeletalMeshes);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_StaticMeshes);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Translucency);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Lighting);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_DeferredLighting);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_NaniteMeshes);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_InstancedStaticMeshes);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_InstancedFoliage);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_InstancedGrass);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Paper2DSprites);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_TextRender);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_AmbientOcclusion);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_DynamicShadows);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_SkyLighting);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_VolumetricFog);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_AmbientCubemap);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_DistanceFieldAO);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_LightFunctions);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_LightShafts);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_PostProcessing);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_ReflectionEnvironment);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_ScreenSpaceReflections);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_TexturedLightProfiles);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_AntiAliasing);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_TemporalAA);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_MotionBlur);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Bloom);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_LocalExposure);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_EyeAdaptation);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Game);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_ToneCurve); 
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_PathTracing);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_Tonemapper);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_PostProcessMaterial);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_AllowPrimitiveAlphaHoldout);
	ShowFlagsToAllowForCaptures.Add(FEngineShowFlags::EShowFlag::SF_LensDistortion);

	// Create array of flag name strings for each group
	TArray< TArray<FString> > ShowFlagsByGroup;
	for (int32 GroupIndex = 0; GroupIndex < SFG_Max; ++GroupIndex)
	{
		ShowFlagsByGroup.Add(TArray<FString>());
	}

	// Add the show flags we want to expose to their group's array
	for (FEngineShowFlags::EShowFlag AllowedFlag : ShowFlagsToAllowForCaptures)
	{
		FString FlagName;
		FlagName = FEngineShowFlags::FindNameByIndex(AllowedFlag);
		if (!FlagName.IsEmpty())
		{
			EShowFlagGroup Group = FEngineShowFlags::FindShowFlagGroup(*FlagName);
			ShowFlagsByGroup[Group].Add(FlagName);
		}
	}

	// Sort the flags in their respective group alphabetically
	for (TArray<FString>& ShowFlagGroup : ShowFlagsByGroup)
	{
		ShowFlagGroup.Sort(UE::DetailsCustomization::Private::SortAlphabeticallyByLocalizedText);
	}

	// Add each group
	for (int32 GroupIndex = 0; GroupIndex < SFG_Max; ++GroupIndex)
	{
		// Don't add a group if there are no flags allowed for it
		if (ShowFlagsByGroup[GroupIndex].Num() >= 1)
		{
			FText GroupName;
			FText GroupTooltip;
			switch (GroupIndex)
			{
			case SFG_Normal:
				GroupName = LOCTEXT("CommonShowFlagHeader", "General Show Flags");
				break;
			case SFG_Advanced:
				GroupName = LOCTEXT("AdvancedShowFlagsMenu", "Advanced Show Flags");
				break;
			case SFG_PostProcess:
				GroupName = LOCTEXT("PostProcessShowFlagsMenu", "Post Processing Show Flags");
				break;
			case SFG_Developer:
				GroupName = LOCTEXT("DeveloperShowFlagsMenu", "Developer Show Flags");
				break;
			case SFG_Visualize:
				GroupName = LOCTEXT("VisualizeShowFlagsMenu", "Visualize Show Flags");
				break;
			case SFG_LightTypes:
				GroupName = LOCTEXT("LightTypesShowFlagsMenu", "Light Types Show Flags");
				break;
			case SFG_LightingComponents:
				GroupName = LOCTEXT("LightingComponentsShowFlagsMenu", "Lighting Components Show Flags");
				break;
			case SFG_LightingFeatures:
				GroupName = LOCTEXT("LightingFeaturesShowFlagsMenu", "Lighting Features Show Flags");
				break;
			case SFG_Lumen:
				GroupName = LOCTEXT("LumenFeaturesShowFlagsMenu", "Lumen Show Flags");
				break;
			case SFG_MegaLights:
				GroupName = LOCTEXT("MegaLightsFeaturesShowFlagsMenu", "MegaLights Show Flags");
				break;
			case SFG_Nanite:
				GroupName = LOCTEXT("NaniteFeaturesShowFlagsMenu", "Nanite Show Flags");
				break;
			case SFG_CollisionModes:
				GroupName = LOCTEXT("CollisionModesShowFlagsMenu", "Collision Modes Show Flags");
				break;
			case SFG_Hidden:
			case SFG_Transient:
				GroupName = LOCTEXT("HiddenShowFlagsMenu", "Hidden Show Flags");
				break;
			default:
				// Should not get here unless a new group is added without being updated here
				GroupName = LOCTEXT("MiscFlagsMenu", "Misc Show Flags");
				break;
			}

			FName GroupFName = FName(*(GroupName.ToString()));
			IDetailGroup& Group = SceneCaptureCategoryBuilder.AddGroup(GroupFName, GroupName, true);

			// Add each show flag for this group
			for (FString& FlagName : ShowFlagsByGroup[GroupIndex])
			{
				bool bFlagHidden = false;
				FText LocalizedText;
				FEngineShowFlags::FindShowFlagDisplayName(FlagName, LocalizedText);

				Group.AddWidgetRow()
					.IsEnabled(true)
					.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>::CreateSP(this, &FSceneCaptureDetails::GetShowFlagResetVisibility, FlagName), FSimpleDelegate::CreateSP(this, &FSceneCaptureDetails::OnResetShowFlag, FlagName)))
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LocalizedText)
					]
				.ValueContent()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(this, &FSceneCaptureDetails::OnShowFlagCheckStateChanged, FlagName)
						.IsChecked(this, &FSceneCaptureDetails::OnGetDisplayCheckState, FlagName)
					]
				.FilterString(LocalizedText);
			}
		}
	}

	if (IRenderCaptureProvider::IsAvailable())
	{
		IDetailCategoryBuilder& DebugCategoryBuilder = DetailLayout.EditCategory("Debug");
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailLayout.GetObjectsBeingCustomized(Objects);

		const FText RenderCaptureText = LOCTEXT("RenderCaptureText", "Render Capture");
		const FText RenderCaptureTooltip = LOCTEXT("RenderCaptureTooltip", "Triggers a render capture and opens it in the enabled render capture software (e.g. Render Doc, if RenderDocPlugin is enabled)");
		DebugCategoryBuilder.AddCustomRow(RenderCaptureText).WholeRowContent()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ToolTipText(RenderCaptureTooltip)
					.IsEnabled_Lambda([Objects]() -> bool
					{
						for (TWeakObjectPtr<UObject> Object : Objects)
						{
							if (Object.IsValid() &&
								(Object->IsA<USceneCaptureComponent2D>() || Object->IsA<USceneCaptureComponentCube>()))
							{
								return true;
							}
						}

						return false;
					})
					.OnClicked_Lambda([Objects]() -> FReply
					{
						for (TWeakObjectPtr<UObject> Object : Objects)
						{
							if (USceneCaptureComponent* SceneCaptureComponent = Cast<USceneCaptureComponent>(Object.Get()))
							{
								RenderCaptureInterface::FScopedCapture RenderCapture(/*bEnable = */true, *FString::Format(TEXT("Scene Capture : {0}"), { SceneCaptureComponent->GetOwner()->GetActorLabel() }));
								if (USceneCaptureComponent2D* SceneCaptureComponent2D = Cast<USceneCaptureComponent2D>(SceneCaptureComponent))
								{
									SceneCaptureComponent2D->CaptureScene();
								}
								else if (USceneCaptureComponentCube* SceneCaptureComponentCube = Cast<USceneCaptureComponentCube>(SceneCaptureComponent))
								{
									SceneCaptureComponentCube->CaptureScene();
								}
							}
						}
				
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(RenderCaptureText)
					]
				]
			];
	}
}

ECheckBoxState FSceneCaptureDetails::OnGetDisplayCheckState(FString ShowFlagName) const
{
	TArray<const void*> RawData;
	ShowFlagSettingsProperty->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	ShowFlagSettingsProperty->GetOuterObjects(OuterObjects);

	ECheckBoxState ReturnState = ECheckBoxState::Unchecked;
	bool bReturnStateSet = false;
	for (int32 ObjectIdx = 0; ObjectIdx < RawData.Num(); ++ObjectIdx)
	{
		const void* Data = RawData[ObjectIdx];

		// Data can be nullptr when the SceneCapture is removed while its details are visible
		if (!Data)
		{
			return ECheckBoxState::Unchecked;
		}

		const TArray<FEngineShowFlagsSetting>& ShowFlagSettings = *reinterpret_cast<const TArray<FEngineShowFlagsSetting>*>(Data);
		const FEngineShowFlagsSetting* Setting = ShowFlagSettings.FindByPredicate([&ShowFlagName](const FEngineShowFlagsSetting& S) { return S.ShowFlagName == ShowFlagName; });
		ECheckBoxState ThisObjectState = ECheckBoxState::Unchecked;
		if (Setting)
		{
			ThisObjectState = Setting->Enabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		else
		{
			const UObject* SceneComp = OuterObjects[ObjectIdx];
			const USceneCaptureComponent* SceneCompArchetype = SceneComp ? Cast<USceneCaptureComponent>(SceneComp->GetArchetype()) : nullptr;
			const int32 SettingIndex = SceneCompArchetype ? SceneCompArchetype->ShowFlags.FindIndexByName(*ShowFlagName) : INDEX_NONE;
			if (SettingIndex != INDEX_NONE)
			{
				ThisObjectState = SceneCompArchetype->ShowFlags.GetSingleFlag(SettingIndex) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
		
		if (bReturnStateSet)
		{
			if (ThisObjectState != ReturnState)
			{
				ReturnState = ECheckBoxState::Undetermined;
				break;
			}
		}
		else
		{
			ReturnState = ThisObjectState;
			bReturnStateSet = true;
		}
	}

	return ReturnState;
}

void FSceneCaptureDetails::OnShowFlagCheckStateChanged(ECheckBoxState InNewRadioState, FString FlagName)
{
	if (InNewRadioState == ECheckBoxState::Undetermined)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ShowFlagCheckStateChanged", "Show Flag Check State Changed"));

	ShowFlagSettingsProperty->NotifyPreChange();

	TArray<void*> RawData;
	ShowFlagSettingsProperty->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	ShowFlagSettingsProperty->GetOuterObjects(OuterObjects);

	const bool bNewEnabledState = (InNewRadioState == ECheckBoxState::Checked) ? true : false;
	for (int32 ObjectIdx = 0; ObjectIdx < RawData.Num(); ++ObjectIdx)
	{
		void* Data = RawData[ObjectIdx];
		check(Data);

		USceneCaptureComponent* SceneComp = Cast<USceneCaptureComponent>(OuterObjects[ObjectIdx]);
		const USceneCaptureComponent* SceneCompArchetype = SceneComp ? Cast<USceneCaptureComponent>(SceneComp->GetArchetype()) : nullptr;
		const bool bDefaultValue = UE::DetailsCustomization::Private::GetShowFlagState(SceneCompArchetype, FlagName);

		TArray<FEngineShowFlagsSetting>& ShowFlagSettings = *reinterpret_cast<TArray<FEngineShowFlagsSetting>*>(Data);
		if (bNewEnabledState == bDefaultValue)
		{
			// Just remove settings that are the same as defaults. This lets the flags return to their default state
			ShowFlagSettings.RemoveAll([&FlagName](const FEngineShowFlagsSetting& Setting) { return Setting.ShowFlagName == FlagName; });
		}
		else
		{
			FEngineShowFlagsSetting* Setting = ShowFlagSettings.FindByPredicate([&FlagName](const FEngineShowFlagsSetting& S) { return S.ShowFlagName == FlagName; });
			if (Setting)
			{
				// If the setting exists already for some reason, update it
				Setting->Enabled = bNewEnabledState;
			}
			else
			{
				// Otherwise create a new setting
				FEngineShowFlagsSetting NewFlagSetting;
				NewFlagSetting.ShowFlagName = FlagName;
				NewFlagSetting.Enabled = bNewEnabledState;
				ShowFlagSettings.Add(NewFlagSetting);
			}
		}

		UE::DetailsCustomization::Private::UpdateArchetypeChainShowFlagSettings(SceneComp, FlagName, bNewEnabledState);
	}

	ShowFlagSettingsProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	ShowFlagSettingsProperty->NotifyFinishedChangingProperties();
}

bool FSceneCaptureDetails::GetShowFlagResetVisibility(FString ShowFlagName) const
{
	TArray<UObject*> OuterObjects;
	ShowFlagSettingsProperty->GetOuterObjects(OuterObjects);

	for (UObject* OuterObject : OuterObjects)
	{
		if (const USceneCaptureComponent* SceneCaptureComponent = Cast<USceneCaptureComponent>(OuterObject))
		{
			if (const USceneCaptureComponent* SceneCompArchetype = Cast<USceneCaptureComponent>(SceneCaptureComponent->GetArchetype()))
			{
				return UE::DetailsCustomization::Private::GetShowFlagState(SceneCaptureComponent, ShowFlagName) != UE::DetailsCustomization::Private::GetShowFlagState(SceneCompArchetype, ShowFlagName);
			}
		}
	}

	return false;
}

void FSceneCaptureDetails::OnResetShowFlag(FString ShowFlagName)
{
	FScopedTransaction Transaction(LOCTEXT("OnResetShowFlag", "On Reset Show Flag"));

	FProperty* ShowFlagsSettingsPropertyVariable = FindFieldChecked<FProperty>(USceneCaptureComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USceneCaptureComponent, ShowFlagSettings));
	FPropertyChangedEvent PropertyChangedEvent(ShowFlagsSettingsPropertyVariable);

	ShowFlagSettingsProperty->NotifyPreChange();

	TArray<void*> RawData;
	ShowFlagSettingsProperty->AccessRawData(RawData);

	TArray<UObject*> OuterObjects;
	ShowFlagSettingsProperty->GetOuterObjects(OuterObjects);

	for (int32 ObjectIdx = 0; ObjectIdx < RawData.Num(); ++ObjectIdx)
	{
		void* Data = RawData[ObjectIdx];
		check(Data);

		if (USceneCaptureComponent* SceneCaptureComponent = Cast<USceneCaptureComponent>(OuterObjects[ObjectIdx]))
		{
			// Reset to archetype default
			TArray<FEngineShowFlagsSetting>& ShowFlagSettings = *reinterpret_cast<TArray<FEngineShowFlagsSetting>*>(Data);
			ShowFlagSettings.RemoveAll([&ShowFlagName](const FEngineShowFlagsSetting& InSetting)
				{
					return InSetting.ShowFlagName.Equals(ShowFlagName);
				});

			const USceneCaptureComponent* SceneCompArchetype = Cast<USceneCaptureComponent>(SceneCaptureComponent->GetArchetype());
			const bool bDefaultValue = UE::DetailsCustomization::Private::GetShowFlagState(SceneCompArchetype, ShowFlagName);

			UE::DetailsCustomization::Private::UpdateArchetypeChainShowFlagSettings(SceneCaptureComponent, ShowFlagName, bDefaultValue);
		}
	}

	ShowFlagSettingsProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	ShowFlagSettingsProperty->NotifyFinishedChangingProperties();
}

#undef LOCTEXT_NAMESPACE
