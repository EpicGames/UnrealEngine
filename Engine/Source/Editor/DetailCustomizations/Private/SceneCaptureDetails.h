// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

enum class ECheckBoxState : uint8;

class FSceneCaptureDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	/** The show flags property on the SceneCaptureComponent */
	TSharedPtr<IPropertyHandle> ShowFlagSettingsProperty;

	/**
	* Gets the display state to send to a display filter check box
	*
	* @param The type of checkbox.
	* @return The desired checkbox state.
	*/
	ECheckBoxState OnGetDisplayCheckState(FString ShowFlagName) const;

	/** Show flag settings changed, so update the scene capture */
	void OnShowFlagCheckStateChanged(ECheckBoxState InNewRadioState, FString FlagName);

	/** Gets the visibility of the reset to default property detail. */
	bool GetShowFlagResetVisibility(FString ShowFlagName) const;

	/** Reset the specified show flag to its (archetype) default. */
	void OnResetShowFlag(FString ShowFlagName);
};
