// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneComponentDetails.h"

#include "AssetSelection.h"
#include "ComponentTransformDetails.h"
#include "ComponentUtils.h"
#include "Components/ActorComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/SceneComponent.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Customizations/MobilityCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/EngineTypes.h"
#include "Engine/InheritableComponentHandler.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "SceneComponentDetails"



/**
 * Walks the scene hierarchy looking for inherited components (like ones from
 * a parent class). If it finds one, this returns its mobility setting. 
 * 
 * @param  SceneComponent	The child component who's ancestor hierarchy you want to traverse.
 * @return The mobility of the first scene-component ancestor (EComponentMobility::Static if one wasn't found).
 */
static EComponentMobility::Type GetInheritedMobility(USceneComponent const* const SceneComponent)
{
	// default to "static" since it doesn't restrict anything (in case we don't inherit any at all)
	EComponentMobility::Type InheritedMobility = EComponentMobility::Static;

	USCS_Node* ComponentNode = ComponentUtils::FindCorrespondingSCSNode(SceneComponent);
	
	if(ComponentNode == NULL)
	{
		return EComponentMobility::Static;
	}

	USimpleConstructionScript const* const SceneSCS = ComponentNode->GetSCS();
	check(SceneSCS != NULL);
	
	do 
	{
		bool const bIsParentInherited = !ComponentNode->ParentComponentOwnerClassName.IsNone();
		// if we can't alter the parent component's mobility from the current blueprint
		if (bIsParentInherited)
		{
			USCS_Node const* ParentNode = NULL;

			UClass* ParentClass = SceneSCS->GetOwnerClass();
			// ParentNode should be null, so we need to find it in the class that owns it...
			// first, find the class:
			while((ParentClass != NULL) && (ParentClass->GetFName() != ComponentNode->ParentComponentOwnerClassName))
			{
				ParentClass = ParentClass->GetSuperClass();
			}
			// now look through this blueprint and find the inherited parent node
			if (UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(ParentClass))
			{
				for (USCS_Node const* Node : BlueprintClass->SimpleConstructionScript->GetAllNodes())
				{
					check(Node); // fix for this bug: https://connect.microsoft.com/VisualStudio/feedback/details/3081898
					if (Node->GetVariableName() == ComponentNode->ParentComponentOrVariableName)
					{
						ParentNode = Node;
						break;
					}
				}
			}

			if (ParentNode != NULL)
			{
				if (USceneComponent const* const ParentComponent = Cast<USceneComponent>(ParentNode->ComponentTemplate))
				{
					InheritedMobility = ParentComponent->Mobility;
					break;
				}
			}
		}
		ComponentNode = SceneSCS->FindParentNode(ComponentNode);

	} while (ComponentNode != NULL);

	return InheritedMobility;
}


/**
 * Checks to see if the specified mobility is valid for the passed USceneComponent.
 * 
 * @param  MobilityValue		The mobility you wish to validate.
 * @param  SceneComponent		The component you want to check for.
 * @param  ProhibitedReasonOut	If the mobility is invalid, this will explain why.
 * @return False if the mobility in question is valid, true if it is invalid.
 */
static bool IsMobilitySettingProhibited(EComponentMobility::Type const MobilityValue, USceneComponent const* const SceneComponent, FText& ProhibitedReasonOut)
{
	bool bIsProhibited = false;

	switch (MobilityValue)
	{
	case EComponentMobility::Movable:
		{
			// movable is always an option (parent components can't prevent this from being movable)
			bIsProhibited = false;
			break;
		}

	case EComponentMobility::Stationary:
		{
			if (GetInheritedMobility(SceneComponent) == EComponentMobility::Movable)
			{
				ProhibitedReasonOut = LOCTEXT("ParentMoreMobileRestriction", "Selected objects cannot be less mobile than their inherited parents.");
				// can't be less movable than what we've inherited
				bIsProhibited = true;
			}
			break;
		}

	case EComponentMobility::Static:
		{
			if (GetInheritedMobility(SceneComponent) != EComponentMobility::Static)
			{
				ProhibitedReasonOut = LOCTEXT("ParentMoreMobileRestriction", "Selected objects cannot be less mobile than their inherited parents.");
				// can't be less movable than what we've inherited
				bIsProhibited = true;
			}
		}
	};	

	return bIsProhibited;
};

TSharedRef<IDetailCustomization> FSceneComponentDetails::MakeInstance()
{
	return MakeShareable( new FSceneComponentDetails );
}

void FSceneComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	MakeTransformDetails( DetailBuilder );

	// Put mobility property in Transform section
	IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory( "TransformCommon", LOCTEXT("TransformCommonCategory", "Transform"), ECategoryPriority::Transform );
	TSharedRef<IPropertyHandle> MobilityHandle = DetailBuilder.GetProperty("Mobility");

	if(MobilityHandle->IsValidHandle())
	{
		uint8 RestrictedMobilityBits = 0u;
		bool bAnySelectedIsLight = false;

		TArray< TWeakObjectPtr<UObject> > SelectedSceneComponents;

		// see if any of the selected objects have mobility restrictions
		DetailBuilder.GetObjectsBeingCustomized(SelectedSceneComponents);

		for (TArray<TWeakObjectPtr<UObject>>::TConstIterator ObjectIt(SelectedSceneComponents); ObjectIt; ++ObjectIt)
		{
			if (!ObjectIt->IsValid())
			{
				continue;
			}

			USceneComponent const* SceneComponent = Cast<USceneComponent>((*ObjectIt).Get());
			if (SceneComponent == NULL)
			{
				continue;
			}

			if (SceneComponent->IsA(ULightComponentBase::StaticClass()))
			{
				bAnySelectedIsLight = true;
			}

			// if we haven't restricted the "Static" option yet
			if (!(RestrictedMobilityBits & FMobilityCustomization::StaticMobilityBitMask))
			{
				FText RestrictReason;
				if (IsMobilitySettingProhibited(EComponentMobility::Static, SceneComponent, RestrictReason))
				{
					TSharedPtr<FPropertyRestriction> StaticRestriction = MakeShareable(new FPropertyRestriction(MoveTemp(RestrictReason)));
					const UEnum* const ComponentMobilityEnum = StaticEnum<EComponentMobility::Type>();
					StaticRestriction->AddDisabledValue(ComponentMobilityEnum->GetNameStringByValue((uint8)EComponentMobility::Static));
					MobilityHandle->AddRestriction(StaticRestriction.ToSharedRef());

					RestrictedMobilityBits |= FMobilityCustomization::StaticMobilityBitMask;
				}
			}

			// if we haven't restricted the "Stationary" option yet
			if (!(RestrictedMobilityBits & FMobilityCustomization::StationaryMobilityBitMask))
			{
				FText RestrictReason;
				if (IsMobilitySettingProhibited(EComponentMobility::Stationary, SceneComponent, RestrictReason))
				{
					TSharedPtr<FPropertyRestriction> StationaryRestriction = MakeShareable(new FPropertyRestriction(MoveTemp(RestrictReason)));
					const UEnum* const ComponentMobilityEnum = StaticEnum<EComponentMobility::Type>();
					StationaryRestriction->AddDisabledValue(ComponentMobilityEnum->GetNameStringByValue((uint8)EComponentMobility::Stationary));
					MobilityHandle->AddRestriction(StationaryRestriction.ToSharedRef());

					RestrictedMobilityBits |= FMobilityCustomization::StationaryMobilityBitMask;
				}
			}

			// no need to go through all of them if we can't restrict any more
			if ((RestrictedMobilityBits & FMobilityCustomization::StaticMobilityBitMask) && (RestrictedMobilityBits & FMobilityCustomization::StationaryMobilityBitMask))
			{
				break;
			}
		}

		TransformCategory.AddCustomBuilder(MakeShared<FMobilityCustomization>(MobilityHandle, RestrictedMobilityBits, bAnySelectedIsLight));
	}

	// Only display bHiddenInGame if the property is being flattened in to an Actor.
	// Details panel for BP component will have the base class be the Actor due to how the SKismetInspector works, but in that case we
	// have a class default object selected, so use that to infer that this is the component directly selected and since BPs do not do
	// property flattening it all kind of works
	if (DetailBuilder.GetBaseClass()->IsChildOf<AActor>())
	{
		if (TSharedPtr<IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr())
		{
			if (!DetailsView->HasClassDefaultObject())
			{
				TSharedPtr<IPropertyHandle> ComponentHiddenInGameProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneComponent, bHiddenInGame));
				ComponentHiddenInGameProperty->MarkHiddenByCustomization();
			}
		}
	}
}

void FSceneComponentDetails::MakeTransformDetails( IDetailLayoutBuilder& DetailBuilder )
{
	int SelectedActorsNum = 0;
	const FSelectedActorInfo* SelectedActorInfoPtr = nullptr;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView)
	{
		SelectedActorsNum = DetailsView->GetSelectedActors().Num();
		SelectedActorInfoPtr = &DetailsView->GetSelectedActorInfo();
	}

	// Hide the transform properties so they don't show up
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetAbsoluteLocationPropertyName()));
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetAbsoluteRotationPropertyName()));
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetAbsoluteScalePropertyName()));
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetRelativeLocationPropertyName()));
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetRelativeRotationPropertyName()));
	DetailBuilder.HideProperty(DetailBuilder.GetProperty(USceneComponent::GetRelativeScale3DPropertyName()));

	// Determine whether or not we are editing Class Defaults through the CDO
	bool bIsEditingBlueprintDefaults = false;
	const TArray<TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
	for(int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		UObject* SelectedObject = SelectedObjects[ObjectIndex].Get();
		if(SelectedObject && SelectedObject->HasAnyFlags(RF_ClassDefaultObject))
		{
			bIsEditingBlueprintDefaults = !!UBlueprint::GetBlueprintFromClass(SelectedObject->GetClass());
			if(!bIsEditingBlueprintDefaults)
			{
				break;
			}
		}
	}

	// If there are any actors selected and we're not editing Class Defaults, the transform section is shown as part of the actor customization
	if(SelectedActorsNum == 0 || bIsEditingBlueprintDefaults)
	{
		TArray< TWeakObjectPtr<UObject> > SceneComponentObjects;
		DetailBuilder.GetObjectsBeingCustomized( SceneComponentObjects );

		// Default to showing the transform for all components unless we are viewing a non-Blueprint class default object (the transform is not used in that case)
		bool bShouldShowTransform = true;
		if (DetailsView)
		{
			bShouldShowTransform  = !DetailsView->HasClassDefaultObject() || bIsEditingBlueprintDefaults;
		}

		for (int32 ComponentIndex = 0; bShouldShowTransform && ComponentIndex < SceneComponentObjects.Num(); ++ComponentIndex)
		{
			USceneComponent* SceneComponent = Cast<USceneComponent>(SceneComponentObjects[ComponentIndex].Get());
			if (SceneComponent == nullptr)
			{
				continue;
			}

			if (SceneComponent->GetAttachParent() == NULL && SceneComponent->GetOuter()->HasAnyFlags(RF_ClassDefaultObject))
			{
				bShouldShowTransform = false;
			}
			else if (const USimpleConstructionScript* SCS = ComponentUtils::GetSimpleConstructionScript(SceneComponent))
			{
				const TArray<USCS_Node*>& RootNodes = SCS->GetRootNodes();
				for(int32 RootNodeIndex = 0; bShouldShowTransform && RootNodeIndex < RootNodes.Num(); ++RootNodeIndex)
				{
					USCS_Node* RootNode = RootNodes[RootNodeIndex];
					check(RootNode);

					if(RootNode->ComponentTemplate == SceneComponent && RootNode->ParentComponentOrVariableName == NAME_None)
					{
						bShouldShowTransform = false;
					}
				}
			}

			if (bShouldShowTransform && SceneComponent->HasAnyFlags(RF_InheritableComponentTemplate))
			{
				UClass* OwnerClass = Cast<UClass>(SceneComponent->GetOuter());
				if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(OwnerClass))
				{
					if (UInheritableComponentHandler* InheritableComponentHandler = Blueprint->GetInheritableComponentHandler(false))
					{
						FComponentKey ComponentKey = InheritableComponentHandler->FindKey(SceneComponent);
						if (ComponentKey.IsValid())
						{
							if (USCS_Node* SCSNode = ComponentKey.FindSCSNode())
							{
								const bool bProperRootNodeFound = SCSNode->IsRootNode() && (SCSNode->ParentComponentOrVariableName == NAME_None);
								if (bProperRootNodeFound)
								{
									bShouldShowTransform = false;
								}
							}
						}
					}
				}
			}
		}

		static const FSelectedActorInfo EmptyActorInfo;

		const FSelectedActorInfo& SelectedActorInfo = SelectedActorInfoPtr ? *SelectedActorInfoPtr : EmptyActorInfo;
		TSharedRef<FComponentTransformDetails> TransformDetails = MakeShareable( new FComponentTransformDetails( bIsEditingBlueprintDefaults ? SceneComponentObjects : DetailBuilder.GetSelectedObjects(), SelectedActorInfo, DetailBuilder ) );

		if(!bShouldShowTransform)
		{
			TransformDetails->HideTransformField(ETransformField::Location);
			TransformDetails->HideTransformField(ETransformField::Rotation);
		}

		IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory( "TransformCommon", LOCTEXT("TransformCommonCategory", "Transform"), ECategoryPriority::Transform );

		TransformCategory.AddCustomBuilder( TransformDetails );
	}

}
#undef LOCTEXT_NAMESPACE
