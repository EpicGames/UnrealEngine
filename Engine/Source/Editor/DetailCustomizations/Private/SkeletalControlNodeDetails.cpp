// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalControlNodeDetails.h"

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_GetClassDefaults.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_StructOperation.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SkeletalControlNodeDetails"

/////////////////////////////////////////////////////
// FSkeletalControlNodeDetails 

TSharedRef<IDetailCustomization> FSkeletalControlNodeDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalControlNodeDetails());
}

void FSkeletalControlNodeDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	DetailCategory = &DetailBuilder.EditCategory("PinOptions");
	TSharedRef<IPropertyHandle> AvailablePins = DetailBuilder.GetProperty("ShowPinForProperties");
	ArrayProperty = AvailablePins->AsArray();

	bool bShowAvailablePins = true;
	bool bHideInputPins = false;

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() == 1)
	{
		UObject* CurObj = SelectedObjects[0].Get();
		HideUnconnectedPinsNode = ((CurObj && (CurObj->IsA<UK2Node_BreakStruct>() || CurObj->IsA<UK2Node_MakeStruct>() || CurObj->IsA<UK2Node_GetClassDefaults>())) ? static_cast<UK2Node*>(CurObj) : nullptr);
		bHideInputPins = (CurObj && CurObj->IsA<UK2Node_MakeStruct>());
	}
	else
	{
		UStruct* CommonStruct = nullptr;
		for (const TWeakObjectPtr<UObject>& CurrentObject : SelectedObjects)
		{
			UObject* CurObj = CurrentObject.Get();
			if (UK2Node_StructOperation* StructOpNode = Cast<UK2Node_StructOperation>(CurObj))
			{
				if (CommonStruct && CommonStruct != StructOpNode->StructType)
				{
					bShowAvailablePins = false;
					break;
				}
				CommonStruct = StructOpNode->StructType;
			}
			else if (UK2Node_GetClassDefaults* CDONode = Cast<UK2Node_GetClassDefaults>(CurObj))
			{
				if (CommonStruct && CommonStruct != CDONode->GetInputClass())
				{
					bShowAvailablePins = false;
					break;
				}
				CommonStruct = CDONode->GetInputClass();
			}
		}
	}

	if (bShowAvailablePins)
	{
		TSet<FName> UniqueCategoryNames;
		{
			int32 NumElements = 0;
			{
				uint32 UnNumElements = 0;
				if (ArrayProperty.IsValid() && (FPropertyAccess::Success == ArrayProperty->GetNumElements(UnNumElements)))
				{
					NumElements = static_cast<int32>(UnNumElements);
				}
			}
			for (int32 Index = 0; Index < NumElements; ++Index)
			{
				TSharedRef<IPropertyHandle> StructPropHandle = ArrayProperty->GetElement(Index);
				TSharedPtr<IPropertyHandle> CategoryPropHandle = StructPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, CategoryName));
				check(CategoryPropHandle.IsValid());
				FName CategoryNameValue;
				const FPropertyAccess::Result Result = CategoryPropHandle->GetValue(CategoryNameValue);
				if (ensure(FPropertyAccess::Success == Result))
				{
					UniqueCategoryNames.Add(CategoryNameValue);
				}
			}
		}

		//@TODO: Shouldn't show this if the available pins array is empty!
		const bool bGenerateHeader = true;
		const bool bDisplayResetToDefault = false;
		const bool bDisplayElementNum = false;
		const bool bForAdvanced = false;
		for (const FName& CategoryName : UniqueCategoryNames)
		{
			//@TODO: Pay attention to category filtering here
		
		
			TSharedRef<FDetailArrayBuilder> AvailablePinsBuilder = MakeShareable(new FDetailArrayBuilder(AvailablePins, bGenerateHeader, bDisplayResetToDefault, bDisplayElementNum));
			AvailablePinsBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FSkeletalControlNodeDetails::OnGenerateElementForPropertyPin, CategoryName));
			AvailablePinsBuilder->SetDisplayName((CategoryName == NAME_None) ? LOCTEXT("DefaultCategory", "Default Category") : FText::FromName(CategoryName));
			DetailCategory->AddCustomBuilder(AvailablePinsBuilder, bForAdvanced);
		}

		// Add the action buttons
		if(HideUnconnectedPinsNode.IsValid())
		{
			FDetailWidgetRow& GroupActionsRow = DetailCategory->AddCustomRow(LOCTEXT("GroupActionsSearchText", "Split Sort"))
			.ValueContent()
			.HAlign(HAlign_Left)
			.MaxDesiredWidth(250.f)
			[
				SNew(SButton)
				.OnClicked(this, &FSkeletalControlNodeDetails::HideAllUnconnectedPins, bHideInputPins)
				.ToolTipText(LOCTEXT("HideAllUnconnectedPinsTooltip", "All unconnected pins get hidden (removed from the graph node)"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("HideAllUnconnectedPins", "Hide Unconnected Pins"))
					.Font(DetailBuilder.GetDetailFont())
				]
			];
		}
	}
	else
	{
		DetailBuilder.HideProperty(AvailablePins);
	}
}

ECheckBoxState FSkeletalControlNodeDetails::GetShowPinValueForProperty(TSharedRef<IPropertyHandle> InElementProperty) const
{
	FString Value;
	InElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))->GetValueAsFormattedString(Value);

	if (Value == TEXT("true"))
	{
		return ECheckBoxState::Checked;
	}
	else if (Value == TEXT("false"))
	{
		return ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FSkeletalControlNodeDetails::OnShowPinChanged(ECheckBoxState InNewState, TSharedRef<IPropertyHandle> InElementProperty)
{
	InElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))->SetValueFromFormattedString(InNewState == ECheckBoxState::Checked ? TEXT("true") : TEXT("false"));
}

void FSkeletalControlNodeDetails::OnGenerateElementForPropertyPin(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, FName CategoryName)
{
	{
		TSharedPtr<IPropertyHandle> CategoryPropHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, CategoryName));
		check(CategoryPropHandle.IsValid());
		FName CategoryNameValue;
		const FPropertyAccess::Result Result = CategoryPropHandle->GetValue(CategoryNameValue);
		const bool bProperCategory = ensure(FPropertyAccess::Success == Result) && (CategoryNameValue == CategoryName);

		if (!bProperCategory)
		{
			return;
		}
	}

	FString FilterString = CategoryName.ToString();

	FText PropertyFriendlyName(LOCTEXT("Invalid", "Invalid"));
	{
		TSharedPtr<IPropertyHandle> PropertyFriendlyNameHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyFriendlyName));
		if (PropertyFriendlyNameHandle.IsValid())
		{
			FString DisplayFriendlyName;
			switch (PropertyFriendlyNameHandle->GetValue(/*out*/ DisplayFriendlyName))
			{
			case FPropertyAccess::Success:
				FilterString += TEXT(" ") + DisplayFriendlyName;
				PropertyFriendlyName = FText::FromString(DisplayFriendlyName);
				break;
			case FPropertyAccess::MultipleValues:
				ChildrenBuilder.AddCustomRow(FText::GetEmpty())
					[
						SNew(STextBlock).Text(LOCTEXT("OnlyWorksInSingleSelectMode", "Multiple types selected"))
					];
				return;
			case FPropertyAccess::Fail:
			default:
				check(false);
				break;
			}
		}
	}

	{
		TSharedPtr<IPropertyHandle> PropertyNameHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyName));
		if (PropertyNameHandle.IsValid())
		{
			FString RawName;
			if (PropertyNameHandle->GetValue(/*out*/ RawName) == FPropertyAccess::Success)
			{
				FilterString += TEXT(" ") + RawName;
			}
		}
	}

	FText PinTooltip;
	{
		TSharedPtr<IPropertyHandle> PropertyTooltipHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyTooltip));
		if (PropertyTooltipHandle.IsValid())
		{
			FString PinTooltipString;
			if (PropertyTooltipHandle->GetValue(/*out*/ PinTooltip) == FPropertyAccess::Success)
			{
				FilterString += TEXT(" ") + PinTooltip.ToString();
			}
		}
	}

	TSharedPtr<IPropertyHandle> HasOverrideValueHandle = ElementProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bHasOverridePin));
	bool bHasOverrideValue;
	HasOverrideValueHandle->GetValue(/*out*/ bHasOverrideValue);
	FText OverrideCheckBoxTooltip;

	// Setup a tooltip based on whether the property has an override value or not.
	if (bHasOverrideValue)
	{
		OverrideCheckBoxTooltip = LOCTEXT("HasOverridePin", "Enabling this pin will make it visible for setting on the node and automatically enable the value for override when using the struct. Any updates to the resulting struct will require the value be set again or the override will be automatically disabled.");
	}
	else
	{
		OverrideCheckBoxTooltip = LOCTEXT("HasNoOverridePin", "Enabling this pin will make it visible for setting on the node.");
	}

	ChildrenBuilder.AddCustomRow( PropertyFriendlyName )
	.FilterString(FText::AsCultureInvariant(FilterString))
	.NameContent()
	[
		ElementProperty->CreatePropertyNameWidget(PropertyFriendlyName, PinTooltip)
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.ToolTipText(OverrideCheckBoxTooltip)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalControlNodeDetails::GetShowPinValueForProperty, ElementProperty)
			.OnCheckStateChanged(this, &FSkeletalControlNodeDetails::OnShowPinChanged, ElementProperty)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AsPin", " (As pin)"))
			.Font(ChildrenBuilder.GetParentCategory().GetParentLayout().GetDetailFont())
		]
	];
}

FReply FSkeletalControlNodeDetails::HideAllUnconnectedPins(const bool bHideInputPins)
{
	if (ArrayProperty.IsValid() && HideUnconnectedPinsNode.IsValid())
	{
		uint32 NumChildren = 0;
		ArrayProperty->GetNumElements(NumChildren);

		FScopedTransaction Transaction(LOCTEXT("HideUnconnectedPinsTransaction", "Hide Unconnected Pins"));

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ElementHandle = ArrayProperty->GetElement(ChildIndex);
			
			TSharedPtr<IPropertyHandle> PropertyNameHandle = ElementHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, PropertyName));
			FName ActualPropertyName;
			if (PropertyNameHandle.IsValid() && PropertyNameHandle->GetValue(ActualPropertyName) == FPropertyAccess::Success)
			{
				const UEdGraphPin* Pin = HideUnconnectedPinsNode->FindPin(ActualPropertyName.ToString(), bHideInputPins ? EGPD_Input : EGPD_Output);
				if (Pin && Pin->LinkedTo.Num() <= 0)
				{
					OnShowPinChanged(ECheckBoxState::Unchecked, ElementHandle);
				}
			}
		}
	}

	return FReply::Handled();
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
