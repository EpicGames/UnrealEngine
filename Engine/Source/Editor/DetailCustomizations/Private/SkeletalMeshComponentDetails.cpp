// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshComponentDetails.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimationAsset.h"
#include "Animation/Skeleton.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Selection.h"
#include "SingleAnimationPlayData.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Settings/AnimBlueprintSettings.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SkeletalMeshComponentDetails"

// Filter class for animation blueprint picker
class FAnimBlueprintFilter : public IClassViewerFilter
{
public:
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if(InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed)
		{
			return true;
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}

	/** Only children of the classes in this set will be unfiltered */
	TSet<const UClass*> AllowedChildrenOfClasses;
};

FSkeletalMeshComponentDetails::FSkeletalMeshComponentDetails()
	: CurrentDetailBuilder(NULL)
	, Skeleton(nullptr)
	, bAnimPickerEnabled(false)
{

}

FSkeletalMeshComponentDetails::~FSkeletalMeshComponentDetails()
{
	UnregisterAllMeshPropertyChangedCallers();
}

TSharedRef<IDetailCustomization> FSkeletalMeshComponentDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalMeshComponentDetails);
}

void FSkeletalMeshComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if(!CurrentDetailBuilder)
	{
		CurrentDetailBuilder = &DetailBuilder;
	}
	DetailBuilder.EditCategory("SkeletalMesh", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Materials", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.EditCategory("Physics", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	DetailBuilder.HideProperty("bCastStaticShadow", UPrimitiveComponent::StaticClass());
	DetailBuilder.HideProperty("LightmapType", UPrimitiveComponent::StaticClass());
	DetailBuilder.EditCategory("Animation", FText::GetEmpty(), ECategoryPriority::Important);	

	PerformInitialRegistrationOfSkeletalMeshes(DetailBuilder);

	UpdateAnimationCategory(DetailBuilder);
	UpdatePhysicsCategory(DetailBuilder);
}

void FSkeletalMeshComponentDetails::UpdateAnimationCategory(IDetailLayoutBuilder& DetailBuilder)
{
	// Custom skeletal mesh components may hide the animation category, so we won't assume it's visible
	if (DetailBuilder.GetBaseClass() && FEditorCategoryUtils::IsCategoryHiddenFromClass(DetailBuilder.GetBaseClass(), "Animation"))
	{
		return;
	}

	UpdateSkeletonNameAndPickerVisibility();

	IDetailCategoryBuilder& AnimationCategory = DetailBuilder.EditCategory("Animation", FText::GetEmpty(), ECategoryPriority::Important);

	// Force the mode switcher to be first
	AnimationModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, AnimationMode));
	check (AnimationModeHandle->IsValidHandle());

	const FName AnimationBlueprintName = GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, AnimClass);
	AnimationBlueprintHandle = DetailBuilder.GetProperty(AnimationBlueprintName);
	check(AnimationBlueprintHandle->IsValidHandle());

	AnimationCategory.AddProperty(AnimationModeHandle)
		.Visibility({ this, &FSkeletalMeshComponentDetails::VisibilityForAnimModeProperty });

	// Place the blueprint property next (which may be hidden, depending on the mode)
	TAttribute<EVisibility> BlueprintVisibility( this, &FSkeletalMeshComponentDetails::VisibilityForBlueprintMode );

	AnimationCategory.AddProperty(AnimationBlueprintHandle)
		.Visibility(BlueprintVisibility)
		.CustomWidget()
		.NameContent()
		[
			AnimationBlueprintHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.f)
		.MaxDesiredWidth(250.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ClassPickerComboButton, SComboButton)
				.OnGetMenuContent(this, &FSkeletalMeshComponentDetails::GetClassPickerMenuContent)
				.ContentPadding(0.f)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &FSkeletalMeshComponentDetails::GetSelectedAnimBlueprintName)
					.MinDesiredWidth(200.f)
				]
			]
		];

	// Hide the parent AnimationData property, and inline the children with custom visibility delegates
	const FName AnimationDataFName(GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, AnimationData));

	TSharedPtr<IPropertyHandle> AnimationDataHandle = DetailBuilder.GetProperty(AnimationDataFName);
	check(AnimationDataHandle->IsValidHandle());
	TAttribute<EVisibility> SingleAnimVisibility(this, &FSkeletalMeshComponentDetails::VisibilityForSingleAnimMode);
	DetailBuilder.HideProperty(AnimationDataFName);

	// Process Animation asset selection
	uint32 TotalChildren=0;
	AnimationDataHandle->GetNumChildren(TotalChildren);
	for (uint32 ChildIndex=0; ChildIndex < TotalChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = AnimationDataHandle->GetChildHandle(ChildIndex);
	
		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSingleAnimationPlayData, AnimToPlay))
		{
			// Hide the property, as we're about to add it differently
			DetailBuilder.HideProperty(ChildHandle);

			// Add it differently
			TSharedPtr<SWidget> NameWidget = ChildHandle->CreatePropertyNameWidget();

			TSharedRef<SWidget> PropWidget = SNew(SObjectPropertyEntryBox)
				.ThumbnailPool(DetailBuilder.GetThumbnailPool())
				.PropertyHandle(ChildHandle)
				.AllowedClass(UAnimationAsset::StaticClass())
				.AllowClear(true)
				.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(this, &FSkeletalMeshComponentDetails::OnShouldFilterAnimAsset));

			TAttribute<bool> AnimPickerEnabledAttr(this, &FSkeletalMeshComponentDetails::AnimPickerIsEnabled);

			AnimationCategory.AddProperty(ChildHandle)
				.Visibility(SingleAnimVisibility)
				.CustomWidget()
				.IsEnabled(AnimPickerEnabledAttr)
				.NameContent()
				[
					NameWidget.ToSharedRef()
				]
				.ValueContent()
				.MinDesiredWidth(600.f)
				.MaxDesiredWidth(600.f)
				[
					PropWidget
				]
				.PropertyHandleList({ ChildHandle });
		}
		else
		{
			AnimationCategory.AddProperty(ChildHandle).Visibility(SingleAnimVisibility);
		}
	}
}

EVisibility FSkeletalMeshComponentDetails::VisibilityForAnimModeProperty() const
{
	return GetDefault<UAnimBlueprintSettings>()->bAllowAnimBlueprints ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FSkeletalMeshComponentDetails::VisibilityForBlueprintMode() const
{
	if (!GetDefault<UAnimBlueprintSettings>()->bAllowAnimBlueprints)
	{
		return EVisibility::Hidden;
	}
	return VisibilityForAnimationMode(EAnimationMode::AnimationBlueprint);
}

void FSkeletalMeshComponentDetails::UpdatePhysicsCategory(IDetailLayoutBuilder& DetailBuilder)
{
	// Force hiding ClothTeleportMode. If you stop force hiding this, please update the comment in SkeletalMeshComponent.h where the property is declared.
	const FName ClothTeleportModeFName(GET_MEMBER_NAME_CHECKED(USkeletalMeshComponent, ClothTeleportMode));
	DetailBuilder.HideProperty(ClothTeleportModeFName);
}

EVisibility FSkeletalMeshComponentDetails::VisibilityForAnimationMode(EAnimationMode::Type AnimationMode) const
{
	uint8 AnimationModeValue=0;
	FPropertyAccess::Result Ret = AnimationModeHandle.Get()->GetValue(AnimationModeValue);
	if (Ret == FPropertyAccess::Result::Success)
	{
		return (AnimationModeValue == AnimationMode) ? EVisibility::Visible : EVisibility::Hidden;
	}

	return EVisibility::Hidden; //Hidden if we get fail or MultipleValues from the property
}

bool FSkeletalMeshComponentDetails::OnShouldFilterAnimAsset( const FAssetData& AssetData )
{
	// Check the compatible skeletons.
	if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
	{
		return false;
	}

	return true;
}

void FSkeletalMeshComponentDetails::SkeletalMeshPropertyChanged()
{
	UpdateSkeletonNameAndPickerVisibility();
}

void FSkeletalMeshComponentDetails::UpdateSkeletonNameAndPickerVisibility()
{
	// Update the selected skeleton name and the picker visibility
	Skeleton = GetValidSkeletonFromRegisteredMeshes();

	if (Skeleton)
	{
		bAnimPickerEnabled = true;
		SelectedSkeletonName = FObjectPropertyBase::GetExportPath(Skeleton);
	}
	else
	{
		bAnimPickerEnabled = false;
		SelectedSkeletonName = "";
	}
}

void FSkeletalMeshComponentDetails::RegisterSkeletalMeshPropertyChanged(TWeakObjectPtr<USkeletalMeshComponent> Mesh)
{
	if(Mesh.IsValid() && OnSkeletalMeshPropertyChanged.IsBound())
	{
		OnSkeletalMeshPropertyChangedDelegateHandles.Add(Mesh.Get(), Mesh->RegisterOnSkeletalMeshPropertyChanged(OnSkeletalMeshPropertyChanged));
	}
}

void FSkeletalMeshComponentDetails::UnregisterSkeletalMeshPropertyChanged(TWeakObjectPtr<USkeletalMeshComponent> Mesh)
{
	if(Mesh.IsValid())
	{
		Mesh->UnregisterOnSkeletalMeshPropertyChanged(OnSkeletalMeshPropertyChangedDelegateHandles.FindRef(Mesh.Get()));
		OnSkeletalMeshPropertyChangedDelegateHandles.Remove(Mesh.Get());
	}
}

void FSkeletalMeshComponentDetails::UnregisterAllMeshPropertyChangedCallers()
{
	for(auto MeshIter = SelectedObjects.CreateIterator() ; MeshIter ; ++MeshIter)
	{
		if(USkeletalMeshComponent* Mesh = Cast<USkeletalMeshComponent>(MeshIter->Get()))
		{
			Mesh->UnregisterOnSkeletalMeshPropertyChanged(OnSkeletalMeshPropertyChangedDelegateHandles.FindRef(Mesh));
			OnSkeletalMeshPropertyChangedDelegateHandles.Remove(Mesh);
		}
	}
}

bool FSkeletalMeshComponentDetails::AnimPickerIsEnabled() const
{
	return bAnimPickerEnabled;
}

TSharedRef<SWidget> FSkeletalMeshComponentDetails::GetClassPickerMenuContent()
{
	TSharedPtr<FAnimBlueprintFilter> Filter = MakeShareable(new FAnimBlueprintFilter);
	Filter->AllowedChildrenOfClasses.Add(UAnimInstance::StaticClass());

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	FClassViewerInitializationOptions InitOptions;
	InitOptions.Mode = EClassViewerMode::ClassPicker;
	InitOptions.ClassFilters.Add(Filter.ToSharedRef());
	InitOptions.bShowNoneOption = true;

	return SNew(SBorder)
		.Padding(3)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.ForegroundColor(FAppStyle::GetColor("DefaultForeground"))
		[
			SNew(SBox)
			.WidthOverride(280.f)
			[
				ClassViewerModule.CreateClassViewer(InitOptions, FOnClassPicked::CreateSP(this, &FSkeletalMeshComponentDetails::OnClassPicked))
			]
		];
}

FText FSkeletalMeshComponentDetails::GetSelectedAnimBlueprintName() const
{
	check(AnimationBlueprintHandle->IsValidHandle());

	UObject* Object = NULL;
	AnimationBlueprintHandle->GetValue(Object);
	if(Object)
	{
		return FText::FromString(Object->GetName());
	}
	else
	{
		return LOCTEXT("None", "None");
	}
}

void FSkeletalMeshComponentDetails::OnClassPicked( UClass* PickedClass )
{
	check(AnimationBlueprintHandle->IsValidHandle());

	ClassPickerComboButton->SetIsOpen(false);

	AnimationBlueprintHandle->SetValue(PickedClass);
}

void FSkeletalMeshComponentDetails::OnBrowseToAnimBlueprint()
{
	check(AnimationBlueprintHandle->IsValidHandle());

	UObject* Object = NULL;
	AnimationBlueprintHandle->GetValue(Object);

	TArray<UObject*> Objects;
	Objects.Add(Object);
	GEditor->SyncBrowserToObjects(Objects);
}

void FSkeletalMeshComponentDetails::UseSelectedAnimBlueprint()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	USelection* AssetSelection = GEditor->GetSelectedObjects();
	if (AssetSelection && AssetSelection->Num() == 1)
	{
		UAnimBlueprint* AnimBlueprintToAssign = AssetSelection->GetTop<UAnimBlueprint>();
		if (AnimBlueprintToAssign)
		{
			if(USkeleton* AnimBlueprintSkeleton = AnimBlueprintToAssign->TargetSkeleton)
			{
				if (Skeleton && Skeleton->IsCompatibleForEditor(AnimBlueprintSkeleton))
				{
					OnClassPicked(AnimBlueprintToAssign->GetAnimBlueprintGeneratedClass());
				}
			}
		}
	}
}

void FSkeletalMeshComponentDetails::PerformInitialRegistrationOfSkeletalMeshes(IDetailLayoutBuilder& DetailBuilder)
{
	OnSkeletalMeshPropertyChanged = USkeletalMeshComponent::FOnSkeletalMeshPropertyChanged::CreateSP(this, &FSkeletalMeshComponentDetails::SkeletalMeshPropertyChanged);

	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);

	check(SelectedObjects.Num() > 0);

	for (auto ObjectIter = SelectedObjects.CreateIterator(); ObjectIter; ++ObjectIter)
	{
		if (USkeletalMeshComponent* Mesh = Cast<USkeletalMeshComponent>(ObjectIter->Get()))
		{
			RegisterSkeletalMeshPropertyChanged(Mesh);
		}
	}
}

USkeleton* FSkeletalMeshComponentDetails::GetValidSkeletonFromRegisteredMeshes() const
{
	USkeleton* ResultSkeleton = NULL;

	for (auto ObjectIter = SelectedObjects.CreateConstIterator(); ObjectIter; ++ObjectIter)
	{
		USkeletalMeshComponent* const Mesh = Cast<USkeletalMeshComponent>(ObjectIter->Get());
		if ( !Mesh || !Mesh->GetSkeletalMeshAsset())
		{
			continue;
		}

		// If we've not come across a valid skeleton yet, store this one.
		if (!ResultSkeleton)
		{
			ResultSkeleton = Mesh->GetSkeletalMeshAsset()->GetSkeleton();
			continue;
		}

		// We've encountered a valid skeleton before.
		// If this skeleton is not the same one, that means there are multiple
		// skeletons selected, so we don't want to take any action.
		if (Mesh->GetSkeletalMeshAsset()->GetSkeleton() != ResultSkeleton)
		{
			return NULL;
		}
	}

	return ResultSkeleton;
}

#undef LOCTEXT_NAMESPACE
