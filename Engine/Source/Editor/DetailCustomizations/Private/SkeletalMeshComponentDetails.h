// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDelegateHandle;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SComboButton;
class SWidget;
class UClass;
class UObject;
class USkeleton;
struct FAssetData;

class FSkeletalMeshComponentDetails : public IDetailCustomization
{
public:
	UE_API FSkeletalMeshComponentDetails();
	UE_API ~FSkeletalMeshComponentDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	static UE_API TSharedRef<SWidget> CreateAsyncSceneValueWidgetWithWarning(const TSharedPtr<IPropertyHandle>& AsyncScenePropertyHandle);

private:
	UE_API void UpdateAnimationCategory(IDetailLayoutBuilder& DetailBuilder);
	UE_API void UpdatePhysicsCategory(IDetailLayoutBuilder& DetailBuilder);

	/** Function that returns whether the specified animation mode should be visible */
	UE_API EVisibility VisibilityForAnimationMode(EAnimationMode::Type AnimationMode) const;

	/** Helper wrapper functions for VisibilityForAnimationMode */
	UE_API EVisibility VisibilityForBlueprintMode() const;
	EVisibility VisibilityForSingleAnimMode() const { return VisibilityForAnimationMode(EAnimationMode::AnimationSingleNode); }
	UE_API bool AnimPickerIsEnabled() const;

	UE_API EVisibility VisibilityForAnimModeProperty() const;

	/** Handler for filtering animation assets in the UI picker when asset mode is selected */
	UE_API bool OnShouldFilterAnimAsset(const FAssetData& AssetData);

	/** Delegate called when a skeletal mesh property is changed on a selected object */
	USkeletalMeshComponent::FOnSkeletalMeshPropertyChanged OnSkeletalMeshPropertyChanged;
	
	/** Register/Unregister the mesh changed delegate to TargetComponent */
	UE_API void PerformInitialRegistrationOfSkeletalMeshes(IDetailLayoutBuilder& DetailBuilder);
	UE_API void RegisterSkeletalMeshPropertyChanged(TWeakObjectPtr<USkeletalMeshComponent> Mesh);
	UE_API void UnregisterSkeletalMeshPropertyChanged(TWeakObjectPtr<USkeletalMeshComponent> Mesh);
	UE_API void UnregisterAllMeshPropertyChangedCallers();

	/**
	* Iterates over registered meshes and returns a pointer to the common skeleton used by all of them.
	* If the meshes use more than one different skeleton, NULL is returned.
	*/
	UE_API USkeleton* GetValidSkeletonFromRegisteredMeshes() const;

	/** Bound to the delegate used to detect changes in skeletal mesh properties */
	UE_API void SkeletalMeshPropertyChanged();

	/** Generates menu content for the class picker when it is clicked */
	UE_API TSharedRef<SWidget> GetClassPickerMenuContent();

	/** Gets the currently selected blueprint name to display on the class picker combo button */
	UE_API FText GetSelectedAnimBlueprintName() const;

	/** Callback from the class picker when the user selects a class */
	UE_API void OnClassPicked(UClass* PickedClass);

	/** Callback from the detail panel to browse to the selected anim asset */
	UE_API void OnBrowseToAnimBlueprint();

	/** Callback from the details panel to use the currently selected asset in the content browser */
	UE_API void UseSelectedAnimBlueprint();

	/** Called when a skeletal mesh property changes. */
	UE_API void UpdateSkeletonNameAndPickerVisibility();

	/** Cached layout builder for use after customization */
	IDetailLayoutBuilder* CurrentDetailBuilder;

	/** Cached selected objects to use when the skeletal mesh property changes */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	/** Cache of mesh components in the current selection */
	TArray<TWeakObjectPtr<USkeletalMeshComponent>> SelectedSkeletalMeshComponents;

	/** Caches the AnimationMode Handle so we can look up its value after customization has finished */
	TSharedPtr<IPropertyHandle> AnimationModeHandle;

	/** Caches the AnimationBlueprintGeneratedClass Handle so we can look up its value after customization has finished */
	TSharedPtr<IPropertyHandle> AnimationBlueprintHandle;

	/** Caches the AsyncScene handle so we can look up its value after customization has finished. */
	TSharedPtr<IPropertyHandle> AsyncSceneHandle;

	/** Full name of the currently selected skeleton to use for filtering animation assets */
	FString SelectedSkeletonName;

	/** The skeleton that we grab the name from for filtering. */
	USkeleton* Skeleton;

	/** Current enabled state of the animation asset picker in the details panel */
	bool bAnimPickerEnabled;

	/** The combo button for the class picker, Cached so we can close it when the user picks something */
	TSharedPtr<SComboButton> ClassPickerComboButton;

	/** Per-mesh handles to registered OnSkeletalMeshPropertyChanged delegates */
	TMap<USkeletalMeshComponent*, FDelegateHandle> OnSkeletalMeshPropertyChangedDelegateHandles;
};

#undef UE_API
