// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshDetails.h"

#include "Containers/Array.h"
#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformMath.h"
#include "IMeshReductionManagerModule.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshDetails"

void FSkeletalMeshDetails::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	static const auto AllowSkinnedMeshes = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Nanite.AllowSkinnedMeshes"));
	static const bool bAllowSkinnedMeshes = (AllowSkinnedMeshes && AllowSkinnedMeshes->GetValueOnAnyThread() != 0);
	if (!bAllowSkinnedMeshes)
	{
		TSharedRef<IPropertyHandle> SettingsHandle = LayoutBuilder.GetProperty(FName("NaniteSettings"));
		LayoutBuilder.HideProperty(SettingsHandle);
	}
}

TSharedRef<IDetailCustomization> FSkeletalMeshDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalMeshDetails);
}
#undef LOCTEXT_NAMESPACE