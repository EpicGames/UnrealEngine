// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshLODSettingsDetails.h"

#include "Containers/Array.h"
#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "HAL/PlatformMath.h"
#include "IMeshReductionManagerModule.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshLODSettingsDetails"

void FSkeletalMeshLODSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder)
{
	// hide LOD 0's reduction settings since it won't do anything
	TSharedRef<IPropertyHandle> SettingsHandle= LayoutBuilder.GetProperty(FName("LODGroups"));

	uint32 SettingChildHandleNum = 0;

	ensure(SettingsHandle->GetNumChildren(SettingChildHandleNum) != FPropertyAccess::Fail);

	// see if auto mesh reduction is available
	static bool bAutoMeshReductionAvailable = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetSkeletalMeshReductionInterface() != NULL;
	
	for (uint32 Index = 0; Index < SettingChildHandleNum; ++Index)
	{
		TSharedPtr<IPropertyHandle> LODChildHandle = SettingsHandle->GetChildHandle(Index);

		// we want to hide index 0 element
		if (LODChildHandle->IsValidHandle())
		{
			if (LODChildHandle->GetIndexInArray() == 0)
			{
				static const TArray<FName> HiddenProperties = { GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODGroupSettings, BoneFilterActionOption),
					GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODGroupSettings, BoneList) };
				for (int32 HiddenPropIndex = 0; HiddenPropIndex < HiddenProperties.Num(); ++HiddenPropIndex)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = LODChildHandle->GetChildHandle(HiddenProperties[HiddenPropIndex]);
					LayoutBuilder.HideProperty(ChildHandle);
				}
			}
			if(!bAutoMeshReductionAvailable)
			{
				static const TArray<FName> HiddenProperties = { GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODGroupSettings, ReductionSettings) };
				for (int32 HiddenPropIndex = 0; HiddenPropIndex < HiddenProperties.Num(); ++HiddenPropIndex)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = LODChildHandle->GetChildHandle(HiddenProperties[HiddenPropIndex]);
					LayoutBuilder.HideProperty(ChildHandle);
				}
			}
		}
	}
}

TSharedRef<IDetailCustomization> FSkeletalMeshLODSettingsDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalMeshLODSettingsDetails);
}
#undef LOCTEXT_NAMESPACE
