// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

class FSkeletalMeshLODSettingsDetails : public IDetailCustomization
{
public:
	/** Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& LayoutBuilder) override;
	/** End IDetailCustomization interface */

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

};
