// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshReductionSettingsDetails.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Visibility.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SkeletalMeshReductionSettings.h"
#include "SkeletalRenderPublic.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "SkeletalMeshReductionSettingsDetails"

TSharedRef<IPropertyTypeCustomization> FSkeletalMeshReductionSettingsDetails::MakeInstance()
{
	return MakeShareable(new FSkeletalMeshReductionSettingsDetails);
}

void FSkeletalMeshReductionSettingsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

bool FSkeletalMeshReductionSettingsDetails::UseNativeReductionTool() const
{
	if (IMeshReduction* SkeletalReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetSkeletalMeshReductionInterface())
	{
		FString ModuleVersionString = SkeletalReductionModule->GetVersionString();

		TArray<FString> SplitVersionString;
		ModuleVersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);
		return SplitVersionString[0].Equals("QuadricSkeletalMeshReduction");
	}

	return false;
}

void FSkeletalMeshReductionSettingsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// here, we have to keep track of customizing properties, so that we don't display twice
	const TArray<FName> CustomizedProperties = {
		GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, ReductionMethod),
		GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, NumOfTrianglesPercentage),
		GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxDeviationPercentage)
	};

	USkeletalMesh* SkeletalMesh = nullptr;
	TArray<UObject*> Objects;
	StructPropertyHandle->GetOuterObjects(Objects);
	if (Objects.Num() > 0)
	{
		SkeletalMesh = Cast<USkeletalMesh>(Objects[0]);
	}

	ReductionMethodPropertyHandle = StructPropertyHandle->GetChildHandle(CustomizedProperties[0]);
	NumTrianglesPercentagePropertyHandle = StructPropertyHandle->GetChildHandle(CustomizedProperties[1]);
	MaxDeviationPercentagePropertyHandle = StructPropertyHandle->GetChildHandle(CustomizedProperties[2]);

	bool bUseThirdPartyUI = !UseNativeReductionTool();

	const int32 LODIndex = [StructPropertyHandle]() -> int32
	{
		if (StructPropertyHandle->GetParentHandle().IsValid())
		{
			if (StructPropertyHandle->GetParentHandle()->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkeletalMeshObject, LODInfo) ||
				StructPropertyHandle->GetParentHandle()->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(USkeletalMeshLODSettings, LODGroups))
			{
				return StructPropertyHandle->GetParentHandle()->GetIndexInArray();
			}
		}

		return INDEX_NONE;
	}();

	auto BaseLODCustomization = [LODIndex, &StructBuilder, &SkeletalMesh](TSharedPtr<IPropertyHandle>& BaseLODPropertyHandle)
	{
		// Only able to do this for LOD1 and above, so only show the property if this is the case
		if (LODIndex > 0)
		{
			bool AllowInline = SkeletalMesh && SkeletalMesh->HasMeshDescription(LODIndex);
			// Add and retrieve the default widgets
			IDetailPropertyRow& Row = StructBuilder.AddProperty(BaseLODPropertyHandle->AsShared());

			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			FDetailWidgetRow DefaultWidgetRow;
			Row.GetDefaultWidgets(NameWidget, ValueWidget, DefaultWidgetRow);

			// Customize the value property to be a spinbox with Max value cap so it is always < CurrentLODIndex
			Row.CustomWidget()
				.NameContent()
				[
					NameWidget.ToSharedRef()
				]
				.ValueContent()
				.MinDesiredWidth(DefaultWidgetRow.ValueWidget.MinWidth)
				.MaxDesiredWidth(DefaultWidgetRow.ValueWidget.MaxWidth)
				[
					SNew(SSpinBox<int32>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.MinValue(0)
					.MaxValue(FMath::Max(LODIndex - (AllowInline ? 0 : 1), 0))
					.Value_Lambda([BaseLODPropertyHandle]() -> int32
					{
						int32 Value = INDEX_NONE;
						BaseLODPropertyHandle->GetValue(Value);
						return Value;
					})					
					.OnValueChanged_Lambda([BaseLODPropertyHandle](int32 NewValue)
					{
						BaseLODPropertyHandle->SetValue(NewValue);
					})
				];
		}
	};
	
	
	if (bUseThirdPartyUI)
	{
		StructBuilder.AddProperty(ReductionMethodPropertyHandle.ToSharedRef());

		StructBuilder.AddCustomRow(LOCTEXT("PercentTriangles_Row", "Triangle Percentage"))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::GetVisibiltyIfCurrentReductionMethodIsNot, SMOT_MaxDeviation)))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("PercentTriangles", "Triangle Percentage"))
			.ToolTipText(LOCTEXT("PercentTriangles_ToolTip", "The simplification uses this percentage of source mesh's triangle count as a target."))
			]
		.ValueContent()
			[
				SNew(SSpinBox<float>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0.0f)
			.MaxValue(100.0f)
			.Value(this, &FSkeletalMeshReductionSettingsDetails::GetNumTrianglesPercentage)
			.OnValueChanged(this, &FSkeletalMeshReductionSettingsDetails::SetNumTrianglesPercentage)
			];

		StructBuilder.AddCustomRow(LOCTEXT("Accuracy_Row", "Accuracy Percentage"))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::GetVisibiltyIfCurrentReductionMethodIsNot, SMOT_NumOfTriangles)))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("PercentAccuracy", "Accuracy Percentage"))
			.ToolTipText(LOCTEXT("PercentAccuracy_ToolTip", "The simplification uses this as how much deviate from source mesh. Better works with hard surface meshes."))
			]
		.ValueContent()
			[
				SNew(SSpinBox<float>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.MinValue(0.0f)
			// if you set 100% accuracy, which will set 0.f as max deviation, simplygon ignores the value. Considered invalid.
			.MaxValue(100.f)
			.Value(this, &FSkeletalMeshReductionSettingsDetails::GetAccuracyPercentage)
			.OnValueChanged(this, &FSkeletalMeshReductionSettingsDetails::SetAccuracyPercentage)
			];

		// Parameters not used by simplygon
		const TArray<FName> CustomSimplifierOnlyProperties = {
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, NumOfVertPercentage),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfVerts),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfTriangles),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfVertsPercentage),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfTrianglesPercentage),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, TerminationCriterion),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bLockEdges),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bEnforceBoneBoundaries),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bMergeCoincidentVertBones),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, VolumeImportance),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bLockColorBounaries),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bImproveTrianglesForCloth)
		};

		uint32 NumChildren = 0;

		if (StructPropertyHandle->GetNumChildren(NumChildren) != FPropertyAccess::Fail)
		{
			for (uint32 Index = 0; Index < NumChildren; ++Index)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(Index);
				// we don't want to add the things that we added first
				// maybe we make array later if we have a lot. 
				FName PropertyName = ChildHandle->GetProperty()->GetFName();

				if (PropertyName == GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, BaseLOD))
				{
					BaseLODCustomization(ChildHandle);
				}
				else
				{
					if (!CustomizedProperties.Contains(PropertyName) && !CustomSimplifierOnlyProperties.Contains(PropertyName))
					{
						StructBuilder.AddProperty(ChildHandle.ToSharedRef());
					}
				}				
			}
		}
	}
	else  // Not third party: Using our own skeletal simplifier.
	{
		// Store structure's child properties
		// in Map for later filtering

		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);
		TMap<FName, TSharedPtr< IPropertyHandle > > PropertyHandles;
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			{
				PropertyHandles.Add(PropertyName, ChildHandle);
			}

		}

		// Third party only parameters:
		// E.g. PropertyHandles of parameters our native tool doesn't support

		const TArray<FName> UnWantedPropertyNames = {
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, ReductionMethod),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxDeviationPercentage),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, SilhouetteImportance),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, TextureImportance),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, NormalsThreshold),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, ShadingImportance),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, SkinningImportance),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, WeldingThreshold),
			GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, bRecalcNormals),
		};

		TArray<TSharedPtr<IPropertyHandle>> UnwantedPropertyHandles;
		for (const auto& UnwantedName : UnWantedPropertyNames)
		{
			UnwantedPropertyHandles.Add(PropertyHandles.FindChecked(UnwantedName));
		}

		// Pull down that selects that termination criterion to use.
		TerminationCriterionPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, TerminationCriterion));

		// These may be hidden depending on the termination criterion
		TSharedPtr<IPropertyHandle> VertPercentPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, NumOfVertPercentage));
		TSharedPtr<IPropertyHandle> TriPercentPropertyHandle  = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, NumOfTrianglesPercentage));

		TSharedPtr<IPropertyHandle> MaxNumOfVertsPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfVerts));
		TSharedPtr<IPropertyHandle> MaxNumOfTrisPropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfTriangles));

		TSharedPtr<IPropertyHandle> MaxNumOfVertsPercentagePropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfVertsPercentage));
		TSharedPtr<IPropertyHandle> MaxNumOfTrisPercentagePropertyHandle = PropertyHandles.FindChecked(GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, MaxNumOfTrianglesPercentage));

		for (auto Iter(PropertyHandles.CreateIterator()); Iter; ++Iter)
		{
			if (UnwantedPropertyHandles.Contains(Iter.Value())) 
			{
				IDetailPropertyRow& SettingsRow = StructBuilder.AddProperty(Iter.Value().ToSharedRef());
				SettingsRow.Visibility(TAttribute<EVisibility>(this, &FSkeletalMeshReductionSettingsDetails::GetVisibilityForThirdPartyTool));
			}
			else if (Iter.Value()->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkeletalMeshOptimizationSettings, BaseLOD))
			{
				BaseLODCustomization(Iter.Value());
			}
			else
			{
				IDetailPropertyRow& SettingsRow = StructBuilder.AddProperty(Iter.Value().ToSharedRef());

				// depending on the value of the pull down, optionally hide at most one of these.
				if (Iter.Value() == VertPercentPropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList = {
					SMTC_NumOfVerts,
					SMTC_TriangleOrVert
					};

					// Hide property if using triangle percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
				else if (Iter.Value() == TriPercentPropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList = 
					{
						SMTC_NumOfTriangles,
						SMTC_TriangleOrVert
					};
					// Hide property if using vert percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
				else if (Iter.Value() == MaxNumOfVertsPropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList = 
					{
						SMTC_AbsNumOfVerts,
						SMTC_AbsTriangleOrVert
					};
					// Hide property if using triangle percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
				else if (Iter.Value() == MaxNumOfVertsPercentagePropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList =
					{
						SMTC_NumOfVerts
					};
					// Hide property if using triangle percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
				else if (Iter.Value() == MaxNumOfTrisPropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList = 
					{
						SMTC_AbsNumOfTriangles,
						SMTC_AbsTriangleOrVert
					};
					// Hide property if using triangle percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
				else if (Iter.Value() == MaxNumOfTrisPercentagePropertyHandle)
				{
					const TArray< SkeletalMeshTerminationCriterion > VizList =
					{
						SMTC_NumOfTriangles
					};
					// Hide property if using triangle percentage
					SettingsRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs, VizList)));
				}
			}
		}

	}

}

float FSkeletalMeshReductionSettingsDetails::ConvertToPercentage(float Input) const
{
	return FMath::Clamp(Input*100.f, 0.f, 100.f);
}

float FSkeletalMeshReductionSettingsDetails::ConvertToDecimal(float Input) const
{
	return FMath::Clamp(Input / 100.f, 0.f, 1.f);
}

float FSkeletalMeshReductionSettingsDetails::GetNumTrianglesPercentage() const
{
	float CurrentValue;
	if (NumTrianglesPercentagePropertyHandle->GetValue(CurrentValue) != FPropertyAccess::Fail)
	{
		return ConvertToPercentage(CurrentValue);
	}

	return 0.f;
}

void FSkeletalMeshReductionSettingsDetails::SetNumTrianglesPercentage(float Value)
{
	float PropertyValue = ConvertToDecimal(Value);
	ensure(NumTrianglesPercentagePropertyHandle->SetValue(PropertyValue) != FPropertyAccess::Fail);
}


float FSkeletalMeshReductionSettingsDetails::GetAccuracyPercentage() const
{
	float CurrentValue;
	if (MaxDeviationPercentagePropertyHandle->GetValue(CurrentValue) != FPropertyAccess::Fail)
	{
		return ConvertToPercentage(1.f-CurrentValue);
	}

	return 0.f;
}
void FSkeletalMeshReductionSettingsDetails::SetAccuracyPercentage(float Value)
{
	float PropertyValue = 1.f - ConvertToDecimal(Value);
	ensure(MaxDeviationPercentagePropertyHandle->SetValue(PropertyValue) != FPropertyAccess::Fail);
}

EVisibility FSkeletalMeshReductionSettingsDetails::GetVisibiltyIfCurrentReductionMethodIsNot(enum SkeletalMeshOptimizationType ReductionType) const
{
	uint8 CurrentEnum;
	if (ReductionMethodPropertyHandle->GetValue(CurrentEnum) != FPropertyAccess::Fail)
	{
		enum SkeletalMeshOptimizationType CurrentReductionType = (SkeletalMeshOptimizationType)CurrentEnum;
		if (CurrentReductionType != ReductionType)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

EVisibility FSkeletalMeshReductionSettingsDetails::ShowIfCurrentCriterionIs(TArray<SkeletalMeshTerminationCriterion> TerminationCriterionArray) const
{
	uint8 CurrentEnum;
	if (TerminationCriterionPropertyHandle->GetValue(CurrentEnum) != FPropertyAccess::Fail)
	{
		enum SkeletalMeshTerminationCriterion CurrentReductionType = (SkeletalMeshTerminationCriterion)CurrentEnum;
		if (TerminationCriterionArray.Contains(CurrentReductionType))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

bool FSkeletalMeshReductionSettingsDetails::UseNativeLODTool() const
{

	bool bRequestNative = true;
	return bRequestNative;
}

EVisibility FSkeletalMeshReductionSettingsDetails::GetVisibilityForThirdPartyTool() const
{
	EVisibility VisiblityValue = EVisibility::Visible;

	if (UseNativeLODTool())
	{
		VisiblityValue = EVisibility::Hidden;
	}

	return VisiblityValue;
}
#undef LOCTEXT_NAMESPACE
