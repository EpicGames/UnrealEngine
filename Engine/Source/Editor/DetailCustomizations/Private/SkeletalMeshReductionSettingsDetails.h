// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "SkeletalMeshReductionSettings.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
struct EVisibility;

class FSkeletalMeshReductionSettingsDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	float ConvertToPercentage(float Input) const;
	float ConvertToDecimal(float Input) const;

	TSharedPtr<IPropertyHandle> TerminationCriterionPropertyHandle;
	TSharedPtr<IPropertyHandle> ReductionMethodPropertyHandle;
	TSharedPtr<IPropertyHandle> NumTrianglesPercentagePropertyHandle;
	TSharedPtr<IPropertyHandle> MaxDeviationPercentagePropertyHandle;

	float GetNumTrianglesPercentage() const;
	void SetNumTrianglesPercentage(float Value);

	float GetAccuracyPercentage() const;
	void SetAccuracyPercentage(float Value);

	// Used the the thrid-party UI.  
	EVisibility GetVisibiltyIfCurrentReductionMethodIsNot(SkeletalMeshOptimizationType ReductionType) const;

	// Used by the native tool UI.
	EVisibility ShowIfCurrentCriterionIs(TArray<SkeletalMeshTerminationCriterion> TerminationCriterionArray) const;

	/** Detect usage of thirdparty vs native tool */
	bool UseNativeLODTool() const;
	bool UseNativeReductionTool() const;

	/**
	Used to hide parameters that only make sense for the third party tool.
	@return EVisibility::Visible if we are using the simplygon tool, otherwise EVisibility::Hidden
	*/
	EVisibility GetVisibilityForThirdPartyTool() const;
};
