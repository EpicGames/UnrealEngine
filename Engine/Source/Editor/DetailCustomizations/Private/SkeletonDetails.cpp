// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonDetails.h"

#include "Animation/Skeleton.h"
#include "DetailLayoutBuilder.h"
#include "Misc/AssertionMacros.h"

TSharedRef<IDetailCustomization> FSkeletonDetails::MakeInstance()
{
	return MakeShareable(new FSkeletonDetails());
}

void FSkeletonDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(USkeleton, BoneTree));
}

