// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinnedMeshComponentDetails.h"

#include "Rendering/SkinWeightProfile.h"
#include "Components/ActorComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/SkeletalMesh.h"
#include "Fonts/SlateFontInfo.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SNameComboBox.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SkinnedMeshComponentDetails"

TSharedRef<IDetailCustomization> FSkinnedMeshComponentDetails::MakeInstance()
{
	return MakeShareable(new FSkinnedMeshComponentDetails);
}

void FSkinnedMeshComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.EditCategory("Mesh", FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& PhysicsCategory = DetailBuilder.EditCategory("Physics", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	IDetailCategoryBuilder& LODCategory = DetailBuilder.EditCategory("LOD", LOCTEXT("LODCategoryName", "Level of Detail"), ECategoryPriority::Default);
	
	// show extra field about actually used physics asset, but make sure to show it under physics asset override
	TSharedRef<IPropertyHandle> PhysicsAssetProperty =  DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinnedMeshComponent, PhysicsAssetOverride));
	if (PhysicsAssetProperty->IsValidHandle())
	{
		PhysicsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(USkinnedMeshComponent, PhysicsAssetOverride));
		CreateActuallyUsedPhysicsAssetWidget(PhysicsCategory.AddCustomRow( LOCTEXT("CurrentPhysicsAsset", "Currently used Physics Asset"), true), &DetailBuilder);
	}

	IDetailCategoryBuilder& SkinWeightCategory = DetailBuilder.EditCategory("SkinWeights", LOCTEXT("SkinWeightsLabel", "Skin Weights"), ECategoryPriority::Default);

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	// Only allow skin weight profile selection when having a single selected component
	if (Objects.Num() == 1)
	{
		if (USkinnedMeshComponent* Component = Cast<USkinnedMeshComponent>(Objects[0]))
		{
			WeakSkinnedMeshComponent = Component;
			PopulateSkinWeightProfileNames();
			CreateSkinWeightProfileSelectionWidget(SkinWeightCategory);			
		}
	}
}

void FSkinnedMeshComponentDetails::CreateSkinWeightProfileSelectionWidget(IDetailCategoryBuilder &SkinWeightCategory)
{
	SkinWeightCategory.AddCustomRow(LOCTEXT("SkinWeightProfileLabel", "Skin Weight Profile"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CurrentSkinWeightProfile", "Skin Weight Profile"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SAssignNew(SkinWeightCombo, SNameComboBox)
		.OptionsSource(&SkinWeightProfileNames)
		.InitiallySelectedItem([this]() 
		{
			// Make sure we check whether or not a profile has previously been set up to profile, and if so set the default selected combobox item accordingly
			const USkinnedMeshComponent* MeshComponent = WeakSkinnedMeshComponent.Get();
			if (MeshComponent)
			{
				const int32 Index = SkinWeightProfileNames.IndexOfByPredicate([MeshComponent](TSharedPtr<FName> Name) { return *Name == MeshComponent->GetCurrentSkinWeightProfileName(); });
				if (Index != INDEX_NONE)
				{
					return SkinWeightProfileNames[Index];
				}
			}
			// This always return first item which is Name_None
			return SkinWeightProfileNames[0];
		}())
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OnComboBoxOpening(FOnComboBoxOpening::CreateLambda([this]()
		{
			// Retrieve currently selected value, and check whether or not it is still valid, it could be that a profile has been renamed or removed without updating the entries
			FName Name = SkinWeightCombo->GetSelectedItem().IsValid() ? *SkinWeightCombo->GetSelectedItem().Get() : NAME_None;
			PopulateSkinWeightProfileNames();
			
			const int32 Index = SkinWeightProfileNames.IndexOfByPredicate([Name](TSharedPtr<FName> SearchName) { return Name == *SearchName; });
			if (Index != INDEX_NONE)
			{
				SkinWeightCombo->SetSelectedItem(SkinWeightProfileNames[Index]);
			}
		}))
		.OnSelectionChanged(SNameComboBox::FOnNameSelectionChanged::CreateLambda([this](TSharedPtr<FName> SelectedProfile, ESelectInfo::Type SelectInfo)
		{
			// Apply the skin weight profile to the component, according to the selected the name, 
			if (WeakSkinnedMeshComponent.IsValid() && SelectedProfile.IsValid())
			{
				USkinnedMeshComponent* MeshComponent = WeakSkinnedMeshComponent.Get();
				if (MeshComponent)
				{
					MeshComponent->ClearSkinWeightProfile();

					if (*SelectedProfile != NAME_None)
					{
						MeshComponent->SetSkinWeightProfile(*SelectedProfile);
					}
				}
			}
		}))
	];
}

void FSkinnedMeshComponentDetails::CreateActuallyUsedPhysicsAssetWidget(FDetailWidgetRow& OutWidgetRow, IDetailLayoutBuilder* DetailBuilder) const
{
	OutWidgetRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CurrentPhysicsAsset", "Currently used Physics Asset"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth( 1 )
		[
			SNew(SEditableTextBox)
			.Text(this, &FSkinnedMeshComponentDetails::GetUsedPhysicsAssetAsText, DetailBuilder)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsReadOnly(true)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding( 2.0f, 1.0f )
		[
			PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FSkinnedMeshComponentDetails::BrowseUsedPhysicsAsset, DetailBuilder))
		]
	];
}

bool FSkinnedMeshComponentDetails::FindUniqueUsedPhysicsAsset(IDetailLayoutBuilder* DetailBuilder, UPhysicsAsset*& OutFoundPhysicsAsset) const
{
	int32 UsedPhysicsAssetCount = 0;
	OutFoundPhysicsAsset = nullptr;
	
	for (const TWeakObjectPtr<UObject>& SelectedObject : DetailBuilder->GetSelectedObjects())
	{
		if (AActor* Actor = Cast<AActor>(SelectedObject.Get()))
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (USkinnedMeshComponent* SkinnedMeshComp = Cast<USkinnedMeshComponent>(Component))
				{
					// Only use registered and visible primitive components when calculating bounds
					if (UsedPhysicsAssetCount > 0)
					{
						return false;
					}
					++UsedPhysicsAssetCount;
					OutFoundPhysicsAsset = SkinnedMeshComp->GetPhysicsAsset();
				}
			}
		}
	}
	return true;
}

void FSkinnedMeshComponentDetails::PopulateSkinWeightProfileNames()
{
	SkinWeightProfileNames.Empty();

	// Always make sure we have a default 'none' option
	const FName DefaultProfileName = NAME_None;
	SkinWeightProfileNames.Add(MakeShared<FName>(DefaultProfileName));

	// Retrieve all possible skin weight profiles from the component
	if (USkinnedMeshComponent* Component = WeakSkinnedMeshComponent.Get())
	{
		if (USkeletalMesh* Mesh = Cast<USkeletalMesh>(Component->GetSkinnedAsset()))
		{			
			for (const FSkinWeightProfileInfo& Profile : Mesh->GetSkinWeightProfiles())
			{
				SkinWeightProfileNames.AddUnique(MakeShared<FName>(Profile.Name));
			}
		}
	}
}

FText FSkinnedMeshComponentDetails::GetUsedPhysicsAssetAsText(IDetailLayoutBuilder* DetailBuilder) const
{
	UPhysicsAsset* UsedPhysicsAsset = NULL;
	if (! FindUniqueUsedPhysicsAsset(DetailBuilder, UsedPhysicsAsset))
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	if (UsedPhysicsAsset)
	{
		return FText::FromString(UsedPhysicsAsset->GetName());
	}
	else
	{
		return FText::GetEmpty();
	}
}

void FSkinnedMeshComponentDetails::BrowseUsedPhysicsAsset(IDetailLayoutBuilder* DetailBuilder) const
{
	UPhysicsAsset* UsedPhysicsAsset = NULL;
	if (FindUniqueUsedPhysicsAsset(DetailBuilder, UsedPhysicsAsset) &&
		UsedPhysicsAsset != NULL)
	{
		TArray<UObject*> Objects;
		Objects.Add(UsedPhysicsAsset);
		GEditor->SyncBrowserToObjects(Objects);
	}
}

#undef LOCTEXT_NAMESPACE
