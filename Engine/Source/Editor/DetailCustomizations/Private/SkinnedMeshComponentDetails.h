// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FDetailWidgetRow;
class FName;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SNameComboBox;
class UPhysicsAsset;
class USkinnedMeshComponent;

class FSkinnedMeshComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	void CreateActuallyUsedPhysicsAssetWidget(FDetailWidgetRow& OutWidgetRow, IDetailLayoutBuilder* DetailBuilder) const;

	FText GetUsedPhysicsAssetAsText(IDetailLayoutBuilder* DetailBuilder) const;
	void BrowseUsedPhysicsAsset(IDetailLayoutBuilder* DetailBuilder) const;

	bool FindUniqueUsedPhysicsAsset(IDetailLayoutBuilder* DetailBuilder, UPhysicsAsset*& OutFoundPhysicsAsset) const;

	void CreateSkinWeightProfileSelectionWidget(IDetailCategoryBuilder& SkinWeightCategory);
	void PopulateSkinWeightProfileNames();

	/** Skin Weight Profile Selector */
	TSharedPtr<SNameComboBox> SkinWeightCombo;
	TArray<TSharedPtr<FName>> SkinWeightProfileNames;
	TSharedPtr<IPropertyHandle> SkeletalMeshHandle;

	TWeakObjectPtr<USkinnedMeshComponent> WeakSkinnedMeshComponent;
	
};
