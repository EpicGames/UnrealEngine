// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkyLightComponentDetails.h"

#include "Components/LightComponentBase.h"
#include "Components/SceneComponent.h"
#include "Components/SkyLightComponent.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/SkyLight.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SkyLightComponentDetails"



TSharedRef<IDetailCustomization> FSkyLightComponentDetails::MakeInstance()
{
	return MakeShareable( new FSkyLightComponentDetails );
}

void FSkyLightComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailLayout.GetSelectedObjects();
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		const TWeakObjectPtr<UObject>& CurrentObject = SelectedObjects[ObjectIndex];
		if (CurrentObject.IsValid())
		{
			ASkyLight* CurrentCaptureActor = Cast<ASkyLight>(CurrentObject.Get());
			if (CurrentCaptureActor != NULL)
			{
				SkyLight = CurrentCaptureActor;
				break;
			}
		}
	}

	USkyLightComponent* SkyLightComponent = SkyLight.IsValid() ? SkyLight->GetLightComponent() : nullptr;

	// Mobility property is on the scene component base class not the light component and that is why we have to use USceneComponent::StaticClass
	TSharedRef<IPropertyHandle> MobilityHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USkyLightComponent, Mobility), USceneComponent::StaticClass());
	// Set a mobility tooltip specific to lights
	MobilityHandle->SetToolTipText(LOCTEXT("SkyLightMobilityTooltip", "Mobility for sky light components determines what rendering methods will be used.  A Stationary sky light has its shadowing baked into Bent Normal AO by Lightmass, but its lighting can be changed in game."));

	TSharedPtr<IPropertyHandle> LightIntensityProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USkyLightComponent, Intensity), ULightComponentBase::StaticClass());

	if( LightIntensityProperty->IsValidHandle() )
	{
		// Point lights need to override the ui min and max for units of lumens, so we have to undo that
		LightIntensityProperty->SetInstanceMetaData("UIMin", TEXT("0.0f"));
		LightIntensityProperty->SetInstanceMetaData("UIMax", TEXT("50000.0f"));
		LightIntensityProperty->SetInstanceMetaData("SliderExponent", TEXT("10.0f"));

		if (!SkyLightComponent || (SkyLightComponent->SourceType != SLS_CapturedScene && SkyLightComponent->IsRealTimeCaptureEnabled()))
		{
			LightIntensityProperty->SetInstanceMetaData("Units", TEXT("CandelaPerMeter2"));
		}
		else
		{
			LightIntensityProperty->SetPropertyDisplayName(LOCTEXT("LightIntensityScaleDisplayName", "Intensity Scale"));
		}
	}

	DetailLayout.EditCategory("Light", FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	// The bVisible checkbox in the rendering category is frequently used on lights
	// Editing the rendering category and giving it TypeSpecific priority will place it just under the Light category
	DetailLayout.EditCategory("Rendering", FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	TSharedPtr<IPropertyHandle> SourceTypeProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USkyLightComponent, SourceType), USkyLightComponent::StaticClass());
	SourceTypeProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FSkyLightComponentDetails::OnSourceTypeChanged));


	DetailLayout.EditCategory( "SkyLight" )
	.AddCustomRow( NSLOCTEXT("SkyLightDetails", "UpdateSkyLight", "Recapture Scene") )
		.NameContent()
		[
			SNew( STextBlock )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text( NSLOCTEXT("SkyLightDetails", "UpdateSkyLight", "Recapture Scene") )
		]
		.ValueContent()
		.MaxDesiredWidth(125.f)
		.MinDesiredWidth(125.f)
		[
			SNew(SButton)
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.OnClicked( this, &FSkyLightComponentDetails::OnUpdateSkyCapture )
			[
				SNew( STextBlock )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.Text( NSLOCTEXT("SkyLightDetails", "UpdateSkyCapture", "Recapture") )
			]
		];
}

void FSkyLightComponentDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CachedDetailBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

FReply FSkyLightComponentDetails::OnUpdateSkyCapture()
{
	if (SkyLight.IsValid())
	{
		if (UWorld* SkyLightWorld = SkyLight->GetWorld())
		{
			SkyLightWorld->UpdateAllSkyCaptures();
		}
	}

	return FReply::Handled();
}

void FSkyLightComponentDetails::OnSourceTypeChanged()
{
	IDetailLayoutBuilder* DetailBuilder = CachedDetailBuilder.Pin().Get();
	if (DetailBuilder)
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
