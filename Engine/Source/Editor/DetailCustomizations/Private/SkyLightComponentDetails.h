// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ASkyLight;
class IDetailLayoutBuilder;

class FSkyLightComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
private:

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

	FReply OnUpdateSkyCapture();

	void OnSourceTypeChanged();

private:

	// The detail builder for this cusomtomisation
	TWeakPtr<IDetailLayoutBuilder> CachedDetailBuilder;

	/** The selected sky light */
	TWeakObjectPtr<ASkyLight> SkyLight;
};
