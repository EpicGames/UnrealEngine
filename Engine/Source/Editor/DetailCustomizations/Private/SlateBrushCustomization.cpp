// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SlateBrushCustomization.h"

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Texture2D.h"
#include "Fonts/SlateFontInfo.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "Slate/SlateTextureAtlasInterface.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "TextureCompiler.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

/**
 * Slate Brush Preview widget
 */
class SSlateBrushPreview : public SBorder
{
	/**
	 * The widget zone the user is interacting with
	 */
	enum EWidgetZone
	{
		WZ_NotInWidget			= 0,
		WZ_InWidget				= 1,
		WZ_RightBorder			= 2,
		WZ_BottomBorder			= 3,
		WZ_BottomRightBorder	= 4
	};

public:
	SLATE_BEGIN_ARGS( SSlateBrushPreview )
		{}
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, DrawAsProperty )
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, TilingProperty )
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, ImageSizeProperty )
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, MarginProperty )
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, ResourceObjectProperty )
		SLATE_ARGUMENT(	TSharedPtr<IPropertyHandle>, ImageTypeProperty )
		SLATE_ARGUMENT( FSlateBrush*, SlateBrush )
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 */
	void Construct( const FArguments& InArgs )
	{
		DrawAsProperty = InArgs._DrawAsProperty;
		TilingProperty = InArgs._TilingProperty;
		ImageSizeProperty = InArgs._ImageSizeProperty;
		MarginProperty = InArgs._MarginProperty;
		ResourceObjectProperty = InArgs._ResourceObjectProperty;
		ImageTypeProperty = InArgs._ImageTypeProperty;

		FSimpleDelegate OnDrawAsChangedDelegate = FSimpleDelegate::CreateSP( this, &SSlateBrushPreview::OnDrawAsChanged );
		DrawAsProperty->SetOnPropertyValueChanged( OnDrawAsChangedDelegate );

		FSimpleDelegate OnTilingChangedDelegate = FSimpleDelegate::CreateSP( this, &SSlateBrushPreview::OnTilingChanged );
		TilingProperty->SetOnPropertyValueChanged( OnTilingChangedDelegate );

		FSimpleDelegate OnBrushResourceChangedDelegate = FSimpleDelegate::CreateSP( this, &SSlateBrushPreview::OnBrushResourceChanged );
		ResourceObjectProperty->SetOnPropertyValueChanged( OnBrushResourceChangedDelegate );

		FSimpleDelegate OnImageSizeChangedDelegate = FSimpleDelegate::CreateSP( this, &SSlateBrushPreview::OnImageSizeChanged );
		ImageSizeProperty->SetOnPropertyValueChanged( OnImageSizeChangedDelegate );

		uint32 NumChildren = 0;
		ImageSizeProperty->GetNumChildren( NumChildren );
		for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
		{
			TSharedPtr<IPropertyHandle> Child = ImageSizeProperty->GetChildHandle( ChildIndex );
			Child->SetOnPropertyValueChanged( OnImageSizeChangedDelegate );
		}

		FSimpleDelegate OnMarginChangedDelegate = FSimpleDelegate::CreateSP( this, &SSlateBrushPreview::OnMarginChanged );
		MarginProperty->SetOnPropertyValueChanged( OnMarginChangedDelegate );

		MarginProperty->GetNumChildren( NumChildren );
		for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
		{
			TSharedPtr<IPropertyHandle> Child = MarginProperty->GetChildHandle( ChildIndex );
			Child->SetOnPropertyValueChanged( OnMarginChangedDelegate );
		}

		HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;
		VerticalAlignment = EVerticalAlignment::VAlign_Fill;
		bUserIsResizing = false;
		MouseZone = WZ_NotInWidget;

		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage( FAppStyle::GetBrush( "PropertyEditor.SlateBrushPreview" ) )
			.Padding( FMargin( 4.0f, 4.0f, 4.0f, 14.0f ) )
			[
				SNew( SBox )
				.WidthOverride( this, &SSlateBrushPreview::GetPreviewWidth ) 
				.HeightOverride( this, &SSlateBrushPreview::GetPreviewHeight )
				[
					SNew( SOverlay )
					+SOverlay::Slot()
					[
						SNew( SImage )
						.Image( FAppStyle::GetBrush( "Checkerboard" ) )
					]

					+SOverlay::Slot()
					.Padding( FMargin( ImagePadding ) )
					.Expose( OverlaySlot )
					[
						SNew( SImage )
						.Image( InArgs._SlateBrush )
					]

					+SOverlay::Slot()
					.HAlign( HAlign_Left )
					.VAlign( VAlign_Fill )
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( SSpacer )
							.Size( this, &SSlateBrushPreview::GetLeftMarginLinePosition )
						]
						+SHorizontalBox::Slot()
						[
							SNew( SImage )
							.Image( FAppStyle::GetBrush( "PropertyEditor.VerticalDottedLine" ) )
							.Visibility( this, &SSlateBrushPreview::GetMarginLineVisibility )
						]
					]

					+SOverlay::Slot()
					.HAlign( HAlign_Left )
					.VAlign( VAlign_Fill )
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( SSpacer )
							.Size( this, &SSlateBrushPreview::GetRightMarginLinePosition )
						]
						+SHorizontalBox::Slot()
						[
							SNew( SImage )
							.Image( FAppStyle::GetBrush( "PropertyEditor.VerticalDottedLine" ) )
							.Visibility( this, &SSlateBrushPreview::GetMarginLineVisibility )
						]
					]

					+SOverlay::Slot()
					.HAlign( HAlign_Fill )
					.VAlign( VAlign_Top )
					[
						SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew( SSpacer )
							.Size( this, &SSlateBrushPreview::GetTopMarginLinePosition )
						]
						+SVerticalBox::Slot()
						[
							SNew( SImage )
							.Image( FAppStyle::GetBrush( "PropertyEditor.HorizontalDottedLine" ) )
							.Visibility( this, &SSlateBrushPreview::GetMarginLineVisibility )
						]
					]

					+SOverlay::Slot()
					.HAlign( HAlign_Fill )
					.VAlign( VAlign_Top )
					[
						SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew( SSpacer )
							.Size( this, &SSlateBrushPreview::GetBottomMarginLinePosition )
						]
						+SVerticalBox::Slot()
						[
							SNew( SImage )
							.Image( FAppStyle::GetBrush( "PropertyEditor.HorizontalDottedLine" ) )
							.Visibility( this, &SSlateBrushPreview::GetMarginLineVisibility )
						]
					]
				]
			]
		);

		CachedTextureSize = FVector2f::ZeroVector;
		CachePropertyValues();
		SetDefaultAlignment();
		UpdatePreviewImageSize();
		UpdateMarginLinePositions();
	}

	/**
	 * Generate the alignment combo box widgets
	 */
	TSharedRef<SWidget> GenerateAlignmentComboBoxes()
	{
		HorizontalAlignmentComboItems.Add( MakeShareable( new EHorizontalAlignment( EHorizontalAlignment::HAlign_Fill ) ) );
		HorizontalAlignmentComboItems.Add( MakeShareable( new EHorizontalAlignment( EHorizontalAlignment::HAlign_Left ) ) );
		HorizontalAlignmentComboItems.Add( MakeShareable( new EHorizontalAlignment( EHorizontalAlignment::HAlign_Center ) ) );
		HorizontalAlignmentComboItems.Add( MakeShareable( new EHorizontalAlignment( EHorizontalAlignment::HAlign_Right ) ) );
		VerticalAlignmentComboItems.Add( MakeShareable( new EVerticalAlignment( EVerticalAlignment::VAlign_Fill ) ) );
		VerticalAlignmentComboItems.Add( MakeShareable( new EVerticalAlignment( EVerticalAlignment::VAlign_Top ) ) );
		VerticalAlignmentComboItems.Add( MakeShareable( new EVerticalAlignment( EVerticalAlignment::VAlign_Center ) ) );
		VerticalAlignmentComboItems.Add( MakeShareable( new EVerticalAlignment( EVerticalAlignment::VAlign_Bottom ) ) );

		return
			SNew( SUniformGridPanel )
			.SlotPadding( FAppStyle::GetMargin( "StandardDialog.SlotPadding" ) )
			+SUniformGridPanel::Slot( 0, 0 )
			.HAlign( HAlign_Right )
			.VAlign( VAlign_Center )
			[
				SNew( STextBlock )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.Text( NSLOCTEXT( "UnrealEd", "HorizontalAlignment", "Horizontal Alignment" ) )
				.ToolTipText( NSLOCTEXT( "UnrealEd", "PreviewHorizontalAlignment", "Horizontal alignment for the preview" ) )
			]

			+SUniformGridPanel::Slot( 1, 0 )
			[
				SAssignNew( HorizontalAlignmentCombo, SComboBox< TSharedPtr<EHorizontalAlignment> > )
				.OptionsSource( &HorizontalAlignmentComboItems )
				.OnGenerateWidget( this, &SSlateBrushPreview::MakeHorizontalAlignmentComboButtonItemWidget )
				.InitiallySelectedItem( HorizontalAlignmentComboItems[ 0 ] )
				.OnSelectionChanged( this, &SSlateBrushPreview::OnHorizontalAlignmentComboSelectionChanged )
				.Content()
				[
					SNew( STextBlock )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
					.Text( this, &SSlateBrushPreview::GetHorizontalAlignmentComboBoxContent )
					.ToolTipText( this, &SSlateBrushPreview::GetHorizontalAlignmentComboBoxContentToolTip )
				]
			]

			+SUniformGridPanel::Slot( 2, 0 )
			.HAlign( HAlign_Right )
			.VAlign( VAlign_Center )
			[
				SNew( STextBlock )
				.Font( IDetailLayoutBuilder::GetDetailFont() )
				.Text( NSLOCTEXT( "UnrealEd", "VerticalAlignment", "Vertical Alignment" ) )
				.ToolTipText( NSLOCTEXT( "UnrealEd", "PreviewVerticalAlignment", "Vertical alignment for the preview" ) )
			]

			+SUniformGridPanel::Slot( 3, 0 )
			[
				SAssignNew( VerticalAlignmentCombo, SComboBox< TSharedPtr<EVerticalAlignment> > )
				.OptionsSource( &VerticalAlignmentComboItems )
				.OnGenerateWidget( this, &SSlateBrushPreview::MakeVerticalAlignmentComboButtonItemWidget )
				.InitiallySelectedItem( VerticalAlignmentComboItems[ 0 ] )
				.OnSelectionChanged( this, &SSlateBrushPreview::OnVerticalAlignmentComboSelectionChanged )
				.Content()
				[
					SNew( STextBlock )
					.Font( IDetailLayoutBuilder::GetDetailFont() )
					.Text( this, &SSlateBrushPreview::GetVerticalAlignmentComboBoxContent )
					.ToolTipText( this, &SSlateBrushPreview::GetVerticalAlignmentComboBoxContentToolTip )
				]
			];
	}

private:

	/** Margin line types */
	enum EMarginLine
	{
		MarginLine_Left,
		MarginLine_Top,
		MarginLine_Right,
		MarginLine_Bottom,
		MarginLine_Count
	};
	
	/**
	 * SWidget interface
	 */
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton ) 
		{
			bUserIsResizing = true;
			ResizeAnchorPosition = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
			ResizeAnchorSize = PreviewImageSize;
			return FReply::Handled().CaptureMouse( SharedThis( this ) );
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bUserIsResizing )
		{
			bUserIsResizing = false;
			return FReply::Handled().ReleaseMouseCapture();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		const FVector2f LocalMouseCoordinates( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );

		if( bUserIsResizing )
		{
			if( MouseZone >= WZ_RightBorder && MouseZone <= WZ_BottomRightBorder )
			{
				FVector2f Delta( LocalMouseCoordinates - ResizeAnchorPosition );

				if( MouseZone == WZ_RightBorder )
				{
					Delta.Y = 0.0f;
				}
				else if( MouseZone == WZ_BottomBorder )
				{
					Delta.X = 0.0f;
				}

				PreviewImageSize.Set( FMath::Max( ResizeAnchorSize.X + Delta.X, 16.0f ), FMath::Max( ResizeAnchorSize.Y + Delta.Y, 16.0f ) );
				UpdateMarginLinePositions();
			}
		}
		else
		{
			MouseZone = FindMouseZone( LocalMouseCoordinates );
		}
		
		return FReply::Unhandled();
	}
	
	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		const FVector2f LocalMouseCoordinates( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );
		MouseZone = FindMouseZone( LocalMouseCoordinates );
		SBorder::OnMouseEnter( MyGeometry, MouseEvent );
	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override
	{
		if( !bUserIsResizing )
		{
			MouseZone = WZ_NotInWidget;
			SBorder::OnMouseLeave( MouseEvent );
		}
	}

	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override
	{
		if( MouseZone == WZ_RightBorder )
		{
			return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight );
		}
		else if( MouseZone == WZ_BottomBorder)
		{
			return FCursorReply::Cursor( EMouseCursor::ResizeUpDown );
		}
		else if( MouseZone == WZ_BottomRightBorder )
		{
			return FCursorReply::Cursor( EMouseCursor::ResizeSouthEast );
		}
		else
		{
			return FCursorReply::Unhandled();
		}
	}

	/**
	 * End of SWidget interface
	 */

	/**
	 * Determine which zone of the widget that the mouse is in
	 */
	EWidgetZone FindMouseZone( const FVector2f& LocalMouseCoordinates ) const
	{
		EWidgetZone InMouseZone = WZ_NotInWidget;
		const FVector2f DesiredZoneSize( GetDesiredSize() );

		if( LocalMouseCoordinates.X > DesiredZoneSize.X - BorderHitSize )
		{
			InMouseZone = LocalMouseCoordinates.Y > DesiredZoneSize.Y - BorderHitSize ? WZ_BottomRightBorder : WZ_RightBorder;
		}
		else if( LocalMouseCoordinates.Y > DesiredZoneSize.Y - BorderHitSize )
		{
			InMouseZone = WZ_BottomBorder;
		}
		else if( LocalMouseCoordinates.X >= BorderHitSize && LocalMouseCoordinates.Y >= BorderHitSize )
		{
			InMouseZone = WZ_InWidget;
		}

		return InMouseZone;
	}

	/**
	 * Return the text for the specified horizontal alignment
	 */
	FText MakeHorizontalAlignmentComboText( EHorizontalAlignment Alignment ) const
	{
		FText AlignmentText;
	
		switch( Alignment )
		{
			case EHorizontalAlignment::HAlign_Fill:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentFill", "Fill" );
				break;
	
			case EHorizontalAlignment::HAlign_Left:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentLeft", "Left" );
				break;
	
			case EHorizontalAlignment::HAlign_Center:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentCenter", "Center" );
				break;
	
			case EHorizontalAlignment::HAlign_Right:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentRight", "Right" );
				break;
		}
	
		return AlignmentText;
	}

	/**
	 * Return the text for the specified vertical alignment
	 */
	FText MakeVerticalAlignmentComboText( EVerticalAlignment Alignment ) const
	{
		FText AlignmentText;
	
		switch( Alignment )
		{
			case EVerticalAlignment::VAlign_Fill:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentFill", "Fill" );
				break;
	
			case EVerticalAlignment::VAlign_Top:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentTop", "Top" );
				break;
	
			case EVerticalAlignment::VAlign_Center:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentCenter", "Center" );
				break;
	
			case EVerticalAlignment::VAlign_Bottom:
				AlignmentText = NSLOCTEXT( "UnrealEd", "AlignmentBottom", "Bottom" );
				break;
		}
	
		return AlignmentText;
	}

	/**
	 * Return the tooltip text for the specified horizontal alignment
	 */
	FText MakeHorizontalAlignmentComboToolTipText( EHorizontalAlignment Alignment ) const
	{
		FText ToolTipText;
	
		switch( Alignment )
		{
			case EHorizontalAlignment::HAlign_Fill:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentFillToolTip", "The image will fill the preview" );
				break;
	
			case EHorizontalAlignment::HAlign_Left:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentLeftToolTip", "The image will be aligned to the left of the preview" );
				break;
	
			case EHorizontalAlignment::HAlign_Center:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentCenterToolTip", "The image will be positioned in the centre of the preview" );
				break;
	
			case EHorizontalAlignment::HAlign_Right:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentRightToolTip", "The image will be aligned from the right of the preview" );
				break;
		}
	
		return ToolTipText;
	}

	/**
	 * Return the tooltip text for the specified vertical alignment
	 */
	FText MakeVerticalAlignmentComboToolTipText( EVerticalAlignment Alignment ) const
	{
		FText ToolTipText;
	
		switch( Alignment )
		{
			case EVerticalAlignment::VAlign_Fill:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentFillToolTip", "The image will fill the preview" );
				break;
	
			case EVerticalAlignment::VAlign_Top:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentTopToolTip", "The image will be aligned to the top of the preview" );
				break;
	
			case EVerticalAlignment::VAlign_Center:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentCenterToolTip", "The image will be positioned in the centre of the preview" );
				break;
	
			case EVerticalAlignment::VAlign_Bottom:
				ToolTipText = NSLOCTEXT( "UnrealEd", "AlignmentBottomToolTip", "The image will be aligned from the bottom of the preview" );
				break;
		}
	
		return ToolTipText;
	}

	/**
	 * Make the horizontal alignment combo button item widget
	 */
	TSharedRef<SWidget> MakeHorizontalAlignmentComboButtonItemWidget( TSharedPtr<EHorizontalAlignment> Alignment )
	{
		return
			SNew( STextBlock )
			.Text( MakeHorizontalAlignmentComboText( *Alignment ) )
			.ToolTipText( MakeHorizontalAlignmentComboToolTipText( *Alignment ) )
			.Font( IDetailLayoutBuilder::GetDetailFont() );
	}

	/**
	 * Make the vertical alignment combo button item widget
	 */
	TSharedRef<SWidget> MakeVerticalAlignmentComboButtonItemWidget( TSharedPtr<EVerticalAlignment> Alignment )
	{
		return
			SNew( STextBlock )
			.Text( MakeVerticalAlignmentComboText( *Alignment ) )
			.ToolTipText( MakeVerticalAlignmentComboToolTipText( *Alignment ) )
			.Font( IDetailLayoutBuilder::GetDetailFont() );
	}

	/**
	 * Get the horizontal alignment combo box content text
	 */
	FText GetHorizontalAlignmentComboBoxContent() const
	{
		return MakeHorizontalAlignmentComboText( HorizontalAlignment );
	}
	
	/**
	 * Get the vertical alignment combo box content text
	 */
	FText GetVerticalAlignmentComboBoxContent() const
	{
		return MakeVerticalAlignmentComboText( VerticalAlignment );
	}

	/**
	 * Get the horizontal alignment combo box content tooltip text
	 */
	FText GetHorizontalAlignmentComboBoxContentToolTip() const
	{
		return MakeHorizontalAlignmentComboToolTipText( HorizontalAlignment );
	}
	
	/**
	 * Get the vertical alignment combo box content tooltip text
	 */
	FText GetVerticalAlignmentComboBoxContentToolTip() const
	{
		return MakeVerticalAlignmentComboToolTipText( VerticalAlignment );
	}

	/**
	 * Cache the slate brush property values
	 */
	void CachePropertyValues()
	{
		UObject* ResourceObject;
		FPropertyAccess::Result Result = ResourceObjectProperty->GetValue( ResourceObject );
		if( Result == FPropertyAccess::Success )
		{
			using ImageSizeType = decltype(FSlateBrush::ImageSize);

			TArray<void*> RawData;
			ImageSizeProperty->AccessRawData(RawData);
			if (RawData.Num() > 0 && RawData[0] != NULL)
			{
				CachedImageSizeValue = *static_cast<ImageSizeType*>(RawData[0]);
			}

			UTexture2D* BrushTexture = Cast<UTexture2D>(ResourceObject);
			if( BrushTexture )
			{
				if ( BrushTexture->IsDefaultTexture() )
				{
					// GetSizeX/Y will return the incorrect value if this texture is being compiled so we need to wait for it here
					UTexture* const BaseTexture = BrushTexture;
					FTextureCompilingManager::Get().FinishCompilation( MakeArrayView(&BaseTexture, 1) );
				}
				CachedTextureSize = FVector2f( static_cast<float>(BrushTexture->GetSizeX()), static_cast<float>(BrushTexture->GetSizeY()) );
			}
			else if ( ISlateTextureAtlasInterface* AtlasedTextureObject = Cast<ISlateTextureAtlasInterface>(ResourceObject) )
			{
				CachedTextureSize = UE::Slate::CastToVector2f(AtlasedTextureObject->GetSlateAtlasData().GetSourceDimensions());
			}
			else if( CachedTextureSize == FVector2f::ZeroVector )
			{
				// If the cached texture size is not initialized, create a default value now for materials 
				CachedTextureSize = CachedImageSizeValue;
			}

			uint8 DrawAsType;
			Result = DrawAsProperty->GetValue( DrawAsType );
			if( Result == FPropertyAccess::Success )
			{
				CachedDrawAsType = static_cast<ESlateBrushDrawType::Type>(DrawAsType);
			}

			uint8 TilingType;
			Result = TilingProperty->GetValue( TilingType );
			if( Result == FPropertyAccess::Success )
			{
				CachedTilingType = static_cast<ESlateBrushTileType::Type>(TilingType);
			}

			MarginProperty->AccessRawData( RawData );
			if( RawData.Num() > 0 && RawData[ 0 ] != NULL )
			{
				CachedMarginPropertyValue = *static_cast<FMargin*>(RawData[ 0 ]);
			}
		}
	}

	/**
	 * On horizontal alignment selection change
	 */
	void OnHorizontalAlignmentComboSelectionChanged( TSharedPtr<EHorizontalAlignment> NewSelection, ESelectInfo::Type /*SelectInfo*/ )
	{
		HorizontalAlignment = *NewSelection;
		UpdateOverlayAlignment();
		UpdateMarginLinePositions();
	}

	/**
	 * On vertical alignment selection change
	 */
	void OnVerticalAlignmentComboSelectionChanged( TSharedPtr<EVerticalAlignment> NewSelection, ESelectInfo::Type /*SelectInfo*/ )
	{
		VerticalAlignment = *NewSelection;
		UpdateOverlayAlignment();
		UpdateMarginLinePositions();
	}

	/**
	 * Get the horizontal alignment
	 */
	EHorizontalAlignment GetHorizontalAlignment() const
	{
		return HorizontalAlignment;
	}

	/**
	 * Get the vertical alignment
	 */
	EVerticalAlignment GetVerticalAlignment() const
	{
		return VerticalAlignment;
	}

	/**
	 * Update the margin line positions
	 */
	void UpdateMarginLinePositions()
	{
		const FVector2f DrawSize( (HorizontalAlignment == EHorizontalAlignment::HAlign_Fill || PreviewImageSize.X < CachedImageSizeValue.X) ? PreviewImageSize.X : CachedImageSizeValue.X,
								  (VerticalAlignment == EVerticalAlignment::VAlign_Fill || PreviewImageSize.Y < CachedImageSizeValue.Y) ? PreviewImageSize.Y : CachedImageSizeValue.Y );

		FVector2f Position( 0.0f, 0.0f );

		if( PreviewImageSize.X > DrawSize.X )
		{
			if( HorizontalAlignment == EHorizontalAlignment::HAlign_Center )
			{
				Position.X = (PreviewImageSize.X - DrawSize.X) * 0.5f;
			}
			else if( HorizontalAlignment == EHorizontalAlignment::HAlign_Right )
			{
				Position.X = PreviewImageSize.X - DrawSize.X;
			}
		}

		if( PreviewImageSize.Y > DrawSize.Y )
		{
			if( VerticalAlignment == EVerticalAlignment::VAlign_Center )
			{
				Position.Y = (PreviewImageSize.Y - DrawSize.Y) * 0.5f;
			}
			else if( VerticalAlignment == EVerticalAlignment::VAlign_Bottom )
			{
				Position.Y = PreviewImageSize.Y - DrawSize.Y;
			}
		}

		float LeftMargin = CachedTextureSize.X * CachedMarginPropertyValue.Left;
		float RightMargin = DrawSize.X - CachedTextureSize.X * CachedMarginPropertyValue.Right;
		float TopMargin = CachedTextureSize.Y * CachedMarginPropertyValue.Top;
		float BottomMargin = DrawSize.Y - CachedTextureSize.Y * CachedMarginPropertyValue.Bottom;

		if( RightMargin < LeftMargin )
		{
			LeftMargin = DrawSize.X * 0.5f;
			RightMargin = LeftMargin;
		}

		if( BottomMargin < TopMargin )
		{
			TopMargin = DrawSize.Y * 0.5f;
			BottomMargin = TopMargin;
		}

		MarginLinePositions[ MarginLine_Left ] = FVector2f( ImagePadding + Position.X + LeftMargin, 1.0f );
		MarginLinePositions[ MarginLine_Right ] = FVector2f( ImagePadding + Position.X + RightMargin, 1.0f );
		MarginLinePositions[ MarginLine_Top ] = FVector2f( 1.0f, ImagePadding + Position.Y + TopMargin );
		MarginLinePositions[ MarginLine_Bottom ] = FVector2f( 1.0f, ImagePadding + Position.Y + BottomMargin );
	}

	/**
	 * Set the default preview alignment based on the DrawAs and Tiling properties
	 */
	void SetDefaultAlignment()
	{
		HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;
		VerticalAlignment = EVerticalAlignment::VAlign_Fill;

		if( CachedDrawAsType == ESlateBrushDrawType::Image )
		{
			switch( CachedTilingType )
			{
			case ESlateBrushTileType::NoTile:
				HorizontalAlignment = EHorizontalAlignment::HAlign_Center;
				VerticalAlignment = EVerticalAlignment::VAlign_Center;
				break;

			case ESlateBrushTileType::Horizontal:
				VerticalAlignment = EVerticalAlignment::VAlign_Center;
				break;

			case ESlateBrushTileType::Vertical:
				HorizontalAlignment = EHorizontalAlignment::HAlign_Center;
				break;
			case ESlateBrushTileType::Both:
				break;
			}
		}

		UpdateOverlayAlignment();

		if( HorizontalAlignmentCombo.IsValid() )
		{
			HorizontalAlignmentCombo->SetSelectedItem( HorizontalAlignmentComboItems[ HorizontalAlignment ] );
			HorizontalAlignmentCombo->RefreshOptions();
			VerticalAlignmentCombo->SetSelectedItem( VerticalAlignmentComboItems[ VerticalAlignment ] );
			VerticalAlignmentCombo->RefreshOptions();
		}
	}

	/**
	 * Update the preview image overlay slot alignment
	 */
	void UpdateOverlayAlignment()
	{
		OverlaySlot->SetHorizontalAlignment( HorizontalAlignment );
		OverlaySlot->SetVerticalAlignment( VerticalAlignment );
	}

	/**
	 * Update the preview image size
	 */
	void UpdatePreviewImageSize()
	{
		PreviewImageSize = CachedTextureSize;
	}

	/**
	 * Called on change of Slate Brush DrawAs property
	 */
	void OnDrawAsChanged()
	{
		CachePropertyValues();
		
		if( CachedDrawAsType != ESlateBrushDrawType::Box && CachedDrawAsType != ESlateBrushDrawType::Border )
		{
			TArray<void*> RawData;

			if ( MarginProperty.IsValid() && MarginProperty->GetProperty() )
			{
				MarginProperty->AccessRawData( RawData );
				check( RawData[ 0 ] != NULL );
				*static_cast<FMargin*>(RawData[ 0 ]) = FMargin();
			}
		}
		else
		{
			CachedTilingType = ESlateBrushTileType::NoTile;
			FPropertyAccess::Result Result = TilingProperty->SetValue( static_cast<uint8>(ESlateBrushTileType::NoTile) );
			check( Result == FPropertyAccess::Success );
		}

		SetDefaultAlignment();
		UpdateMarginLinePositions();
	}

	/**
	 * Called on change of Slate Brush Tiling property
	 */
	void OnTilingChanged()
	{
		CachePropertyValues();
		SetDefaultAlignment();
		UpdateMarginLinePositions();
	}

	/**
	 * Called on change of Slate Brush ResourceObject property
	 */
	void OnBrushResourceChanged()
	{
		CachePropertyValues();
		UpdatePreviewImageSize();
		UpdateMarginLinePositions();
	}

	/**
	 * Called on change of Slate Brush ImageSize property
	 */
	void OnImageSizeChanged()
	{
		CachePropertyValues();
		UpdateMarginLinePositions();
	}

	/**
	 * Called on change of Slate Brush Margin property
	 */
	void OnMarginChanged()
	{
		CachePropertyValues();
		UpdateMarginLinePositions();
	}

	/**
	 * Get the preview width
	 */
	FOptionalSize GetPreviewWidth() const
	{
		return PreviewImageSize.X + ImagePadding * 2.0f;
	}

	/**
	 * Get the preview height
	 */
	FOptionalSize GetPreviewHeight() const
	{
		return PreviewImageSize.Y + ImagePadding * 2.0f;
	}

	/**
	 * Get the margin line visibility
	 */
	EVisibility GetMarginLineVisibility() const
	{
		return (CachedDrawAsType == ESlateBrushDrawType::Box || CachedDrawAsType == ESlateBrushDrawType::Border) ? EVisibility::Visible : EVisibility::Hidden;
	}

	/**
	 * Get the left margin line position
	 */
	FVector2D GetLeftMarginLinePosition() const
	{
		return FVector2D(MarginLinePositions[ MarginLine_Left ]);
	}

	/**
	 * Get the right margin line position
	 */
	FVector2D GetRightMarginLinePosition() const
	{
		return FVector2D(MarginLinePositions[ MarginLine_Right ]);
	}

	/**
	 * Get the top margin line position
	 */
	FVector2D GetTopMarginLinePosition() const
	{
		return FVector2D(MarginLinePositions[ MarginLine_Top ]);
	}

	/**
	 * Get the bottom margin line position
	 */
	FVector2D GetBottomMarginLinePosition() const
	{
		return FVector2D(MarginLinePositions[ MarginLine_Bottom ]);
	}

	/** Padding between the preview image and the checkerboard background */
	static const float ImagePadding;

	/** The thickness of the border for mouse hit testing */
	static const float BorderHitSize;

	/** Alignment combo items */
	TArray< TSharedPtr<EHorizontalAlignment> > HorizontalAlignmentComboItems;
	TArray< TSharedPtr<EVerticalAlignment> > VerticalAlignmentComboItems;

	/** Alignment combos */
	TSharedPtr< SComboBox< TSharedPtr<EHorizontalAlignment> > > HorizontalAlignmentCombo;
	TSharedPtr< SComboBox< TSharedPtr<EVerticalAlignment> > > VerticalAlignmentCombo;

	/** Overlay slot which contains the preview image */
	SOverlay::FOverlaySlot* OverlaySlot;

	/** Slate Brush properties */
	TSharedPtr<IPropertyHandle> DrawAsProperty;
	TSharedPtr<IPropertyHandle> TilingProperty;
	TSharedPtr<IPropertyHandle> ImageSizeProperty;
	TSharedPtr<IPropertyHandle> MarginProperty;
	TSharedPtr<IPropertyHandle> ResourceObjectProperty;
	TSharedPtr<IPropertyHandle> ImageTypeProperty;

	/** Cached Slate Brush property values */
	FVector2f CachedTextureSize;
	FVector2f CachedImageSizeValue;
	ESlateBrushDrawType::Type CachedDrawAsType;
	ESlateBrushTileType::Type CachedTilingType;
	FMargin CachedMarginPropertyValue;

	/** Preview alignment */
	EHorizontalAlignment HorizontalAlignment;
	EVerticalAlignment VerticalAlignment;

	/** Preview image size */
	FVector2f PreviewImageSize;

	/** Margin line positions */
	FVector2f MarginLinePositions[ MarginLine_Count ];

	/** The current widget zone the mouse is in */
	EWidgetZone MouseZone;

	/** If true the user is resizing the preview */
	bool bUserIsResizing;

	/** Preview resize anchor position */
	FVector2f ResizeAnchorPosition;

	/** Size of the preview image on begin of resize */
	FVector2f ResizeAnchorSize;
};

const float SSlateBrushPreview::ImagePadding = 5.0f;
const float SSlateBrushPreview::BorderHitSize = 8.0f;

// SSlateBrushStaticPreview
////////////////////////////////////////////////////////////////////////////////

class SSlateBrushStaticPreview : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSlateBrushStaticPreview) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InResourceObjectProperty)
	{
		ResourceObjectProperty = InResourceObjectProperty;

		UpdateBrush();

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SBorder)
				.Visibility(this, &SSlateBrushStaticPreview::GetPreviewVisibilityBorder)
				.BorderImage(this, &SSlateBrushStaticPreview::GetPropertyBrush)
				[
					SNew(SSpacer)
					.Size(FVector2f(1.f, 1.f))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(this, &SSlateBrushStaticPreview::GetScaledImageBrushWidth)
				.HeightOverride(TargetHeight)
				[
					SNew(SImage)
					.Visibility(this, &SSlateBrushStaticPreview::GetPreviewVisibilityImage)
					.Image(this, &SSlateBrushStaticPreview::GetPropertyBrush)
				]
			]
		];
	}

	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		UpdateBrush();
	}

private:

	const FSlateBrush* GetPropertyBrush() const
	{
		return &TemporaryBrush;
	}

	FOptionalSize GetScaledImageBrushWidth() const
	{
		if (TemporaryBrush.DrawAs == ESlateBrushDrawType::Image)
		{
			const FVector2f& Size = TemporaryBrush.ImageSize;
			if (Size.X > 0 && Size.Y > 0)
			{
				return Size.X * TargetHeight / Size.Y;
			}
		}

		// Default square
		return TargetHeight;
	}

	EVisibility GetPreviewVisibilityBorder() const
	{
		return TemporaryBrush.DrawAs == ESlateBrushDrawType::Image ? EVisibility::Collapsed : EVisibility::Visible;
	}

	EVisibility GetPreviewVisibilityImage() const
	{
		return TemporaryBrush.DrawAs == ESlateBrushDrawType::Image ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void UpdateBrush()
	{
		if (ResourceObjectProperty.IsValid() && ResourceObjectProperty->GetProperty())
		{
			TArray<void*> RawData;
			ResourceObjectProperty->AccessRawData(RawData);

			// RawData will be empty when creating a new Data Table, an idiosyncrasy
			// of the Data Table Editor...
			if (RawData.Num() > 0)
			{
				FSlateBrush* SlateBrush = static_cast<FSlateBrush*>(RawData[0]);
				if (SlateBrush)
				{
					TemporaryBrush = *SlateBrush;
				}				
			}
		}
	}

private:

	/**
	 * Temporary brush data used to store the structure returned from the property handy so that we have stable
	 * pointer to give to slate.
	 */
	FSlateBrush TemporaryBrush;

	TSharedPtr<IPropertyHandle> ResourceObjectProperty;

	static float TargetHeight;
};

float SSlateBrushStaticPreview::TargetHeight = 18.0f;

class SBrushResourceError : public SBorder
{
public:
	SLATE_BEGIN_ARGS( SBrushResourceError ) {}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		SBorder::Construct( SBorder::FArguments()
			.BorderBackgroundColor( FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor") )
			.BorderImage( FCoreStyle::Get().GetBrush("ErrorReporting.Box") )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding( FMargin(3,0) )
			[
				InArgs._Content.Widget
			]
		);
	}
};

// SBrushResourceObjectBox
////////////////////////////////////////////////////////////////////////////////

class SBrushResourceObjectBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SBrushResourceObjectBox ) {}
	
	SLATE_END_ARGS()

	struct FPropertyParams
	{
		TSharedPtr<IPropertyHandle> ResourceObjectProperty;
		TSharedPtr<IPropertyHandle> ResourceNameProperty;
		TSharedPtr<IPropertyHandle> ImageSizeProperty;
		TSharedPtr<IPropertyHandle> ImageTypeProperty;
		TSharedPtr<IPropertyHandle> DrawAsProperty;
	};

	void Construct(const FArguments& InArgs
		, IStructCustomizationUtils* StructCustomizationUtils
		, FPropertyParams InParams)
	{
		ResourceObjectProperty = InParams.ResourceObjectProperty;
		ResourceNameProperty = InParams.ResourceNameProperty;
		ImageSizeProperty = InParams.ImageSizeProperty;
		ImageTypeProperty = InParams.ImageTypeProperty;
		DrawAsProperty = InParams.DrawAsProperty;

		FSimpleDelegate OnBrushResourceChangedDelegate = FSimpleDelegate::CreateSP(this, &SBrushResourceObjectBox::OnBrushResourceChanged);
		ResourceObjectProperty->SetOnPropertyValueChanged(OnBrushResourceChangedDelegate);

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SObjectPropertyEntryBox)
				.PropertyHandle(InParams.ResourceObjectProperty)
				.ThumbnailPool(StructCustomizationUtils->GetThumbnailPool())
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 3.0f )
			[
				SAssignNew(ResourceError, SBrushResourceError )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.HAlign( HAlign_Left )
					[
						SNew( STextBlock )
						.Text( NSLOCTEXT("FSlateBrushStructCustomization", "ResourceErrorText", "This material does not use the UI material domain" ) )
					]
					+ SVerticalBox::Slot()
					.HAlign( HAlign_Left )
					[
						SAssignNew(ChangeDomainLink, SHyperlink )
						.Text( NSLOCTEXT("FSlateBrushStructCustomization", "ChangeMaterialDomain_ErrorMessage", "Change the Material Domain?" ) )
						.OnNavigate( this, &SBrushResourceObjectBox::OnErrorLinkClicked )
					]
					+ SVerticalBox::Slot()
					.HAlign( HAlign_Left )
					[
						SAssignNew(IsEngineMaterialError, STextBlock)
						.Text( NSLOCTEXT("FSlateBrushStructCustomization", "IsEngineMaterialErrorText", "Assign a parent UI Material" ) )
					]
				]
			]
		];

		UpdateResourceError();
	}

	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		UpdateResourceError();
	}

private:
	void OnBrushResourceChanged()
	{
		UObject* ResourceObject;
		FPropertyAccess::Result Result = ResourceObjectProperty->GetValue(ResourceObject);
		if ( Result == FPropertyAccess::Success )
		{
			TSharedPtr<IPropertyHandle> BrushHandle = ResourceObjectProperty->GetParentHandle();

			TArray<void*> RawBrushData;
			BrushHandle->AccessRawData(RawBrushData);
			for (int32 BrushIndex = 0; BrushIndex < RawBrushData.Num(); BrushIndex++)
			{
				FSlateBrush* TemporaryBrush = static_cast<FSlateBrush*>(RawBrushData[BrushIndex]);
				if (TemporaryBrush)
				{
					TemporaryBrush->InvalidateResourceHandle();
				}
			}

			using ImageSizeType = decltype(FSlateBrush::ImageSize);
			ImageSizeType CachedTextureSize = {};

			TArray<void*> RawData;
			ImageSizeProperty->AccessRawData(RawData);
			if ( RawData.Num() > 0 && RawData[0] != NULL )
			{
				CachedTextureSize = *static_cast<ImageSizeType*>( RawData[0] );
			}

			UTexture2D* BrushTexture = Cast<UTexture2D>(ResourceObject);
			if ( BrushTexture )
			{
				if ( BrushTexture->IsDefaultTexture() )
				{
					UTexture* const BaseTexture = BrushTexture;
					// GetSizeX/Y will return the incorrect value if this texture is being compiled so we need to wait for it here
					FTextureCompilingManager::Get().FinishCompilation( MakeArrayView(&BaseTexture, 1) );
				}
				CachedTextureSize = ImageSizeType(static_cast<float>(BrushTexture->GetSizeX()), static_cast<float>(BrushTexture->GetSizeY()));
			}
			else if ( ISlateTextureAtlasInterface* AtlasedTextureObject = Cast<ISlateTextureAtlasInterface>(ResourceObject) )
			{
				CachedTextureSize = AtlasedTextureObject->GetSlateAtlasData().GetSourceDimensions();
			}

			// Update the image size to match that of the incoming new texture.
			// TODO: Should we always do this?  Or should we avoid doing it if there's already some 'set value'
			// problem is we don't have a way to track that right now.
			ImageSizeProperty->SetValue(CachedTextureSize);

			// When you assign a resource object, if the current draw type is 'None' we go ahead and update it to 'Image'.
			// Also update ResourceName to be null (Object name will be used), & set ImageType
			if (ResourceObject)
			{
				TArray<FString> OutPerObjectValues;
				DrawAsProperty->GetPerObjectValues(OutPerObjectValues);

				TArray<FString> NewPerObjectValues;
				for (int32 ObjectIndex = 0; ObjectIndex < OutPerObjectValues.Num(); ObjectIndex++)
				{
					FString& ExistingValue = OutPerObjectValues[ObjectIndex];
					NewPerObjectValues.Add(ExistingValue == TEXT("NoDrawType") ? TEXT("Image") : ExistingValue);
				}

				DrawAsProperty->SetPerObjectValues(NewPerObjectValues);

				ResourceNameProperty->SetValue(NAME_None);

				static_assert(sizeof(decltype(FSlateBrush::ImageType)) == sizeof(uint8));
				uint8 Value = ESlateBrushImageType::FullColor;
				ImageTypeProperty->SetValue(Value);
			}
		}
	}

	void OnErrorLinkClicked()
	{
		UObject* Resource = nullptr;

		if( ResourceObjectProperty->GetValue(Resource) == FPropertyAccess::Success && Resource && Resource->IsA<UMaterialInterface>() )
		{
			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>( Resource );
			UMaterial* BaseMaterial = MaterialInterface->GetBaseMaterial();
			if ( BaseMaterial && !BaseMaterial->IsUIMaterial() )
			{
				FProperty* MaterialDomainProp = FindFProperty<FProperty>(UMaterial::StaticClass(), GET_MEMBER_NAME_CHECKED(UMaterial,MaterialDomain) );

				FScopedTransaction Transaction( FText::Format( NSLOCTEXT("FSlateBrushStructCustomization", "ChangeMaterialDomainTransaction", "Changed {0} to use the UI material domain"), FText::FromString( BaseMaterial->GetName() ) ) );
				FMaterialUpdateContext MaterialUpdateContext;
				MaterialUpdateContext.AddMaterial(BaseMaterial);
				BaseMaterial->PreEditChange( MaterialDomainProp );

				BaseMaterial->MaterialDomain = MD_UI;

				FPropertyChangedEvent ChangeEvent( MaterialDomainProp );
				BaseMaterial->PostEditChangeProperty( ChangeEvent );
			}
		}
	}

	void UpdateResourceError()
	{
		UObject* Resource = nullptr;

		if( ResourceObjectProperty->GetValue(Resource) == FPropertyAccess::Success && Resource && Resource->IsA<UMaterialInterface>() )
		{
			UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>( Resource );
			UMaterial* BaseMaterial = MaterialInterface->GetBaseMaterial();
			if( BaseMaterial && !BaseMaterial->IsUIMaterial() )
			{
				ResourceError->SetVisibility( EVisibility::Visible );

				// Special engine materials cannot change domain. This typically occurs when
				// the user creates or assigns a material instance with no parent material.
				// In this case, we warn the user rather than offer to change the domain.
				if (BaseMaterial->bUsedAsSpecialEngineMaterial)
				{
					ChangeDomainLink->SetVisibility( EVisibility::Collapsed );
					IsEngineMaterialError->SetVisibility( EVisibility::Visible );
				}
				else
				{
					ChangeDomainLink->SetVisibility( EVisibility::Visible );
					IsEngineMaterialError->SetVisibility( EVisibility::Collapsed );
				}
			}
			else
			{
				ResourceError->SetVisibility( EVisibility::Collapsed );
			}
		}
		else if( ResourceError->GetVisibility() != EVisibility::Collapsed )
		{
			ResourceError->SetVisibility( EVisibility::Collapsed );
		}
	}

private:
	TSharedPtr<IPropertyHandle> ResourceObjectProperty;
	TSharedPtr<IPropertyHandle> ResourceNameProperty;
	TSharedPtr<IPropertyHandle> ImageSizeProperty;
	TSharedPtr<IPropertyHandle> ImageTypeProperty;
	TSharedPtr<IPropertyHandle> DrawAsProperty;
	TSharedPtr<SBrushResourceError> ResourceError;
	TSharedPtr<SHyperlink> ChangeDomainLink;
	TSharedPtr<STextBlock> IsEngineMaterialError;
};

// FSlateBrushStructCustomization
////////////////////////////////////////////////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FSlateBrushStructCustomization::MakeInstance(bool bIncludePreview) 
{
	return MakeShareable( new FSlateBrushStructCustomization(bIncludePreview) );
}

FSlateBrushStructCustomization::FSlateBrushStructCustomization(bool bInIncludePreview)
	: bIncludePreview(bInIncludePreview)
{
}

void FSlateBrushStructCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	bool ShowOnlyInnerProperties = StructPropertyHandle->GetProperty()->HasMetaData(TEXT("ShowOnlyInnerProperties"));

	if ( !ShowOnlyInnerProperties )
	{
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SSlateBrushStaticPreview, StructPropertyHandle)
		];
	}
}

void FSlateBrushStructCustomization::CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	// Add the child properties
	ImageSizeProperty = StructPropertyHandle->GetChildHandle( TEXT("ImageSize") );
	DrawAsProperty = StructPropertyHandle->GetChildHandle( TEXT("DrawAs") );
	TSharedPtr<IPropertyHandle> TilingProperty = StructPropertyHandle->GetChildHandle( TEXT("Tiling") );
	TSharedPtr<IPropertyHandle> MarginProperty = StructPropertyHandle->GetChildHandle( TEXT("Margin") );
	TSharedPtr<IPropertyHandle> TintProperty = StructPropertyHandle->GetChildHandle( TEXT("TintColor") );
	TSharedPtr<IPropertyHandle> OutlineSettingsProperty = StructPropertyHandle->GetChildHandle(TEXT("OutlineSettings"));
	ResourceObjectProperty = StructPropertyHandle->GetChildHandle( TEXT("ResourceObject") );
	ResourceNameProperty = StructPropertyHandle->GetChildHandle(TEXT("ResourceName"));
	ImageTypeProperty = StructPropertyHandle->GetChildHandle(TEXT("ImageType"));
	
	FDetailWidgetRow& ResourceObjectRow = StructBuilder.AddProperty(ResourceObjectProperty.ToSharedRef()).CustomWidget();

	SBrushResourceObjectBox::FPropertyParams Params;
	Params.ResourceObjectProperty = ResourceObjectProperty;
	Params.ResourceNameProperty = ResourceNameProperty;
	Params.ImageSizeProperty = ImageSizeProperty;
	Params.ImageTypeProperty = ImageTypeProperty;
	Params.DrawAsProperty = DrawAsProperty;

	ResourceObjectRow
		.NameContent()
		[
			ResourceObjectProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(0.0f)
		[
			SNew(SBrushResourceObjectBox, &StructCustomizationUtils, Params)
		];

	// Add the image size property with custom reset delegates that also affect the child properties (the components)
	const bool bOverrideDefaultOnVectorChildren = true;
	StructBuilder.AddProperty( ImageSizeProperty.ToSharedRef() )
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateSP(this, &FSlateBrushStructCustomization::IsImageSizeResetToDefaultVisible),
		FResetToDefaultHandler::CreateSP(this, &FSlateBrushStructCustomization::OnImageSizeResetToDefault),
		bOverrideDefaultOnVectorChildren));

	StructBuilder.AddProperty( TintProperty.ToSharedRef() );
	StructBuilder.AddProperty( DrawAsProperty.ToSharedRef() );
	StructBuilder.AddProperty( OutlineSettingsProperty.ToSharedRef() )
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP( this, &FSlateBrushStructCustomization::GetOutlineSettingsPropertyVisibility ) ) );
	StructBuilder.AddProperty( TilingProperty.ToSharedRef() )
	.Visibility( TAttribute<EVisibility>::Create( TAttribute<EVisibility>::FGetter::CreateSP( this, &FSlateBrushStructCustomization::GetTilingPropertyVisibility ) ) );
	StructBuilder.AddProperty( MarginProperty.ToSharedRef() )
	.Visibility( TAttribute<EVisibility>::Create( TAttribute<EVisibility>::FGetter::CreateSP( this, &FSlateBrushStructCustomization::GetMarginPropertyVisibility ) ) );

	// Don't show the preview area when in slim view mode.
	if ( bIncludePreview )
	{
		// Create the Slate Brush Preview widget and add the Preview group
		TArray<void*> RawData;
		StructPropertyHandle->AccessRawData(RawData);

		// Can only display the preview with one brush
		if ( RawData.Num() == 1 )
		{
			FSlateBrush* Brush = static_cast<FSlateBrush*>( RawData[0] );

			TSharedRef<SSlateBrushPreview> Preview = SNew(SSlateBrushPreview)
				.DrawAsProperty(DrawAsProperty)
				.TilingProperty(TilingProperty)
				.ImageSizeProperty(ImageSizeProperty)
				.MarginProperty(MarginProperty)
				.ResourceObjectProperty(ResourceObjectProperty)
				.ImageTypeProperty(ImageTypeProperty)
				.SlateBrush(Brush);

			IDetailGroup& PreviewGroup = StructBuilder.AddGroup(TEXT("Preview"), FText::GetEmpty());

			PreviewGroup
				.HeaderRow()
				.NameContent()
				[
					StructPropertyHandle->CreatePropertyNameWidget(NSLOCTEXT("UnrealEd", "Preview", "Preview"))
				]
			.ValueContent()
				.MinDesiredWidth(1)
				.MaxDesiredWidth(4096)
				[
					Preview->GenerateAlignmentComboBoxes()
				];

			PreviewGroup
				.AddWidgetRow()
				.ValueContent()
				.MinDesiredWidth(1)
				.MaxDesiredWidth(4096)
				[
					Preview
				];
		}
	}
}

EVisibility FSlateBrushStructCustomization::GetOutlineSettingsPropertyVisibility() const
{
	uint8 DrawAsType;
	FPropertyAccess::Result Result = DrawAsProperty->GetValue(DrawAsType);

	return (Result == FPropertyAccess::MultipleValues || DrawAsType == ESlateBrushDrawType::RoundedBox) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FSlateBrushStructCustomization::GetTilingPropertyVisibility() const
{
	uint8 DrawAsType;
	FPropertyAccess::Result Result = DrawAsProperty->GetValue( DrawAsType );

	return (Result == FPropertyAccess::MultipleValues || DrawAsType == ESlateBrushDrawType::Image) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FSlateBrushStructCustomization::GetMarginPropertyVisibility() const
{
	uint8 DrawAsType;
	FPropertyAccess::Result Result = DrawAsProperty->GetValue( DrawAsType );

	return (Result == FPropertyAccess::MultipleValues || DrawAsType == ESlateBrushDrawType::Box || DrawAsType == ESlateBrushDrawType::Border) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FSlateBrushStructCustomization::IsImageSizeResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	UObject* ResourceObject;
	if (FPropertyAccess::Success == ResourceObjectProperty->GetValue(ResourceObject) && ResourceObject)
	{
		// get texture size from ResourceObjectProperty and compare to image size prop value
		const FVector2f SizeDefault = GetDefaultImageSize();

		FVector2f Size;
		{
			FVector2D SizeAsDouble;
			ImageSizeProperty->GetValue(SizeAsDouble);
			Size = UE::Slate::CastToVector2f(SizeAsDouble);
		}

		if (PropertyHandle->GetProperty() == ImageSizeProperty->GetProperty())
		{
			// reseting the whole vector
			return SizeDefault != Size;
		}
		else if (PropertyHandle->GetProperty() == ImageSizeProperty->GetChildHandle(0)->GetProperty())	// X
		{
			// reseting the vector.X
			return SizeDefault.X != Size.X;
		}
		else if (PropertyHandle->GetProperty() == ImageSizeProperty->GetChildHandle(1)->GetProperty())	// Y
		{
			// reseting the vector.Y
			return SizeDefault.Y != Size.Y;
		}

		ensureMsgf(false, TEXT("Property handle mismatch in brush size FVector2D struct"));
		return false;
	}

	// Fall back to default handler
	return PropertyHandle->DiffersFromDefault();
}

void FSlateBrushStructCustomization::OnImageSizeResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	UObject* ResourceObject;
	if (FPropertyAccess::Success == ResourceObjectProperty->GetValue(ResourceObject) && ResourceObject)
	{
		// Set image size prop value to the texture size in ResourceObjectProperty
		const FVector2f SizeDefault = GetDefaultImageSize();

		if (PropertyHandle->GetProperty() == ImageSizeProperty->GetProperty())
		{
			// reseting the whole vector
			PropertyHandle->SetValue(FVector2D(SizeDefault));
		}
		else if (PropertyHandle->GetProperty() == ImageSizeProperty->GetChildHandle(0)->GetProperty())	// X
		{
			// reseting the vector.X
			PropertyHandle->SetValue(SizeDefault.X);
		}
		else if (PropertyHandle->GetProperty() == ImageSizeProperty->GetChildHandle(1)->GetProperty())	// Y
		{
			// reseting the vector.Y
			PropertyHandle->SetValue(SizeDefault.Y);
		}
		else
		{
			ensureMsgf(false, TEXT("Property handle mismatch in brush size FVector2D struct"));
		}
	}	
	else
	{
		// Fall back to default handler. 
		PropertyHandle->ResetToDefault();
	}
}

FVector2f FSlateBrushStructCustomization::GetDefaultImageSize() const
{
	// Custom default behavior using the texture's size, if one is set as the resource object
	UObject* ResourceObject;
	if (FPropertyAccess::Success == ResourceObjectProperty->GetValue(ResourceObject))
	{
		if ( UTexture2D* Texture = Cast<UTexture2D>(ResourceObject) )
		{
			if ( Texture->IsDefaultTexture() )
			{
				// GetSizeX/Y will return the incorrect value if this texture is being compiled so we need to wait for it here
				UTexture* const BaseTexture = Texture;
				FTextureCompilingManager::Get().FinishCompilation(MakeArrayView(&BaseTexture, 1));
			}
			return FVector2f(static_cast<float>(Texture->GetSizeX()), static_cast<float>(Texture->GetSizeY()));
		}
		else if ( ISlateTextureAtlasInterface* AtlasedTextureObject = Cast<ISlateTextureAtlasInterface>(ResourceObject) )
		{
			return UE::Slate::CastToVector2f(AtlasedTextureObject->GetSlateAtlasData().GetSourceDimensions());
		}
	}

	// Fall back on the standard default size for brush images
	return FVector2f(SlateBrushDefs::DefaultImageSize, SlateBrushDefs::DefaultImageSize);
}
