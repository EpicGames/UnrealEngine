// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateColorCustomization.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;

TSharedRef<IPropertyTypeCustomization> FSlateColorCustomization::MakeInstance() 
{
	return MakeShareable( new FSlateColorCustomization() );
}

void FSlateColorCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	static const FName ColorUseRuleKey(TEXT("ColorUseRule"));
	static const FName SpecifiedColorKey(TEXT("SpecifiedColor"));

	StructPropertyHandle = InStructPropertyHandle;

	ColorRuleHandle = InStructPropertyHandle->GetChildHandle(ColorUseRuleKey);
	SpecifiedColorHandle = InStructPropertyHandle->GetChildHandle(SpecifiedColorKey);

	check(ColorRuleHandle.IsValid());
	check(SpecifiedColorHandle.IsValid());

	ColorRuleHandle->MarkHiddenByCustomization();
	SpecifiedColorHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(SharedThis(this), &FSlateColorCustomization::OnValueChanged));

	FColorStructCustomization::CustomizeHeader(SpecifiedColorHandle.ToSharedRef(), InHeaderRow, StructCustomizationUtils);

	// Slate brushes always default to sRGB mode.
	sRGBOverride = true;
}

void FSlateColorCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row)
{
	// NOTE: Ignore InStructPropertyHandle, it's going to be the specified color handle that we passed to the color customization base class.

	Row
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(StructPropertyHandle->GetPropertyDisplayName())
			.ToolTipText(StructPropertyHandle->GetToolTipText())
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				CreateColorWidget(StructPropertyHandle)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FSlateColorCustomization::GetForegroundCheckState)
				.OnCheckStateChanged(this, &FSlateColorCustomization::HandleForegroundChanged)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(NSLOCTEXT("SlateColorCustomization", "Inherit", "Inherit"))
					.ToolTipText(NSLOCTEXT("SlateColorCustomization", "InheritToolTip", "Uses the foreground color inherited down the widget hierarchy"))
				]
			]
		];
}

void FSlateColorCustomization::OnValueChanged()
{
	ColorRuleHandle->SetValueFromFormattedString(TEXT("UseColor_Specified"));
}

ECheckBoxState FSlateColorCustomization::GetForegroundCheckState() const
{
	FString ColorRuleValue;
	ColorRuleHandle->GetValueAsFormattedString(ColorRuleValue);

	if ( ColorRuleValue == TEXT("UseColor_Foreground") )
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void FSlateColorCustomization::HandleForegroundChanged(ECheckBoxState CheckedState)
{
	if ( CheckedState == ECheckBoxState::Checked )
	{
		ColorRuleHandle->SetValueFromFormattedString(TEXT("UseColor_Foreground"));
	}
	else
	{
		ColorRuleHandle->SetValueFromFormattedString(TEXT("UseColor_Specified"));
	}
}
