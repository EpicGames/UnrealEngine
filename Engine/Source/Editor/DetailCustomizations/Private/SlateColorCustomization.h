// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/ColorStructCustomization.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;

class FSlateColorCustomization
	: public FColorStructCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override;

private:

	/** Called when the value is changed in the property editor. */
	virtual void OnValueChanged();

	ECheckBoxState GetForegroundCheckState() const;

	void HandleForegroundChanged(ECheckBoxState CheckedState);

private:

	/** slate color struct handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	TSharedPtr<IPropertyHandle> ColorRuleHandle;
	TSharedPtr<IPropertyHandle> SpecifiedColorHandle;
};
