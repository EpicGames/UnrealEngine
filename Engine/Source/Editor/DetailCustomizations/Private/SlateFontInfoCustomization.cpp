// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/SlateFontInfoCustomization.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/Font.h"
#include "Engine/UserInterfaceSettings.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IDocumentation.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SlateGlobals.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#include "Internationalization/FastDecimalFormat.h"

class SWidget;

#define LOCTEXT_NAMESPACE "SlateFontInfo"

TSharedRef<IPropertyTypeCustomization> FSlateFontInfoStructCustomization::MakeInstance() 
{
	return MakeShareable(new FSlateFontInfoStructCustomization());
}

void FSlateFontInfoStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	static const FName FontObjectPropertyName = GET_MEMBER_NAME_CHECKED(FSlateFontInfo, FontObject);
	static const FName TypefaceFontNamePropertyName = GET_MEMBER_NAME_CHECKED(FSlateFontInfo, TypefaceFontName);
	static const FName SizePropertyName = GET_MEMBER_NAME_CHECKED(FSlateFontInfo, Size);

	FontObjectProperty = StructPropertyHandle->GetChildHandle(FontObjectPropertyName);
	check(FontObjectProperty.IsValid());

	TypefaceFontNameProperty = StructPropertyHandle->GetChildHandle(TypefaceFontNamePropertyName);
	check(TypefaceFontNameProperty.IsValid());

	FontSizeProperty = StructPropertyHandle->GetChildHandle(SizePropertyName);
	check(FontSizeProperty.IsValid());

	InHeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(0)
	.MaxDesiredWidth(0)
	[
		InStructPropertyHandle->CreatePropertyValueWidget()
	];
}

void FSlateFontInfoStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	IDetailPropertyRow& FontObjectRow = InStructBuilder.AddProperty(FontObjectProperty.ToSharedRef());

	FontObjectRow.CustomWidget()
		.NameContent()
		[
			FontObjectProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(300.f)
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(FontObjectProperty)
			.AllowedClass(UFont::StaticClass())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FSlateFontInfoStructCustomization::OnFilterFontAsset))
			.OnObjectChanged(this, &FSlateFontInfoStructCustomization::OnFontChanged)
			.DisplayUseSelected(true)
			.DisplayBrowse(true)
		];

	IDetailPropertyRow& TypefaceRow = InStructBuilder.AddProperty(TypefaceFontNameProperty.ToSharedRef());

	TypefaceRow.CustomWidget()
	.NameContent()
	[
		TypefaceFontNameProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(FontEntryCombo, SComboBox<TSharedPtr<FName>>)
		.OptionsSource(&FontEntryComboData)
		.IsEnabled(this, &FSlateFontInfoStructCustomization::IsFontEntryComboEnabled)
		.OnComboBoxOpening(this, &FSlateFontInfoStructCustomization::OnFontEntryComboOpening)
		.OnSelectionChanged(this, &FSlateFontInfoStructCustomization::OnFontEntrySelectionChanged)
		.OnGenerateWidget(this, &FSlateFontInfoStructCustomization::MakeFontEntryWidget)
		[
			SNew(STextBlock)
			.Text(this, &FSlateFontInfoStructCustomization::GetFontEntryComboText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];

	AddFontSizeProperty(InStructBuilder);

	InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, LetterSpacing)).ToSharedRef());
	
	InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, SkewAmount)).ToSharedRef());

	const TSharedRef<IPropertyHandle> MonospacingHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, bForceMonospaced)).ToSharedRef();
	InStructBuilder.AddProperty(MonospacingHandle);

	const TSharedRef<IPropertyHandle> MonospacingWidthHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, MonospacedWidth)).ToSharedRef();
	InStructBuilder.AddProperty(MonospacingWidthHandle);
	
	// Set an initial "sensible" value based on the current font size. Won't run if value is already non-default/zero 
	MonospacingHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, MonospacingWidthHandle]()
	{
		if (!MonospacingWidthHandle->DiffersFromDefault())
		{
			float FontSizeValue;
			FontSizeProperty->GetValue(FontSizeValue);
			
			MonospacingWidthHandle->SetValue(static_cast<int32>(FontSizeValue));
		}
	}));

	InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, FontMaterial)).ToSharedRef());

	InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, bMaterialIsStencil)).ToSharedRef());

	InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSlateFontInfo, OutlineSettings)).ToSharedRef());
}

void FSlateFontInfoStructCustomization::AddFontSizeProperty(IDetailChildrenBuilder& InStructBuilder)
{
	TSharedRef<IPropertyHandle> FontSizePropertyRef = FontSizeProperty.ToSharedRef();

	auto GetFloatMetaDataFromKey = [FontSizePropertyRef](const FName& Key) -> const TOptional<float>
	{
		const FString* InstanceValue = FontSizePropertyRef->GetInstanceMetaData(Key);
		const FString MetaDataValueString = (InstanceValue != nullptr) ? *InstanceValue : FontSizePropertyRef->GetMetaData(Key);
		if (MetaDataValueString.Len())
		{
			float FloatValue;
			LexFromString(FloatValue, *MetaDataValueString);
			return FloatValue;
		}

		return TOptional<float>();
	};

	TOptional<float> MinValue = GetFloatMetaDataFromKey("ClampMin");
	TOptional<float> MaxValue = GetFloatMetaDataFromKey("ClampMax");

	IDetailPropertyRow& FontSizeRow = InStructBuilder.AddProperty(FontSizePropertyRef);
	FontSizeRow.CustomWidget()
	.NameContent()
	[
		FontSizePropertyRef->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SNumericEntryBox<float>)
		.Value(this, &FSlateFontInfoStructCustomization::OnFontSizeGetValue)
		.OnValueChanged(this, &FSlateFontInfoStructCustomization::OnFontSizeValueChanged)
		.OnValueCommitted(this, &FSlateFontInfoStructCustomization::OnFontSizeValueCommitted)
		.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
		.OnBeginSliderMovement(this, &FSlateFontInfoStructCustomization::OnFontSizeBeginSliderMovement)
		.OnEndSliderMovement(this, &FSlateFontInfoStructCustomization::OnFontSizeEndSliderMovement)
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.MinFractionalDigits(0)
		.MaxFractionalDigits(2)
		.MinValue(MinValue)
		.MaxValue(MaxValue)
		.MinSliderValue(MinValue)
		.MaxSliderValue(MaxValue)
		.Delta(1.0f)
		.AllowWheel(true)
		.WheelStep(1.0f)
		.AllowSpin(FontSizePropertyRef->GetNumPerObjectValues() == 1) //Don't allow spin for multiple value select. Allowing it would result in the widget background not being displayed.
		.IsEnabled(this, &FSlateFontInfoStructCustomization::IsFontSizeEnabled)
		.ToolTip(IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &FSlateFontInfoStructCustomization::GetFontSizeTooltipText),
													  nullptr,
													  TEXT("Shared/Types/FSlateFontInfo"),
													  TEXT("Size")))
	];
}

bool FSlateFontInfoStructCustomization::IsFontSizeEnabled() const
{
	return FontSizeProperty && !FontSizeProperty->IsEditConst();
}

FText FSlateFontInfoStructCustomization::GetFontSizeTooltipText() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();

	FFormatNamedArguments Args;
	Args.Add(TEXT("DPI"), UISettings->GetFontDPIDisplayString());
	return FText::Format(LOCTEXT("FontSizeToolTip", "Size of the font in points.\nCurrent font resolution : {DPI}"), Args);
}

float FSlateFontInfoStructCustomization::ConvertFontSizeFromNativeToDisplay(float NativeFontSize)
{
	float DisplaySize = 0.0f;
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings)
	{
		DisplaySize = UISettings->ConvertFontSizeFromNativeToDisplay(NativeFontSize);
	}
	return DisplaySize;
}

float FSlateFontInfoStructCustomization::ConvertFontSizeFromDisplayToNative(float DisplayFontSize)
{
	float NativeSize = 0.0f;
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings)
	{
		NativeSize = UISettings->ConvertFontSizeFromDisplayToNative(DisplayFontSize);
	}
	return NativeSize;
}

TOptional<float> FSlateFontInfoStructCustomization::OnFontSizeGetValue() const
{
	float value(0.0f);
	const TSharedRef< IPropertyHandle > PropertyHandle = FontSizeProperty.ToSharedRef();

	if (PropertyHandle->GetValue(value) == FPropertyAccess::Success)
	{
		return ConvertFontSizeFromNativeToDisplay(value);
	}

	// Return an unset value so it displays the "multiple values" indicator instead
	return TOptional<float>();
}

void FSlateFontInfoStructCustomization::OnFontSizeValueChanged(float NewDisplayValue)
{
	if (!bIsUsingSlider)
		return;

	const TSharedRef< IPropertyHandle > PropertyHandle = FontSizeProperty.ToSharedRef();

	float OrgValue(0.0f);
	const float NativeFontSize = ConvertFontSizeFromDisplayToNative(NewDisplayValue);

	if (PropertyHandle->GetValue(OrgValue) != FPropertyAccess::Fail)
	{
		// Value hasn't changed, so lets return now
		if (OrgValue == NativeFontSize)
		{
			return;
		}
	}

	// We don't create a transaction for each property change when using the slider. Only once when the slider first is moved
	EPropertyValueSetFlags::Type Flags = (EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
	PropertyHandle->SetValue(NativeFontSize, Flags);
}

void FSlateFontInfoStructCustomization::OnFontSizeValueCommitted(float NewDisplayValue, ETextCommit::Type CommitInfo)
{
	const TSharedRef< IPropertyHandle > PropertyHandle = FontSizeProperty.ToSharedRef();
	const float NativeFontSize = ConvertFontSizeFromDisplayToNative(NewDisplayValue);

	float OrgValue(0.0f);
	if (bIsUsingSlider || (PropertyHandle->GetValue(OrgValue) == FPropertyAccess::Fail || OrgValue != NativeFontSize))
	{

		PropertyHandle->SetValue(NativeFontSize);
		LastSliderFontSizeCommittedValue = NativeFontSize;
	}
}

void FSlateFontInfoStructCustomization::OnFontSizeBeginSliderMovement()
{
	bIsUsingSlider = true;

	const TSharedRef< IPropertyHandle > PropertyHandle = FontSizeProperty.ToSharedRef();
	PropertyHandle->GetValue(LastSliderFontSizeCommittedValue);

	GEditor->BeginTransaction(LOCTEXT("UpdateFontSizeTransaction", "Edit font size"));
}

void FSlateFontInfoStructCustomization::OnFontSizeEndSliderMovement(float NewDisplayValue)
{
	const float NativeFontSize = ConvertFontSizeFromDisplayToNative(NewDisplayValue);
	bIsUsingSlider = false;

	// When the slider end, we may have not called SetValue(NewValue) without the InteractiveChange|NotTransactable flags.
	//That prevents some transaction and callback to be triggered like the NotifyHook.
	if (LastSliderFontSizeCommittedValue != NativeFontSize)
	{
		const TSharedRef< IPropertyHandle > PropertyHandle = FontSizeProperty.ToSharedRef();
		PropertyHandle->SetValue(NativeFontSize);
	}
	else
	{
		GEditor->EndTransaction();
	}
}

bool FSlateFontInfoStructCustomization::OnFilterFontAsset(const FAssetData& InAssetData)
{
	// We want to filter font assets that aren't valid to use with Slate/UMG
	return Cast<const UFont>(InAssetData.GetAsset())->FontCacheType != EFontCacheType::Runtime;
}

void FSlateFontInfoStructCustomization::OnFontChanged(const FAssetData& InAssetData)
{
	const UFont* const FontAsset = Cast<const UFont>(InAssetData.GetAsset());
	const FCompositeFont* CompositeFont = FontAsset ? FontAsset->GetCompositeFont() : nullptr;
	const FName FirstFontName = (CompositeFont && CompositeFont->DefaultTypeface.Fonts.Num()) ? CompositeFont->DefaultTypeface.Fonts[0].Name : NAME_None;

	TArray<FSlateFontInfo*> SlateFontInfoStructs = GetFontInfoBeingEdited();
	for(FSlateFontInfo* FontInfo : SlateFontInfoStructs)
	{
		// The font has been updated in the editor, so clear the non-UObject pointer so that the two don't conflict
		FontInfo->CompositeFont.Reset();

		// We've changed (or cleared) the font asset, so make sure and update the typeface entry name being used by the font info
		TypefaceFontNameProperty->SetValue(FirstFontName);
	}

	if(!FontAsset)
	{
		const FString PropertyPath = FontObjectProperty->GeneratePathToProperty();
		TArray<UObject*> PropertyOuterObjects;
		FontObjectProperty->GetOuterObjects(PropertyOuterObjects);
		for(const UObject* OuterObject : PropertyOuterObjects)
		{
			UE_LOGF(LogSlate, Warning, "FSlateFontInfo property '%ls' on object '%ls' was set to use a null UFont. Slate will be forced to use the fallback font path which may be slower.", *PropertyPath, *OuterObject->GetPathName());
		}
	}
}

bool FSlateFontInfoStructCustomization::IsFontEntryComboEnabled() const
{
	if (TypefaceFontNameProperty->IsEditConst())
	{
		return false;
	}
	
	TArray<const FSlateFontInfo*> SlateFontInfoStructs = GetFontInfoBeingEdited();
	if(SlateFontInfoStructs.Num() == 0)
	{
		return false;
	}

	const FSlateFontInfo* const FirstSlateFontInfo = SlateFontInfoStructs[0];
	const UFont* const FontObject = Cast<const UFont>(FirstSlateFontInfo->FontObject);
	if(!FontObject)
	{
		return false;
	}

	// Only let people pick an entry if every struct being edited is using the same font object
	for(int32 FontInfoIndex = 1; FontInfoIndex < SlateFontInfoStructs.Num(); ++FontInfoIndex)
	{
		const FSlateFontInfo* const OtherSlateFontInfo = SlateFontInfoStructs[FontInfoIndex];
		const UFont* const OtherFontObject = Cast<const UFont>(OtherSlateFontInfo->FontObject);
		if(FontObject != OtherFontObject)
		{
			return false;
		}
	}

	return true;
}

void FSlateFontInfoStructCustomization::OnFontEntryComboOpening()
{
	TArray<FSlateFontInfo*> SlateFontInfoStructs = GetFontInfoBeingEdited();

	FontEntryComboData.Empty();

	if(SlateFontInfoStructs.Num() > 0)
	{
		const FSlateFontInfo* const FirstSlateFontInfo = SlateFontInfoStructs[0];
		const UFont* const FontObject = Cast<const UFont>(FirstSlateFontInfo->FontObject);
		check(FontObject);

		const FName ActiveFontEntry = GetActiveFontEntry();
		TSharedPtr<FName> SelectedNamePtr;

		if (const FCompositeFont* CompositeFont = FontObject->GetCompositeFont())
		{
			for (const FTypefaceEntry& TypefaceEntry : CompositeFont->DefaultTypeface.Fonts)
			{
				TSharedPtr<FName> NameEntryPtr = MakeShareable(new FName(TypefaceEntry.Name));
				FontEntryComboData.Add(NameEntryPtr);

				if (!TypefaceEntry.Name.IsNone() && TypefaceEntry.Name == ActiveFontEntry)
				{
					SelectedNamePtr = NameEntryPtr;
				}
			}
		}

		FontEntryComboData.Sort([](const TSharedPtr<FName>& One, const TSharedPtr<FName>& Two) -> bool
		{
			return One->ToString() < Two->ToString();
		});

		FontEntryCombo->ClearSelection();
		FontEntryCombo->RefreshOptions();
		FontEntryCombo->SetSelectedItem(SelectedNamePtr);
	}
	else
	{
		FontEntryCombo->ClearSelection();
		FontEntryCombo->RefreshOptions();
	}
}

void FSlateFontInfoStructCustomization::OnFontEntrySelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type)
{
	if(InNewSelection.IsValid())
	{
		TArray<FSlateFontInfo*> SlateFontInfoStructs = GetFontInfoBeingEdited();
		if(SlateFontInfoStructs.Num() > 0)
		{
			const FSlateFontInfo* const FirstSlateFontInfo = SlateFontInfoStructs[0];
			if(FirstSlateFontInfo->TypefaceFontName != *InNewSelection)
			{
				TypefaceFontNameProperty->SetValue(*InNewSelection);
			}
		}
	}
}

TSharedRef<SWidget> FSlateFontInfoStructCustomization::MakeFontEntryWidget(TSharedPtr<FName> InFontEntry)
{
	return
		SNew(STextBlock)
		.Text(FText::FromName(*InFontEntry))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

FText FSlateFontInfoStructCustomization::GetFontEntryComboText() const
{
	return FText::FromName(GetActiveFontEntry());
}

FName FSlateFontInfoStructCustomization::GetActiveFontEntry() const
{
	TArray<const FSlateFontInfo*> SlateFontInfoStructs = GetFontInfoBeingEdited();
	if(SlateFontInfoStructs.Num() > 0)
	{
		const FSlateFontInfo* const FirstSlateFontInfo = SlateFontInfoStructs[0];
		const UFont* const FontObject = Cast<const UFont>(FirstSlateFontInfo->FontObject);
		if(FontObject)
		{
			if (const FCompositeFont* CompositeFont = FontObject->GetCompositeFont())
			{
				return (FirstSlateFontInfo->TypefaceFontName.IsNone() && CompositeFont->DefaultTypeface.Fonts.Num())
					? CompositeFont->DefaultTypeface.Fonts[0].Name
					: FirstSlateFontInfo->TypefaceFontName;
			}
		}
	}

	return NAME_None;
}

TArray<FSlateFontInfo*> FSlateFontInfoStructCustomization::GetFontInfoBeingEdited()
{
	TArray<FSlateFontInfo*> SlateFontInfoStructs;

	if(StructPropertyHandle->IsValidHandle())
	{
		TArray<void*> StructPtrs;
		StructPropertyHandle->AccessRawData(StructPtrs);
		SlateFontInfoStructs.Reserve(StructPtrs.Num());

		for(auto It = StructPtrs.CreateConstIterator(); It; ++It)
		{
			void* RawPtr = *It;
			if(RawPtr)
			{
				FSlateFontInfo* const SlateFontInfo = reinterpret_cast<FSlateFontInfo*>(RawPtr);
				SlateFontInfoStructs.Add(SlateFontInfo);
			}
		}
	}

	return SlateFontInfoStructs;
}

TArray<const FSlateFontInfo*> FSlateFontInfoStructCustomization::GetFontInfoBeingEdited() const
{
	TArray<const FSlateFontInfo*> SlateFontInfoStructs;

	if(StructPropertyHandle->IsValidHandle())
	{
		TArray<const void*> StructPtrs;
		StructPropertyHandle->AccessRawData(StructPtrs);
		SlateFontInfoStructs.Reserve(StructPtrs.Num());

		for(auto It = StructPtrs.CreateConstIterator(); It; ++It)
		{
			const void* RawPtr = *It;
			if(RawPtr)
			{
				const FSlateFontInfo* const SlateFontInfo = reinterpret_cast<const FSlateFontInfo*>(RawPtr);
				SlateFontInfoStructs.Add(SlateFontInfo);
			}
		}
	}

	return SlateFontInfoStructs;
}

#undef LOCTEXT_NAMESPACE
