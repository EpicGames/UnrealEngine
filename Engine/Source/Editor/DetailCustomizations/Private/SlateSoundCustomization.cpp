// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateSoundCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Sound/SlateSound.h"
#include "Sound/SoundBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailChildrenBuilder;

TSharedRef<IPropertyTypeCustomization> FSlateSoundStructCustomization::MakeInstance() 
{
	return MakeShareable(new FSlateSoundStructCustomization());
}

void FSlateSoundStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> ResourceObjectProperty = StructPropertyHandle->GetChildHandle(TEXT("ResourceObject"));
	check(ResourceObjectProperty.IsValid());

	TArray<void*> StructPtrs;
	StructPropertyHandle->AccessRawData(StructPtrs);
	for(auto It = StructPtrs.CreateConstIterator(); It; ++It)
	{
		FSlateSound* const SlateSound = reinterpret_cast<FSlateSound*>(*It);
		SlateSoundStructs.Add(SlateSound);
	}

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(ResourceObjectProperty)
		.AllowedClass(USoundBase::StaticClass())
		.OnObjectChanged(this, &FSlateSoundStructCustomization::OnObjectChanged)
	];
}

void FSlateSoundStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FSlateSoundStructCustomization::OnObjectChanged(const FAssetData&)
{
	// The object has been updated in the editor, so strip out the legacy data now so that the two don't conflict
	for(auto It = SlateSoundStructs.CreateConstIterator(); It; ++It)
	{
		FSlateSound* const SlateSound = *It;
		SlateSound->StripLegacyData_DEPRECATED();
	}
}
