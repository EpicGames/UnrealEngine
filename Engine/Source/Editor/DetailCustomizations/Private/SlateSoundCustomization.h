// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

struct FAssetData;
struct FSlateSound;

/** Customize the appearance of an FSlateSound */
class FSlateSoundStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	/** Called when the resource object used by this FSlateSound has been changed */
	void OnObjectChanged(const FAssetData&);

	/** Array of FSlateSound instances this customization is currently editing */
	TArray<FSlateSound*> SlateSoundStructs;
};
