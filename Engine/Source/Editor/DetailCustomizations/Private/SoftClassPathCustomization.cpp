// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftClassPathCustomization.h"

#include "DetailWidgetRow.h"
#include "EditorClassUtils.h"
#include "HAL/PlatformCrt.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailChildrenBuilder;

void FSoftClassPathCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	const FString& MetaClassName = PropertyHandle->GetMetaData("MetaClass");
	const FString& RequiredInterfaceName = PropertyHandle->GetMetaData("RequiredInterface"); // This was the old name, switch to MustImplement to synchronize with class property
	const FString& MustImplementName = PropertyHandle->GetMetaData("MustImplement");
	TArray<const UClass*> AllowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(PropertyHandle->GetMetaData("AllowedClasses"));
	TArray<const UClass*> DisallowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(PropertyHandle->GetMetaData("DisallowedClasses"));
	const bool bAllowAbstract = PropertyHandle->HasMetaData("AllowAbstract");
	const bool bIsBlueprintBaseOnly = PropertyHandle->HasMetaData("IsBlueprintBaseOnly") || PropertyHandle->HasMetaData("BlueprintBaseOnly");
	const bool bAllowNone = !(PropertyHandle->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);
	const bool bShowTreeView = PropertyHandle->HasMetaData("ShowTreeView");
	const bool bHideViewOptions = PropertyHandle->HasMetaData("HideViewOptions");
	const bool bShowDisplayNames = PropertyHandle->HasMetaData("ShowDisplayNames");
	
	const UClass* const MetaClass = !MetaClassName.IsEmpty()
		? FEditorClassUtils::GetClassFromString(MetaClassName)
		: UObject::StaticClass();
	const UClass* const RequiredInterface = !RequiredInterfaceName.IsEmpty()
		? FEditorClassUtils::GetClassFromString(RequiredInterfaceName)
		: FEditorClassUtils::GetClassFromString(MustImplementName);

	HeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0.0f)
	[
		// Add a class entry box.  Even though this isn't an class entry, we will simulate one
		SNew(SClassPropertyEntryBox)
			.MetaClass(MetaClass)
			.RequiredInterface(RequiredInterface)
			.AllowedClasses(MoveTemp(AllowedClasses))
			.DisallowedClasses(MoveTemp(DisallowedClasses))
			.AllowAbstract(bAllowAbstract)
			.IsBlueprintBaseOnly(bIsBlueprintBaseOnly)
			.AllowNone(bAllowNone)
			.ShowTreeView(bShowTreeView)
			.HideViewOptions(bHideViewOptions)
			.ShowDisplayNames(bShowDisplayNames)
			.SelectedClass(this, &FSoftClassPathCustomization::OnGetClass)
			.OnSetClass(this, &FSoftClassPathCustomization::OnSetClass)
	];
}

void FSoftClassPathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

const UClass* FSoftClassPathCustomization::OnGetClass() const
{
	FString ClassName;
	PropertyHandle->GetValueAsFormattedString(ClassName);

	// Do we have a valid cached class pointer?
	const UClass* Class = CachedClassPtr.Get();
	if(!Class || Class->GetPathName() != ClassName)
	{
		Class = FEditorClassUtils::GetClassFromString(ClassName);
		CachedClassPtr = MakeWeakObjectPtr(const_cast<UClass*>(Class));
	}
	return Class;
}

void FSoftClassPathCustomization::OnSetClass(const UClass* NewClass)
{
	if (PropertyHandle->SetValueFromFormattedString((NewClass) ? NewClass->GetPathName() : "None") == FPropertyAccess::Result::Success)
	{
		CachedClassPtr = MakeWeakObjectPtr(const_cast<UClass*>(NewClass));
	}
}
