// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class UClass;

/**
 * Customizes a string class reference to look like a UClass property
 */
class FSoftClassPathCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable(new FSoftClassPathCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	/** @return The class currently set on this reference */
	const UClass* OnGetClass() const;
	/** Set the class used by this reference */
	void OnSetClass(const UClass* NewClass);

	/** Find or load the class associated with the given string */
	static const UClass* StringToClass(const FString& ClassName);

	/** Handle to the property being customized */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	/** A cache of the currently resolved value for the class name */
	mutable TWeakObjectPtr<UClass> CachedClassPtr;
};
