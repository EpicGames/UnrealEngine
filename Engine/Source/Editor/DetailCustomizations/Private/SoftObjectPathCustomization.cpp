// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftObjectPathCustomization.h"

#include "Containers/UnrealString.h"
#include "DetailWidgetRow.h"
#include "EditorClassUtils.h"
#include "HAL/PlatformCrt.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class IDetailChildrenBuilder;
class UClass;

void FSoftObjectPathCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	StructPropertyHandle = InStructPropertyHandle;
	
	const FString& MetaClassName = InStructPropertyHandle->GetMetaData("MetaClass");
	UClass* MetaClass = !MetaClassName.IsEmpty()
		? FEditorClassUtils::GetClassFromString(MetaClassName)
		: UObject::StaticClass();

	TSharedRef<SObjectPropertyEntryBox> ObjectPropertyEntryBox = SNew(SObjectPropertyEntryBox)
		.AllowedClass(MetaClass)
		.PropertyHandle(InStructPropertyHandle)
		.ThumbnailPool(StructCustomizationUtils.GetThumbnailPool());

	float MinDesiredWidth, MaxDesiredWidth;
	ObjectPropertyEntryBox->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);

	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(MinDesiredWidth)
	.MaxDesiredWidth(MaxDesiredWidth)
	[
		// Add an object entry box.  Even though this isn't an object entry, we will simulate one
		ObjectPropertyEntryBox
	];

	// This avoids making duplicate reset boxes
	StructPropertyHandle->MarkResetToDefaultCustomized();
}

void FSoftObjectPathCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}
