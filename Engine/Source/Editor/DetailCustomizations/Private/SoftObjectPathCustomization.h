// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

/**
 * Customizes a soft object path to look like a UObject property
 */
class FSoftObjectPathCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable( new FSoftObjectPathCustomization );
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:
	/** Handle to the struct property being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
};
