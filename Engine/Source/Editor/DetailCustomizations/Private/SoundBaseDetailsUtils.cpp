// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundBaseDetailsUtils.h"

namespace SoundBaseDetailsUtils
{
	static const int32 DefaultTopSortOrder = 4000;

	static const TArray<FName> DefaultCategoryOrder =
	{
		"Sound",
		"Voice Management",
		"Routing",
		"Modulation",
		"Memory",
		"Info",
		"Developer",
		"Advanced",
	};
} // namespace SoundBaseDetailsUtils

void FSoundBaseDetailsUtils::SortSoundCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
{
	// if no category order is passed in, use the default
	FSoundBaseDetailsUtils::SortSoundCategories(AllCategoryMap, SoundBaseDetailsUtils::DefaultCategoryOrder);
}

void FSoundBaseDetailsUtils::SortSoundCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap, const TArray<FName> CategoryOrder)
{
	// Start at the top of the list, or at a default value if you can't find that
	int32 TopSortOrder = SoundBaseDetailsUtils::DefaultTopSortOrder;
	const IDetailCategoryBuilder* FirstCategory = AllCategoryMap.FindRef(CategoryOrder[0]);
	if (FirstCategory)
	{
		TopSortOrder = FirstCategory->GetSortOrder();
	}

	// Categories not listed in the CategoryOrder will be appended at the end with incoming order
	int32 NumCategoriesNotFound = 0;
	for (const TPair<FName, IDetailCategoryBuilder*>& Pair : AllCategoryMap)
	{
		const FName CategoryName = Pair.Key;
		int32 SortOffset = 1;

		bool CategoryFound = false;
		for (const FName& OrderedName : CategoryOrder)
		{
			if (OrderedName == CategoryName)
			{
				CategoryFound = true;
				break;
			}
			SortOffset++;
		}

		int32 SortOrder = TopSortOrder + SortOffset;
		if (!CategoryFound)
		{
			NumCategoriesNotFound++;
			SortOrder += NumCategoriesNotFound;
		}

		Pair.Value->SetSortOrder(SortOrder);
	}

	return;
}
