// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"

class FSoundBaseDetailsUtils
{
public:
	static void SortSoundCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap);
	static void SortSoundCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap, const TArray<FName> CategoryOrder);
};
