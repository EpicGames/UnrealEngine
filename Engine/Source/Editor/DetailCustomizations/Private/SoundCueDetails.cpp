// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "SoundBaseDetailsUtils.h"

#define LOCTEXT_NAMESPACE "FSoundCueDetails"

namespace SoundCueDetailsUtils
{
	static const TArray<FName> CategoryOrder =
	{
		"Sound",
		"Voice Management",
		"Routing",
		"Memory",
		"Developer",
		"Advanced",
	};

	void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
	{
		FSoundBaseDetailsUtils::SortSoundCategories(AllCategoryMap, SoundCueDetailsUtils::CategoryOrder);
	}
} // namespace SoundCueDetailsUtils

TSharedRef<IDetailCustomization> FSoundCueDetails::MakeInstance()
{
	return MakeShared<FSoundCueDetails>();
}

void FSoundCueDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.SortCategories(SoundCueDetailsUtils::SortCategories);
	DetailBuilder.HideCategory("Info");
}

#undef LOCTEXT_NAMESPACE
