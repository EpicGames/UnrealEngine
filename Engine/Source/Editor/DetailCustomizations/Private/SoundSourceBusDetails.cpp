// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSourceBusDetails.h"
#include "Sound/SoundWave.h"

#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "FSoundSourceBusDetails"

TSharedRef<IDetailCustomization> FSoundSourceBusDetails::MakeInstance()
{
	return MakeShareable(new FSoundSourceBusDetails);
}

void FSoundSourceBusDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Hide details of sound base that aren't relevant for buses
	DetailBuilder.HideCategory("Waveform Processing");
	DetailBuilder.HideProperty("Pitch", USoundWave::StaticClass());
}

#undef LOCTEXT_NAMESPACE
