// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundSubmixSendFilterCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Math/UnitConversion.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SNumericEntryBox.h"

namespace UE::SoundSubmixSendFilterCustomization::Private
{
	static bool IsCutoffPropertyName(FName PropertyName)
	{
		return PropertyName == FName("LPFCutoff") || PropertyName == FName("HPFCutoff");
	}

	static TSharedRef<SWidget> BuildCutoffValueWidget(TSharedRef<IPropertyHandle> CutoffHandle)
	{
		TSharedRef<TNumericUnitTypeInterface<float>> TypeInterface =
			MakeShared<TNumericUnitTypeInterface<float>>(EUnit::Hertz);

		// SliderExponent 5.0 approximates octave-linear drag across 20Hz-20kHz (geometric midpoint ~632Hz lands at slider center).
		return SNew(SNumericEntryBox<float>)
			.AllowSpin(true)
			.MinValue(20.0f)
			.MaxValue(20000.0f)
			.MinSliderValue(20.0f)
			.MaxSliderValue(20000.0f)
			.SliderExponent(5.0f)
			.MinFractionalDigits(1)
			.MaxFractionalDigits(1)
			.Delta(0.1f)
			.TypeInterface(TypeInterface)
			.IsEnabled_Lambda([CutoffHandle]()
			{
				return CutoffHandle->IsEditable() && !CutoffHandle->IsEditConst();
			})
			.Value_Lambda([CutoffHandle]() -> TOptional<float>
			{
				float V = 0.0f;
				return CutoffHandle->GetValue(V) == FPropertyAccess::Success ? TOptional<float>(V) : TOptional<float>();
			})
			.OnValueChanged_Lambda([CutoffHandle](float NewValue)
			{
				CutoffHandle->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange);
			})
			.OnValueCommitted_Lambda([CutoffHandle](float NewValue, ETextCommit::Type)
			{
				CutoffHandle->SetValue(NewValue);
			});
	}
}

TSharedRef<IPropertyTypeCustomization> FSoundSubmixSendFilterCustomization::MakeInstance()
{
	return MakeShareable(new FSoundSubmixSendFilterCustomization);
}

void FSoundSubmixSendFilterCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> StructHandle,
	FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& Utils)
{
	HeaderRow
		.NameContent()  [ StructHandle->CreatePropertyNameWidget()  ]
		.ValueContent() [ StructHandle->CreatePropertyValueWidget() ];
}

void FSoundSubmixSendFilterCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& Utils)
{
	using namespace UE::SoundSubmixSendFilterCustomization::Private;

	uint32 NumChildren = 0;
	StructHandle->GetNumChildren(NumChildren);

	for (uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr<IPropertyHandle> Child = StructHandle->GetChildHandle(i);
		if (!Child.IsValid() || !Child->GetProperty())
		{
			continue;
		}

		if (IsCutoffPropertyName(Child->GetProperty()->GetFName()))
		{
			ChildBuilder.AddProperty(Child.ToSharedRef())
				.CustomWidget(/*bShowChildren*/ true)
				.NameContent()  [ Child->CreatePropertyNameWidget() ]
				.ValueContent() [ BuildCutoffValueWidget(Child.ToSharedRef()) ];
		}
		else
		{
			ChildBuilder.AddProperty(Child.ToSharedRef());
		}
	}
}
