// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;

// Custom LPF/HPF cutoff spinbox with MaxFractionalDigits(1); SSpinBox::Construct clobbers the meta-path value.
class FSoundSubmixSendFilterCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& Utils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& Utils) override;
	//~ End IPropertyTypeCustomization
};
