// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundWaveDetails.h"

#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/CurveTable.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Sound/SoundWave.h"
#include "SoundBaseDetailsUtils.h"
#include "ISoundWaveCloudStreaming.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "FSoundWaveDetails"

static const FName InternalCurveTableName("InternalCurveTable");

namespace SoundWaveDetailsUtils
{
	static const TArray<FName> CategoryOrder =
	{
		"Sound",
		"Voice Management",
		"Routing",
		"Modulation",
		"Memory",
		"Info",
		"Developer",
		"Curves",
		"File Path",
		"Advanced",
		"Waveform Processing",
	};

	void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategoryMap)
	{
		FSoundBaseDetailsUtils::SortSoundCategories(AllCategoryMap, SoundWaveDetailsUtils::CategoryOrder);
	}
} // namespace SoundWaveDetailsUtils

TSharedRef<IDetailCustomization> FSoundWaveDetails::MakeInstance()
{
	return MakeShared<FSoundWaveDetails>();
}

void FSoundWaveDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.SortCategories(SoundWaveDetailsUtils::SortCategories);

	CustomizeCurveDetails(DetailBuilder);
	CustomizeCloudStreamingPlatformDetails(DetailBuilder);

	if(!FModuleManager::Get().IsModuleLoaded("WaveformTransformations"))
	{
		DetailBuilder.HideCategory("Waveform Processing");
	}
}

void FSoundWaveDetails::CustomizeCurveDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() == 1 && Objects[0].IsValid())
	{
		USoundWave* SoundWave = CastChecked<USoundWave>(Objects[0].Get());

		TSharedRef<IPropertyHandle> CurvePropertyHandle = DetailBuilder.GetProperty(USoundWave::GetCurvePropertyName());
		if(CurvePropertyHandle->IsValidHandle())
		{
			IDetailPropertyRow& CurvePropertyRow = DetailBuilder.EditCategory(TEXT("Curves")).AddProperty(CurvePropertyHandle);
			TSharedPtr<SWidget> DefaultNameWidget;
			TSharedPtr<SWidget> DefaultValueWidget;
			CurvePropertyRow.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget);

			CurvePropertyRow.CustomWidget()
			.NameContent()
			[
				DefaultNameWidget.ToSharedRef()
			]
			.ValueContent()
			.MaxDesiredWidth(TOptional<float>())
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					DefaultValueWidget.ToSharedRef()
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Visibility(this, &FSoundWaveDetails::GetMakeInternalCurvesVisibility, SoundWave, CurvePropertyHandle)
					.OnClicked(this, &FSoundWaveDetails::HandleMakeInternalCurves, SoundWave)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MakeInternal", "Copy To Internal"))
						.ToolTipText(LOCTEXT("MakeInternalTooltip", "Convert the currently selected curve table to an internal curve table."))
						.Font(DetailBuilder.GetDetailFont())
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Visibility(this, &FSoundWaveDetails::GetUseInternalCurvesVisibility, SoundWave, CurvePropertyHandle)
					.OnClicked(this, &FSoundWaveDetails::HandleUseInternalCurves, SoundWave, CurvePropertyHandle)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UseInternal", "Use Internal"))
						.ToolTipText(LOCTEXT("UseInternalTooltip", "Use the curve table internal to this sound wave."))
						.Font(DetailBuilder.GetDetailFont())
					]
				]
			];
		}
	}
}

EVisibility FSoundWaveDetails::GetMakeInternalCurvesVisibility(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle) const
{
	UObject* CurrentCurveTable = nullptr;
	if (CurvePropertyHandle->GetValue(CurrentCurveTable) == FPropertyAccess::Success)
	{
		if (CurrentCurveTable != nullptr && CurrentCurveTable->HasAnyFlags(RF_Public) && CurrentCurveTable->GetOutermost() != SoundWave->GetOutermost())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FSoundWaveDetails::GetUseInternalCurvesVisibility(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle) const
{
	UCurveTable* InternalCurveTable = SoundWave->GetInternalCurveData();

	UObject* CurrentCurveTable = nullptr;
	if (CurvePropertyHandle->GetValue(CurrentCurveTable) == FPropertyAccess::Success)
	{
		if (InternalCurveTable != nullptr && InternalCurveTable->HasAnyFlags(RF_Standalone) && CurrentCurveTable != InternalCurveTable)
		{
			return EVisibility::Visible;
		}
}

	return EVisibility::Collapsed;
}

FReply FSoundWaveDetails::HandleMakeInternalCurves(USoundWave* SoundWave)
{
	UCurveTable* ExistingCurves = SoundWave->GetCurveData();
	if (ExistingCurves != nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("MakeInternalCurve", "Copy Curve to Internal"));
		SoundWave->Modify();

		UCurveTable* DuplicatedCurves = DuplicateObject<UCurveTable>(ExistingCurves, SoundWave, InternalCurveTableName);
		DuplicatedCurves->ClearFlags(RF_Public);
		DuplicatedCurves->SetFlags(ExistingCurves->GetFlags() | RF_Standalone | RF_Transactional);
		SoundWave->SetCurveData(DuplicatedCurves);
		SoundWave->SetInternalCurveData(DuplicatedCurves);
	}

	return FReply::Handled();
}

FReply FSoundWaveDetails::HandleUseInternalCurves(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle)
{
	UCurveTable* InternalCurveTable = SoundWave->GetInternalCurveData();
	if (InternalCurveTable != nullptr)
	{
		CurvePropertyHandle->SetValue(InternalCurveTable);
	}
	
	return FReply::Handled();
}

void FSoundWaveDetails::CustomizeCloudStreamingPlatformDetails(IDetailLayoutBuilder& DetailBuilder)
{
#if WITH_EDITOR
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	TArray<Audio::ISoundWaveCloudStreamingFeature*> Features = IModularFeatures::Get().GetModularFeatureImplementations<Audio::ISoundWaveCloudStreamingFeature>(Audio::ISoundWaveCloudStreamingFeature::GetModularFeatureName());
	bool bEnabled = false;
	for(int32 i=0; i<Features.Num(); ++i)
	{
		if (Features[i]->AddCustomizationCloudStreamingPlatformDetails(DetailBuilder))
		{
			bEnabled = true;
			break;
		}
	}
	if (!bEnabled)
	{
		DetailBuilder.HideProperty("PlatformSettings");

		// If there are objects that have the cloud streaming flag set without there being a suitable feature plugin available now,
		// we do not hide the flag so that it can be disabled.
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		Objects = Objects.FilterByPredicate([](const TWeakObjectPtr<UObject>& ObjectPtr) { return ObjectPtr.IsValid() && ObjectPtr->IsA<USoundWave>() && Cast<USoundWave>(ObjectPtr)->IsCloudStreamingEnabled(); });
		TSharedPtr<IPropertyHandle> CloudStreamingProperty = DetailBuilder.GetProperty(USoundWave::GetCloudStreamingEnabledPropertyName());
		if (CloudStreamingProperty.IsValid())
		{
			if (Objects.Num() == 0)
			{
				DetailBuilder.HideProperty(CloudStreamingProperty);
			}
			else
			{
				FText CurrentToolTipText = CloudStreamingProperty->GetToolTipText();
				FText NewToolTipText = FText::FromString(FString::Printf(TEXT("%s\n\nNo suitable plugin has been found. You can disable this flag to get back to non-cloud streaming settings."), *CurrentToolTipText.ToString()));
				CloudStreamingProperty->SetToolTipText(NewToolTipText);
			}
		}
	}
#endif
}



#undef LOCTEXT_NAMESPACE
