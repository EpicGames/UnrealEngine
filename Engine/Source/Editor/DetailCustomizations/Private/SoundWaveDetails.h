// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class USoundWave;

class FSoundWaveDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void CustomizeCurveDetails(IDetailLayoutBuilder& DetailBuilder);

	void CustomizeCloudStreamingPlatformDetails(IDetailLayoutBuilder& DetailBuilder);

	EVisibility GetMakeInternalCurvesVisibility(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle) const;

	EVisibility GetUseInternalCurvesVisibility(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle) const;

	FReply HandleMakeInternalCurves(USoundWave* SoundWave);

	FReply HandleUseInternalCurves(USoundWave* SoundWave, TSharedRef<IPropertyHandle> CurvePropertyHandle);
};
