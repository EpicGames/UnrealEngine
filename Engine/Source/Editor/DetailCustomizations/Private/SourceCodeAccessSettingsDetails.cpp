// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceCodeAccessSettingsDetails.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Features/IModularFeatures.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

class IModularFeature;
class SWidget;

#define LOCTEXT_NAMESPACE "SourceCodeAccessSettingsDetails"

TSharedRef<IDetailCustomization> FSourceCodeAccessSettingsDetails::MakeInstance()
{
	return MakeShareable(new FSourceCodeAccessSettingsDetails());
}

void FSourceCodeAccessSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	TSharedRef<IPropertyHandle> PreferredProviderPropertyHandle = DetailLayout.GetProperty("PreferredAccessor");
	DetailLayout.HideProperty("PreferredAccessor");

	// regenerate accessors list
	Accessors.Empty();

	const int32 FeatureCount = IModularFeatures::Get().GetModularFeatureImplementationCount("SourceCodeAccessor");
	for(int32 FeatureIndex = 0; FeatureIndex < FeatureCount; FeatureIndex++)
	{
		IModularFeature* Feature = IModularFeatures::Get().GetModularFeatureImplementation("SourceCodeAccessor", FeatureIndex);
		check(Feature);

		ISourceCodeAccessor& Accessor = *static_cast<ISourceCodeAccessor*>(Feature);
		if(Accessor.GetFName() != FName("None"))
		{
			Accessors.Add(MakeShareable(new FAccessorItem(Accessor.GetNameText(), Accessor.GetFName())));
		}
	}

	IDetailCategoryBuilder& AccessorCategory = DetailLayout.EditCategory( "Accessor" );
	AccessorCategory.AddCustomRow( LOCTEXT("PreferredAccessorFilterString", "Source Code Editor") )
	.NameContent()
	[
		PreferredProviderPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(113.f)
	.MaxDesiredWidth(113.f)
	[
		SNew(SComboBox< TSharedPtr<FAccessorItem>>)
		.ToolTipText(LOCTEXT("PreferredAccessorToolTip", "Choose the way to access source code."))
		.OptionsSource(&Accessors)
		.OnSelectionChanged(this, &FSourceCodeAccessSettingsDetails::OnSelectionChanged, PreferredProviderPropertyHandle)
		.ContentPadding(2.f)
		.OnGenerateWidget(this, &FSourceCodeAccessSettingsDetails::OnGenerateWidget)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &FSourceCodeAccessSettingsDetails::GetAccessorText)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
	];
}

TSharedRef<SWidget> FSourceCodeAccessSettingsDetails::OnGenerateWidget( TSharedPtr<FAccessorItem> InItem )
{
	return SNew(STextBlock)
		.Text(InItem->Text);
}

void FSourceCodeAccessSettingsDetails::OnSelectionChanged(TSharedPtr<FAccessorItem> InItem, ESelectInfo::Type InSeletionInfo, TSharedRef<IPropertyHandle> PreferredProviderPropertyHandle)
{
	PreferredProviderPropertyHandle->SetValue(InItem->Name.ToString());
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	SourceCodeAccessModule.SetAccessor(InItem->Name);
}

FText FSourceCodeAccessSettingsDetails::GetAccessorText() const
{
	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule.GetAccessor().GetNameText();
}

#undef LOCTEXT_NAMESPACE
