// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;

struct FAccessorItem
{
	FAccessorItem(const FText& InText, const FName& InName)
		: Text(InText)
		, Name(InName)
	{
	}

	/** Text to display */
	FText Text;

	/** Name of the accessor */
	FName Name;
};

class FSourceCodeAccessSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

private:
	/** Generate a row widget for display in the list view */
	TSharedRef<SWidget> OnGenerateWidget( TSharedPtr<FAccessorItem> InItem );

	/** Set the accessor when we change selection */
	void OnSelectionChanged(TSharedPtr<FAccessorItem> InItem, ESelectInfo::Type InSeletionInfo, TSharedRef<IPropertyHandle> PreferredProviderPropertyHandle);

	/** Get the text to display on the accessor drop-down */
	FText GetAccessorText() const;

private:
	/** Accessor names to display in a drop-down list */
	TArray<TSharedPtr<FAccessorItem>> Accessors;
};
