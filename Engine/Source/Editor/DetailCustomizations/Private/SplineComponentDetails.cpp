// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineComponentDetails.h"

#include "BlueprintEditor.h"
#include "BlueprintEditorModule.h"
#include "ComponentVisualizer.h"
#include "ComponentVisualizerManager.h"
#include "Components/SplineComponent.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Features/IModularFeatures.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomNodeBuilder.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "LevelEditorViewport.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Axis.h"
#include "Math/InterpCurve.h"
#include "Math/InterpCurvePoint.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "SlotBase.h"
#include "SplineComponentVisualizer.h"
#include "SplineDetailsProvider.h"
#include "SplineMetadataDetailsFactory.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealEdGlobals.h"
#include "Math/UnitConversion.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class FObjectInitializer;
class IDetailGroup;
class SWidget;

#define LOCTEXT_NAMESPACE "SplineComponentDetails"
DEFINE_LOG_CATEGORY_STATIC(LogSplineComponentDetails, Log, All)

USplineMetadataDetailsFactoryBase::USplineMetadataDetailsFactoryBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

class FSplinePointDetails : public IDetailCustomNodeBuilder, public TSharedFromThis<FSplinePointDetails>
{
public:
	FSplinePointDetails(USplineComponent* InOwningSplineComponent);

	//~ Begin IDetailCustomNodeBuilder interface
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override;
	virtual bool RequiresTick() const override { return true; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override;
	//~ End IDetailCustomNodeBuilder interface

	static bool bAlreadyWarnedInvalidIndex;

private:

	template <typename T>
	struct TSharedValue
	{
		TSharedValue() : bInitialized(false) {}

		void Reset()
		{
			bInitialized = false;
		}

		bool IsValid() const { return bInitialized; }

		void Add(T InValue)
		{
			if (!bInitialized)
			{
				Value = InValue;
				bInitialized = true;
			}
			else
			{
				if (Value.IsSet() && InValue != Value.GetValue()) { Value.Reset(); }
			}
		}

		bool HasMultipleValues() const
		{
			return !Value.IsSet();
		}

		TOptional<T> Value;
		bool bInitialized;
	};

	struct FSharedVectorValue
	{
		FSharedVectorValue() : bInitialized(false) {}

		void Reset()
		{
			bInitialized = false;
		}

		bool IsValid() const { return bInitialized; }

		void Add(const FVector& V)
		{
			if (!bInitialized)
			{
				X = V.X;
				Y = V.Y;
				Z = V.Z;
				bInitialized = true;
			}
			else
			{
				if (X.IsSet() && V.X != X.GetValue()) { X.Reset(); }
				if (Y.IsSet() && V.Y != Y.GetValue()) { Y.Reset(); }
				if (Z.IsSet() && V.Z != Z.GetValue()) { Z.Reset(); }
			}
		}

		bool HasMultipleValues() const 
		{
			return !X.IsSet() || !Y.IsSet() || !Z.IsSet();
		}

		TOptional<FVector::FReal> X;
		TOptional<FVector::FReal> Y;
		TOptional<FVector::FReal> Z;
		bool bInitialized;
	};

	struct FSharedRotatorValue
	{
		FSharedRotatorValue() : bInitialized(false) {}

		void Reset()
		{
			bInitialized = false;
		}

		bool IsValid() const { return bInitialized; }

		void Add(const FRotator& R)
		{
			if (!bInitialized)
			{
				Roll = R.Roll;
				Pitch = R.Pitch;
				Yaw = R.Yaw;
				bInitialized = true;
			}
			else
			{
				if (Roll.IsSet() && R.Roll != Roll.GetValue()) { Roll.Reset(); }
				if (Pitch.IsSet() && R.Pitch != Pitch.GetValue()) { Pitch.Reset(); }
				if (Yaw.IsSet() && R.Yaw != Yaw.GetValue()) { Yaw.Reset(); }
			}
		}

		bool HasMultipleValues() const
		{
			return !Roll.IsSet() || !Pitch.IsSet() || !Yaw.IsSet();
		}

		TOptional<FRotator::FReal> Roll;
		TOptional<FRotator::FReal> Pitch;
		TOptional<FRotator::FReal> Yaw;
		bool bInitialized;
	};

	EVisibility IsEnabled() const { return (SelectedKeys.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	EVisibility IsDisabled() const { return (SelectedKeys.Num() == 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	bool IsOnePointSelected() const { return SelectedKeys.Num() == 1; }
	bool ArePointsSelected() const { return (SelectedKeys.Num() > 0); };
	bool AreNoPointsSelected() const { return (SelectedKeys.Num() == 0); };
	bool CanSetInputKey() const;
	TOptional<float> GetInputKey() const { return InputKey.Value; }
	TOptional<FVector::FReal> GetPositionX() const { return Position.X; }
	TOptional<FVector::FReal> GetPositionY() const { return Position.Y; }
	TOptional<FVector::FReal> GetPositionZ() const { return Position.Z; }
	TOptional<FVector::FReal> GetArriveTangentX() const { return ArriveTangent.X; }
	TOptional<FVector::FReal> GetArriveTangentY() const { return ArriveTangent.Y; }
	TOptional<FVector::FReal> GetArriveTangentZ() const { return ArriveTangent.Z; }
	TOptional<FVector::FReal> GetLeaveTangentX() const { return LeaveTangent.X; }
	TOptional<FVector::FReal> GetLeaveTangentY() const { return LeaveTangent.Y; }
	TOptional<FVector::FReal> GetLeaveTangentZ() const { return LeaveTangent.Z; }
	TOptional<FRotator::FReal> GetRotationRoll() const { return Rotation.Roll; }
	TOptional<FRotator::FReal> GetRotationPitch() const { return Rotation.Pitch; }
	TOptional<FRotator::FReal> GetRotationYaw() const { return Rotation.Yaw; }
	TOptional<FVector::FReal> GetScaleX() const { return Scale.X; }
	TOptional<FVector::FReal> GetScaleY() const { return Scale.Y; }
	TOptional<FVector::FReal> GetScaleZ() const { return Scale.Z; }
	void OnSetInputKey(float NewValue, ETextCommit::Type CommitInfo);
	void OnSetPosition(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);
	void OnSetArriveTangent(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);
	void OnSetLeaveTangent(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);
	void OnSetRotation(FRotator::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);
	void OnSetScale(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis);
	FText GetPointType() const;
	void OnSplinePointTypeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> OnGenerateComboWidget(TSharedPtr<FString> InComboString);

	void GenerateSplinePointSelectionControls(IDetailChildrenBuilder& ChildrenBuilder);
	FReply OnSelectFirstLastSplinePoint(bool bFirst);
	FReply OnSelectPrevNextSplinePoint(bool bNext, bool bAddToSelection);
	FReply OnSelectAllSplinePoints();

	USplineComponent* GetSplineComponentToVisualize() const;

	void UpdateValues();

	enum class ESplinePointProperty
	{
		Location,
		Rotation,
		Scale,
		ArriveTangent,
		LeaveTangent, 
		Type
	};

	TSharedRef<SWidget> BuildSplinePointPropertyLabel(ESplinePointProperty SplinePointProp);
	void OnSetTransformEditingAbsolute(ESplinePointProperty SplinePointProp, bool bIsAbsolute);
	bool IsTransformEditingAbsolute(ESplinePointProperty SplinePointProperty) const;
	bool IsTransformEditingRelative(ESplinePointProperty SplinePointProperty) const;
	FText GetSplinePointPropertyText(ESplinePointProperty SplinePointProp) const;
	void SetSplinePointProperty(ESplinePointProperty SplinePointProp, FVector3f NewValue, EAxisList::Type Axis, bool bCommitted);

	FUIAction CreateCopyAction(ESplinePointProperty SplinePointProp);
	FUIAction CreatePasteAction(ESplinePointProperty SplinePointProp);

	bool OnCanCopy(ESplinePointProperty SplinePointProp) const;
	void OnCopy(ESplinePointProperty SplinePointProp);
	void OnPaste(ESplinePointProperty SplinePointProp);

	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, ESplinePointProperty SplinePointProp);
	void PasteFromText(const FString& InTag, const FString& InText, ESplinePointProperty SplinePointProp);

	void OnBeginPositionSlider();
	void OnBeginScaleSlider();
	void OnEndSlider(FVector::FReal);

	FVector ConvertToLUFEuler(const FQuat& RotationQuaternion) const;
	FQuat ConvertFromLUFEuler(const FVector& RotationEuler) const;

	USplineComponent* SplineComp;
	USplineComponent* SplineCompArchetype;
	TSet<int32> SelectedKeys;

	TSharedValue<float> InputKey;
	FSharedVectorValue Position;
	FSharedVectorValue ArriveTangent;
	FSharedVectorValue LeaveTangent;
	FSharedVectorValue Scale;
	FSharedRotatorValue Rotation;
	TSharedValue<ESplinePointType::Type> PointType;

	void TryAcquireDetailsInterface();
	ISplineDetailsProvider* SplineDetailsInterface = nullptr;
	
	TArray<FProperty*> SplineProperties;
	TArray<TSharedPtr<FString>> SplinePointTypes;
	TSharedPtr<ISplineMetadataDetails> SplineMetaDataDetails;
	FSimpleDelegate OnRegenerateChildren;

	bool bEditingLocationAbsolute = false;
	bool bEditingRotationAbsolute = false;

	bool bInSliderTransaction = false;
};

bool FSplinePointDetails::bAlreadyWarnedInvalidIndex = false;

FSplinePointDetails::FSplinePointDetails(USplineComponent* InOwningSplineComponent)
	: SplineComp(nullptr)
{
	for (const FName& Property : USplineComponent::GetSplinePropertyNames())
	{
		SplineProperties.Add(FindFProperty<FProperty>(USplineComponent::StaticClass(), Property));
	}

	const TArray<ESplinePointType::Type> EnabledSplinePointTypes = InOwningSplineComponent->GetEnabledSplinePointTypes();

	UEnum* SplinePointTypeEnum = StaticEnum<ESplinePointType::Type>();
	check(SplinePointTypeEnum);
	for (int32 EnumIndex = 0; EnumIndex < SplinePointTypeEnum->NumEnums() - 1; ++EnumIndex)
	{
		const ESplinePointType::Type Value = static_cast<ESplinePointType::Type>(SplinePointTypeEnum->GetValueByIndex(EnumIndex));
		if (EnabledSplinePointTypes.Contains(Value))
		{
			SplinePointTypes.Add(MakeShareable(new FString(SplinePointTypeEnum->GetNameStringByIndex(EnumIndex))));
		}
	}

	check(InOwningSplineComponent);
	if (InOwningSplineComponent->IsTemplate())
	{
		// For blueprints, SplineComp will be set to the preview actor in UpdateValues().
		SplineComp = nullptr;
		SplineCompArchetype = InOwningSplineComponent;
	}
	else
	{
		SplineComp = InOwningSplineComponent;
		SplineCompArchetype = nullptr;
	}

	bAlreadyWarnedInvalidIndex = false;
}

void FSplinePointDetails::TryAcquireDetailsInterface()
{
	if (!SplineComp && !SplineCompArchetype)
	{
		return;
	}
	
	if (!SplineDetailsInterface)
	{
		ISplineDetailsProvider* FoundInterface = nullptr;
		for (ISplineDetailsProvider* Interface : IModularFeatures::Get().GetModularFeatureImplementations<ISplineDetailsProvider>(
			ISplineDetailsProvider::GetModularFeatureName()))
		{
			if (Interface && Interface->ShouldUseForSpline(SplineComp ? SplineComp : SplineCompArchetype))
			{
				FoundInterface = Interface;
			}
		}

		if (FoundInterface)
		{
			SplineDetailsInterface = FoundInterface;
			
			IModularFeatures::Get().OnModularFeatureUnregistered().AddSPLambda(this, [this](const FName& Type, class IModularFeature* ModularFeature)
			{
				if (SplineDetailsInterface == ModularFeature)
				{
					SplineDetailsInterface = nullptr;
					IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);
				}
			});
		}
	}
}

void FSplinePointDetails::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

void FSplinePointDetails::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
}

void FSplinePointDetails::GenerateSplinePointSelectionControls(IDetailChildrenBuilder& ChildrenBuilder)
{
	FMargin ButtonPadding(2.f, 0.f);

	ChildrenBuilder.AddCustomRow(LOCTEXT("SelectSplinePoints", "Select Spline Points"))
	.RowTag("SelectSplinePoints")
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("SelectSplinePoints", "Select Spline Points"))
	]
	.ValueContent()
	.VAlign(VAlign_Fill)
	.MaxDesiredWidth(170.f)
	.MinDesiredWidth(170.f)
	[
		SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectFirst")
			.ContentPadding(2.0f)
			.ToolTipText(LOCTEXT("SelectFirstSplinePointToolTip", "Select first spline point."))
			.OnClicked(this, &FSplinePointDetails::OnSelectFirstLastSplinePoint, true)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.AddPrev")
			.ContentPadding(2.f)
			.ToolTipText(LOCTEXT("SelectAddPrevSplinePointToolTip", "Add previous spline point to current selection."))
			.OnClicked(this, &FSplinePointDetails::OnSelectPrevNextSplinePoint, false, true)
			.IsEnabled(this, &FSplinePointDetails::ArePointsSelected)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectPrev")
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("SelectPrevSplinePointToolTip", "Select previous spline point."))
			.OnClicked(this, &FSplinePointDetails::OnSelectPrevNextSplinePoint, false, false)
			.IsEnabled(this, &FSplinePointDetails::ArePointsSelected)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectAll")
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("SelectAllSplinePointToolTip", "Select all spline points."))
			.OnClicked(this, &FSplinePointDetails::OnSelectAllSplinePoints)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectNext")
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("SelectNextSplinePointToolTip", "Select next spline point."))
			.OnClicked(this, &FSplinePointDetails::OnSelectPrevNextSplinePoint, true, false)
			.IsEnabled(this, &FSplinePointDetails::ArePointsSelected)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.AddNext")
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("SelectAddNextSplinePointToolTip", "Add next spline point to current selection."))
			.OnClicked(this, &FSplinePointDetails::OnSelectPrevNextSplinePoint, true, true)
			.IsEnabled(this, &FSplinePointDetails::ArePointsSelected)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectLast")
			.ContentPadding(2.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ToolTipText(LOCTEXT("SelectLastSplinePointToolTip", "Select last spline point."))
			.OnClicked(this, &FSplinePointDetails::OnSelectFirstLastSplinePoint, false)
		]
	];
}

void FSplinePointDetails::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	// Select spline point buttons
	GenerateSplinePointSelectionControls(ChildrenBuilder);

	// Message which is shown when no points are selected
	ChildrenBuilder.AddCustomRow(LOCTEXT("NoneSelected", "None selected"))
		.RowTag(TEXT("NoneSelected"))
		.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsDisabled))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoPointsSelected", "No spline points are selected."))
		.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		];

	if (!SplineComp)
	{
		return;
	}
	// Input key
	ChildrenBuilder.AddCustomRow(LOCTEXT("InputKey", "Input Key"))
		.RowTag(TEXT("InputKey"))
		.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InputKey", "Input Key"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SNumericEntryBox<float>)
			.IsEnabled(TAttribute<bool>(this, &FSplinePointDetails::CanSetInputKey))
			.Value(this, &FSplinePointDetails::GetInputKey)
			.UndeterminedString(LOCTEXT("Multiple", "Multiple"))
			.OnValueCommitted(this, &FSplinePointDetails::OnSetInputKey)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	IDetailCategoryBuilder& ParentCategory = ChildrenBuilder.GetParentCategory();
	TSharedPtr<FOnPasteFromText> PasteFromTextDelegate = ParentCategory.OnPasteFromText();
	const bool bUsePasteFromText = PasteFromTextDelegate.IsValid();	

	// Position
	if (SplineComp->AllowsSpinePointLocationEditing())
	{
		PasteFromTextDelegate->AddSP(this, &FSplinePointDetails::OnPasteFromText, ESplinePointProperty::Location);
		
		ChildrenBuilder.AddCustomRow(LOCTEXT("Location", "Location"))
			.RowTag(TEXT("Location"))
			.CopyAction(CreateCopyAction(ESplinePointProperty::Location))
			.PasteAction(CreatePasteAction(ESplinePointProperty::Location))
			.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
			.NameContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				BuildSplinePointPropertyLabel(ESplinePointProperty::Location)
			]
		.ValueContent()
			.MinDesiredWidth(375.0f)
			.MaxDesiredWidth(375.0f)
			[
				SNew(SNumericVectorInputBox<FVector::FReal>)
				.X(this, &FSplinePointDetails::GetPositionX)
				.Y(this, &FSplinePointDetails::GetPositionY)
				.Z(this, &FSplinePointDetails::GetPositionZ)
				.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
				.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
				.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
				.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
				.AllowSpin(true)
				.bColorAxisLabels(true)
				.SpinDelta(1.f)
				.OnXChanged(this, &FSplinePointDetails::OnSetPosition, ETextCommit::Default, EAxis::X)
				.OnYChanged(this, &FSplinePointDetails::OnSetPosition, ETextCommit::Default, EAxis::Y)
				.OnZChanged(this, &FSplinePointDetails::OnSetPosition, ETextCommit::Default, EAxis::Z)
				.OnXCommitted(this, &FSplinePointDetails::OnSetPosition, EAxis::X)
				.OnYCommitted(this, &FSplinePointDetails::OnSetPosition, EAxis::Y)
				.OnZCommitted(this, &FSplinePointDetails::OnSetPosition, EAxis::Z)
				.OnBeginSliderMovement(this, &FSplinePointDetails::OnBeginPositionSlider)
				.OnEndSliderMovement(this, &FSplinePointDetails::OnEndSlider)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	// Rotation
	if (SplineComp->AllowsSplinePointRotationEditing())
	{
		TSharedPtr<INumericTypeInterface<FRotator::FReal>> TypeInterface = MakeShareable(FUnitConversion::Settings().ShouldDisplayUnits() ? new TNumericUnitTypeInterface<FRotator::FReal>(EUnit::Degrees) : new TDefaultNumericTypeInterface<FRotator::FReal>());
		if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
		{
			TypeInterface->SetMaxFractionalDigits(3);
			TypeInterface->SetIndicateNearlyInteger(false);
		}
		
		PasteFromTextDelegate->AddSP(this, &FSplinePointDetails::OnPasteFromText, ESplinePointProperty::Rotation);

		ChildrenBuilder.AddCustomRow(LOCTEXT("Rotation", "Rotation"))
			.RowTag(TEXT("Rotation"))
			.CopyAction(CreateCopyAction(ESplinePointProperty::Rotation))
			.PasteAction(CreatePasteAction(ESplinePointProperty::Rotation))
			.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
			.NameContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				BuildSplinePointPropertyLabel(ESplinePointProperty::Rotation)
			]
			.ValueContent()
			.MinDesiredWidth(375.0f)
			.MaxDesiredWidth(375.0f)
			[
				SNew(SNumericRotatorInputBox<FRotator::FReal>)
				.AllowSpin(false)
				.Roll(this, &FSplinePointDetails::GetRotationRoll)
				.Pitch(this, &FSplinePointDetails::GetRotationPitch)
				.Yaw(this, &FSplinePointDetails::GetRotationYaw)
				.RollDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Forward))
				.PitchDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Left))
				.YawDisplayName(AxisDisplayInfo::GetRotationAxisToolTip(EAxisList::Up))
				.bColorAxisLabels(true)
				.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
				.OnRollCommitted(this, &FSplinePointDetails::OnSetRotation, EAxis::X)
				.OnPitchCommitted(this, &FSplinePointDetails::OnSetRotation, EAxis::Y)
				.OnYawCommitted(this, &FSplinePointDetails::OnSetRotation, EAxis::Z)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.TypeInterface(TypeInterface)
			];
	}

	// Scale
	if (SplineComp->AllowsSplinePointScaleEditing())
	{
		PasteFromTextDelegate->AddSP(this, &FSplinePointDetails::OnPasteFromText, ESplinePointProperty::Scale);

		ChildrenBuilder.AddCustomRow(LOCTEXT("Scale", "Scale"))
			.RowTag(TEXT("Scale"))
			.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
			.CopyAction(CreateCopyAction(ESplinePointProperty::Scale))
			.PasteAction(CreatePasteAction(ESplinePointProperty::Scale))
			.NameContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ScaleLabel", "Scale"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		.ValueContent()
			.MinDesiredWidth(375.0f)
			.MaxDesiredWidth(375.0f)
			[
				SNew(SNumericVectorInputBox<FVector::FReal>)
				.X(this, &FSplinePointDetails::GetScaleX)
				.Y(this, &FSplinePointDetails::GetScaleY)
				.Z(this, &FSplinePointDetails::GetScaleZ)
				.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
				.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
				.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
				.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
				.AllowSpin(true)
				.bColorAxisLabels(true)
				.OnXChanged(this, &FSplinePointDetails::OnSetScale, ETextCommit::Default, EAxis::X)
				.OnYChanged(this, &FSplinePointDetails::OnSetScale, ETextCommit::Default, EAxis::Y)
				.OnZChanged(this, &FSplinePointDetails::OnSetScale, ETextCommit::Default, EAxis::Z)
				.OnXCommitted(this, &FSplinePointDetails::OnSetScale, EAxis::X)
				.OnYCommitted(this, &FSplinePointDetails::OnSetScale, EAxis::Y)
				.OnZCommitted(this, &FSplinePointDetails::OnSetScale, EAxis::Z)
				.OnBeginSliderMovement(this, &FSplinePointDetails::OnBeginScaleSlider)
				.OnEndSliderMovement(this, &FSplinePointDetails::OnEndSlider)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	{
		TSharedPtr<INumericTypeInterface<FRotator::FReal>> TypeInterface = MakeShareable(new TDefaultNumericTypeInterface<FRotator::FReal>());
		if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
		{
			TypeInterface->SetMaxFractionalDigits(3);
			TypeInterface->SetIndicateNearlyInteger(false);
		}

		// ArriveTangent
		if (SplineComp->AllowsSplinePointArriveTangentEditing())
		{
			PasteFromTextDelegate->AddSP(this, &FSplinePointDetails::OnPasteFromText, ESplinePointProperty::ArriveTangent);

			ChildrenBuilder.AddCustomRow(LOCTEXT("ArriveTangent", "Arrive Tangent"))
				.RowTag(TEXT("ArriveTangent"))
				.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
				.CopyAction(CreateCopyAction(ESplinePointProperty::ArriveTangent))
				.PasteAction(CreatePasteAction(ESplinePointProperty::ArriveTangent))
				.NameContent()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ArriveTangent", "Arrive Tangent"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(LOCTEXT("ArriveTangent_Tooltip", "Incoming tangent. Note that the size shown in viewport "
						"is controlled by Spline Tangent Scale in editor preferences (and hidden if 0). Only allowed to "
						"differ from Leave Tangent if Allow Discontinuous Spline is true."))
				]
			.ValueContent()
				.MinDesiredWidth(375.0f)
				.MaxDesiredWidth(375.0f)
				[
					SNew(SNumericVectorInputBox<FVector::FReal>)
					.X(this, &FSplinePointDetails::GetArriveTangentX)
					.Y(this, &FSplinePointDetails::GetArriveTangentY)
					.Z(this, &FSplinePointDetails::GetArriveTangentZ)
					.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
					.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
					.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
					.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
					.AllowSpin(false)
					.bColorAxisLabels(false)
					.OnXCommitted(this, &FSplinePointDetails::OnSetArriveTangent, EAxis::X)
					.OnYCommitted(this, &FSplinePointDetails::OnSetArriveTangent, EAxis::Y)
					.OnZCommitted(this, &FSplinePointDetails::OnSetArriveTangent, EAxis::Z)
					.TypeInterface(TypeInterface)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}

		// LeaveTangent
		if (SplineComp->AllowsSplinePointLeaveTangentEditing())
		{
			PasteFromTextDelegate->AddSP(this, &FSplinePointDetails::OnPasteFromText, ESplinePointProperty::LeaveTangent);

			ChildrenBuilder.AddCustomRow(LOCTEXT("LeaveTangent", "Leave Tangent"))
				.RowTag(TEXT("LeaveTangent"))
				.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
				.CopyAction(CreateCopyAction(ESplinePointProperty::LeaveTangent))
				.PasteAction(CreatePasteAction(ESplinePointProperty::LeaveTangent))
				.NameContent()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("LeaveTangent", "Leave Tangent"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(LOCTEXT("LeaveTangent_Tooltip", "Outgoing tangent. Note that the size shown in viewport "
						"is controlled by Spline Tangent Scale in editor preferences (and hidden if 0)."))
				]
			.ValueContent()
				.MinDesiredWidth(375.0f)
				.MaxDesiredWidth(375.0f)
				[
					SNew(SNumericVectorInputBox<FVector::FReal>)
					.X(this, &FSplinePointDetails::GetLeaveTangentX)
					.Y(this, &FSplinePointDetails::GetLeaveTangentY)
					.Z(this, &FSplinePointDetails::GetLeaveTangentZ)
					.XDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Forward))
					.YDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Left))
					.ZDisplayName(AxisDisplayInfo::GetAxisToolTip(EAxisList::Up))
					.Swizzle(AxisDisplayInfo::GetTransformAxisSwizzle())
					.AllowSpin(false)
					.bColorAxisLabels(false)
					.OnXCommitted(this, &FSplinePointDetails::OnSetLeaveTangent, EAxis::X)
					.OnYCommitted(this, &FSplinePointDetails::OnSetLeaveTangent, EAxis::Y)
					.OnZCommitted(this, &FSplinePointDetails::OnSetLeaveTangent, EAxis::Z)
					.TypeInterface(TypeInterface)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				];
		}
	}

	// Type
	if (SplineComp->GetEnabledSplinePointTypes().Num() > 1)
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("Type", "Type"))
			.RowTag(TEXT("Type"))
			.Visibility(TAttribute<EVisibility>(this, &FSplinePointDetails::IsEnabled))
			.CopyAction(CreateCopyAction(ESplinePointProperty::Type))
			.PasteAction(CreatePasteAction(ESplinePointProperty::Type))
			.NameContent()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Type", "Type"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		.ValueContent()
			.MinDesiredWidth(125.0f)
			.MaxDesiredWidth(125.0f)
			[
				SNew(SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&SplinePointTypes)
				.OnGenerateWidget(this, &FSplinePointDetails::OnGenerateComboWidget)
				.OnSelectionChanged(this, &FSplinePointDetails::OnSplinePointTypeChanged)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &FSplinePointDetails::GetPointType)
				]
			];
	}

	TryAcquireDetailsInterface();
	if (SplineDetailsInterface && SplineDetailsInterface->GetSelectedKeys().Num() > 0)
	{
		for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
		{
			if (ClassIterator->IsChildOf(USplineMetadataDetailsFactoryBase::StaticClass()) && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				USplineMetadataDetailsFactoryBase* Factory = ClassIterator->GetDefaultObject<USplineMetadataDetailsFactoryBase>();
				const USplineMetadata* SplineMetadata = SplineComp->GetSplinePointsMetadata();
				if (SplineMetadata && SplineMetadata->GetClass() == Factory->GetMetadataClass())
				{
					SplineMetaDataDetails = Factory->Create();
					IDetailGroup& Group = ChildrenBuilder.AddGroup(SplineMetaDataDetails->GetName(), SplineMetaDataDetails->GetDisplayName());
					SplineMetaDataDetails->GenerateChildContent(Group);
					break;
				}
			}
		}
	}
}

void FSplinePointDetails::Tick(float DeltaTime)
{
	UpdateValues();
}

void FSplinePointDetails::UpdateValues()
{
	TryAcquireDetailsInterface();
	
	// If this is a blueprint spline, always update the spline component based on 
	// the spline component visualizer's currently edited spline component.
	if (SplineCompArchetype)
	{
		USplineComponent* EditedSplineComp = SplineDetailsInterface ? SplineDetailsInterface->GetEditedSplineComponent() : nullptr;

		if (!EditedSplineComp || (EditedSplineComp->GetArchetype() != SplineCompArchetype))
		{
			return;
		}

		SplineComp = EditedSplineComp;
	}

	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	bool bNeedsRebuild = false;
	const TSet<int32>& NewSelectedKeys = SplineDetailsInterface->GetSelectedKeys();

	if (NewSelectedKeys.Num() != SelectedKeys.Num())
	{
		bNeedsRebuild = true;
	}
	SelectedKeys = NewSelectedKeys;

	// Cache values to be shown by the details customization.
	// An unset optional value represents 'multiple values' (in the case where multiple points are selected).
	InputKey.Reset();
	Position.Reset();
	ArriveTangent.Reset();
	LeaveTangent.Reset();
	Rotation.Reset();
	Scale.Reset();
	PointType.Reset();

	// Only display point details when there are selected keys
	if (SelectedKeys.Num() > 0)
	{
		bool bValidIndices = true;
		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= SplineComp->GetNumberOfSplinePoints())
			{
				bValidIndices = false;
				if (!bAlreadyWarnedInvalidIndex)
				{
					UE_LOGF(LogSplineComponentDetails, Error, "Spline component details selected keys contains invalid index %d for spline %ls with %d points",
						Index,
						*SplineComp->GetPathName(),
						SplineComp->GetNumberOfSplinePoints());
					bAlreadyWarnedInvalidIndex = true;
				}
				break;
			}
		}

		if (bValidIndices)
		{
			for (int32 Index : SelectedKeys)
			{
				// possibly could get this data in bulk via GetSplinePoint(Index), but doing a 1:1 swap for now.

				InputKey.Add(SplineComp->GetInputKeyValueAtSplinePoint(Index));

				FVector SelectedSplinePointLocation = SplineComp->GetLocationAtSplinePoint(Index, bEditingLocationAbsolute ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local);

				FQuat SelectedSplinePointQuaternion = SplineComp->GetQuaternionAtSplinePoint(Index, bEditingRotationAbsolute ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local);

				Scale.Add(SplineComp->GetScaleAtSplinePoint(Index));

				FVector SelectedPointArriveTangent = SplineComp->GetArriveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local);
				FVector SelectedPointLeaveTangent = SplineComp->GetLeaveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local);

				PointType.Add(SplineComp->GetSplinePointType(Index));

				if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
				{
					SelectedSplinePointLocation.Y = -SelectedSplinePointLocation.Y;

					SelectedPointArriveTangent.Y = -SelectedPointArriveTangent.Y;

					SelectedPointLeaveTangent.Y = -SelectedPointLeaveTangent.Y;

					FVector SelectedSplinePointEuler = ConvertToLUFEuler(SelectedSplinePointQuaternion);
					Rotation.Add(FRotator(SelectedSplinePointEuler.X, SelectedSplinePointEuler.Y, SelectedSplinePointEuler.Z));
				}
				else
				{
					Rotation.Add(SelectedSplinePointQuaternion.Rotator());
				}

				Position.Add(SelectedSplinePointLocation);
				ArriveTangent.Add(SelectedPointArriveTangent);
				LeaveTangent.Add(SelectedPointLeaveTangent);
			}

			if (SplineMetaDataDetails)
			{
				SplineMetaDataDetails->Update(SplineComp, SelectedKeys);
			}
		}
	}

	if (bNeedsRebuild)
	{
		OnRegenerateChildren.ExecuteIfBound();
	}
}

FName FSplinePointDetails::GetName() const
{
	static const FName Name("SplinePointDetails");
	return Name;
}

bool FSplinePointDetails::CanSetInputKey() const
{
	if (!SplineComp)
	{
		return false;
	}

	return IsOnePointSelected();
}

void FSplinePointDetails::OnSetInputKey(float NewValue, ETextCommit::Type CommitInfo)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TryAcquireDetailsInterface();
	
	if ((CommitInfo != ETextCommit::OnEnter && CommitInfo != ETextCommit::OnUserMovedFocus) || !SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	check(SelectedKeys.Num() == 1);
	const int32 Index = *SelectedKeys.CreateConstIterator();

	const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

	bool bModifyOtherPoints = false;
	if ((Index > 0 && NewValue <= SplineComp->GetInputKeyValueAtSplinePoint(Index - 1)) ||
		(Index < NumPoints - 1 && NewValue >= SplineComp->GetInputKeyValueAtSplinePoint(Index + 1)))
	{
		const FText Title(LOCTEXT("InputKeyTitle", "Input key out of range"));
		const FText Message(LOCTEXT("InputKeyMessage", "Spline input keys must be numerically ascending. Would you like to modify other input keys in the spline in order to be able to set this value?"));

		// Ensure input keys remain ascending
		if (FMessageDialog::Open(EAppMsgType::YesNo, Message, Title) == EAppReturnType::No)
		{
			return;
		}

		bModifyOtherPoints = true;
	}

	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointInputKey", "Set spline point input key"));

		SplineComp->Modify();

		if (bModifyOtherPoints)
		{
			// Shuffle the previous or next input keys down or up so the input value remains in sequence
			if (Index > 0 && NewValue <= SplineComp->GetInputKeyValueAtSplinePoint(Index - 1))
			{
				float Delta = (NewValue - SplineComp->GetInputKeyValueAtSplinePoint(Index));
				for (int32 PrevIndex = 0; PrevIndex < Index; PrevIndex++)
				{
					SplineComp->SetInputKeyValueAtSplinePoint(PrevIndex, SplineComp->GetInputKeyValueAtSplinePoint(PrevIndex) + Delta, false);
				}
			}
			else if (Index < NumPoints - 1 && NewValue >= SplineComp->GetInputKeyValueAtSplinePoint(Index + 1))
			{
				float Delta = (NewValue - SplineComp->GetInputKeyValueAtSplinePoint(Index));
				for (int32 NextIndex = NumPoints - 1; NextIndex >= Index + 1; NextIndex--)
				{
					SplineComp->SetInputKeyValueAtSplinePoint(NextIndex, SplineComp->GetInputKeyValueAtSplinePoint(NextIndex) + Delta, false);
				}
			}
		}

		SplineComp->SetInputKeyValueAtSplinePoint(Index, NewValue, false);
	}

	SplineComp->UpdateSpline();
	SplineComp->bSplineHasBeenEdited = true;
	FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties);
	if (AActor* Owner = SplineComp->GetOwner())
	{
		Owner->PostEditMove(true);
	}
	UpdateValues();

	GEditor->RedrawLevelEditingViewports(true);

PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSplinePointDetails::OnSetPosition(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointPosition", "Set spline point position"), !bInSliderTransaction);
		SplineComp->Modify();

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
		
		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= NumPoints)
			{
				UE_LOGF(LogSplineComponentDetails, Error, "Set spline point location: invalid index %d in selected points for spline component %ls which contains %d spline points.",
					Index, *SplineComp->GetPathName(), NumPoints);
				continue;
			}

			const ESplineCoordinateSpace::Type SplineCoordinateSpace = bEditingLocationAbsolute ? ESplineCoordinateSpace::World : ESplineCoordinateSpace::Local;
			FVector PointPosition = SplineComp->GetLocationAtSplinePoint(Index, SplineCoordinateSpace);
			PointPosition.SetComponentForAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward && Axis == EAxis::Type::Y ? -NewValue : NewValue);
			SplineComp->SetLocationAtSplinePoint(Index, PointPosition, SplineCoordinateSpace, false);
		}
	}

	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();
	}

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplinePointDetails::OnSetArriveTangent(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointTangent", "Set spline point tangent"));
		SplineComp->Modify();

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= NumPoints)
			{
				UE_LOGF(LogSplineComponentDetails, Error, "Set spline point arrive tangent: invalid index %d in selected points for spline component %ls which contains %d spline points.",
					Index, *SplineComp->GetPathName(), NumPoints);
				continue;
			}

			FVector LocalArriveTangent = SplineComp->GetArriveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local);
			LocalArriveTangent.SetComponentForAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward && Axis == EAxis::Type::Y ? -NewValue : NewValue);
			FVector LocalLeaveTangent = SplineComp->bAllowDiscontinuousSpline ? SplineComp->GetLeaveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local) : LocalArriveTangent;

			SplineComp->SetTangentsAtSplinePoint(Index, LocalArriveTangent, LocalLeaveTangent, ESplineCoordinateSpace::Local, false);

		}
	}

	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();
	}

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplinePointDetails::OnSetLeaveTangent(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointTangent", "Set spline point tangent"));
		SplineComp->Modify();

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= NumPoints)
			{
				UE_LOGF(LogSplineComponentDetails, Error, "Set spline point leave tangent: invalid index %d in selected points for spline component %ls which contains %d spline points.",
					Index, *SplineComp->GetPathName(), NumPoints);
				continue;
			}

			FVector LocalLeaveTangent = SplineComp->GetLeaveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local);
			LocalLeaveTangent.SetComponentForAxis(Axis, AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward && Axis == EAxis::Type::Y ? -NewValue : NewValue);
			FVector LocalArriveTangent = SplineComp->bAllowDiscontinuousSpline ? SplineComp->GetArriveTangentAtSplinePoint(Index, ESplineCoordinateSpace::Local) : LocalLeaveTangent;

			SplineComp->SetTangentsAtSplinePoint(Index, LocalArriveTangent, LocalLeaveTangent, ESplineCoordinateSpace::Local, false);
		}
	}

	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();
	}

	GEditor->RedrawLevelEditingViewports(true);
}

void FSplinePointDetails::OnSetRotation(FRotator::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	FQuat NewRotationRelative;
	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointRotation", "Set spline point rotation"));
		SplineComp->Modify();

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();

		FQuat SplineComponentRotation = SplineComp->GetComponentQuat();
		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= NumPoints)
			{
				UE_LOGF(LogSplineComponentDetails, Error, "Set spline point rotation: invalid index %d in selected points for spline component %ls which contains %d spline points.",
					Index, *SplineComp->GetPathName(), NumPoints);
				continue;
			}

			const FQuat CurrentRotationRelative = SplineComp->GetQuaternionAtSplinePoint(Index, ESplineCoordinateSpace::Local);

			if (bEditingRotationAbsolute)
			{
				FQuat CurrentAbsoluteRot = SplineComponentRotation * CurrentRotationRelative;

				if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
				{
					FVector CurrentComponentValue = ConvertToLUFEuler(CurrentAbsoluteRot);

					switch (Axis)
					{
					case EAxis::X: CurrentComponentValue.Z = NewValue; break;
					case EAxis::Y: CurrentComponentValue.X = NewValue; break;
					case EAxis::Z: CurrentComponentValue.Y = NewValue; break;
					}

					CurrentAbsoluteRot = ConvertFromLUFEuler(CurrentComponentValue);
				}
				else
				{
					FRotator CurrentComponentValue = (SplineComponentRotation * CurrentRotationRelative).Rotator();

					switch (Axis)
					{
					case EAxis::X: CurrentComponentValue.Roll = NewValue; break;
					case EAxis::Y: CurrentComponentValue.Pitch = NewValue; break;
					case EAxis::Z: CurrentComponentValue.Yaw = NewValue; break;
					}

					CurrentAbsoluteRot = CurrentComponentValue.Quaternion();
				}

				NewRotationRelative = SplineComponentRotation.Inverse() * CurrentAbsoluteRot;
			}
			else
			{

				if (AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward)
				{
					FVector CurrentComponentValue = ConvertToLUFEuler(CurrentRotationRelative);

					switch (Axis)
					{
					case EAxis::X: CurrentComponentValue.Z = NewValue; break;
					case EAxis::Y: CurrentComponentValue.X = NewValue; break;
					case EAxis::Z: CurrentComponentValue.Y = NewValue; break;
					}

					NewRotationRelative = ConvertFromLUFEuler(CurrentComponentValue);
				}
				else
				{
					FRotator NewRotationRotator(CurrentRotationRelative);

					switch (Axis)
					{
					case EAxis::X: NewRotationRotator.Roll = NewValue; break;
					case EAxis::Y: NewRotationRotator.Pitch = NewValue; break;
					case EAxis::Z: NewRotationRotator.Yaw = NewValue; break;
					}

					NewRotationRelative = NewRotationRotator.Quaternion();
				}
			}

			SplineComp->SetQuaternionAtSplinePoint(Index, NewRotationRelative, ESplineCoordinateSpace::Local);
		}
	}

	SplineDetailsInterface->SetCachedRotation(NewRotationRelative);

	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();
	}
	GEditor->RedrawLevelEditingViewports(true);
}

void FSplinePointDetails::OnSetScale(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAxis::Type Axis)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSplinePointScale", "Set spline point scale"));
		SplineComp->Modify();

		const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
		
		for (int32 Index : SelectedKeys)
		{
			if (Index < 0 || Index >= NumPoints)
			{
				UE_LOGF(LogSplineComponentDetails, Error, "Set spline point scale: invalid index %d in selected points for spline component %ls which contains %d spline points.",
					Index, *SplineComp->GetPathName(), NumPoints);
				continue;
			}

			FVector PointScale = SplineComp->GetScaleAtSplinePoint(Index);
			PointScale.SetComponentForAxis(Axis, NewValue);
			SplineComp->SetScaleAtSplinePoint(Index, PointScale, false);
		}
	}

	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties, EPropertyChangeType::ValueSet);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();
	}

	GEditor->RedrawLevelEditingViewports(true);
}

FText FSplinePointDetails::GetPointType() const
{
	if (PointType.Value.IsSet())
	{
		const UEnum* SplinePointTypeEnum = StaticEnum<ESplinePointType::Type>();
		check(SplinePointTypeEnum);
		return SplinePointTypeEnum->GetDisplayNameTextByValue(PointType.Value.GetValue());
	}

	return LOCTEXT("MultipleTypes", "Multiple Types");
}

void FSplinePointDetails::OnSplinePointTypeChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	TryAcquireDetailsInterface();
	
	if (!SplineComp || !SplineDetailsInterface)
	{
		return;
	}

	bool bWasModified = false;
	// Scope the transaction to only include the value change and none of the derived data changes that might arise from NotifyPropertyModified
	{
		EInterpCurveMode Mode = CIM_Unknown;
		if (NewValue.IsValid() && SplinePointTypes.ContainsByPredicate([&NewValue](const TSharedPtr<FString>& InSplinePointType) { return (*InSplinePointType == *NewValue); }))
		{
			const UEnum* SplinePointTypeEnum = StaticEnum<ESplinePointType::Type>();
			check(SplinePointTypeEnum);
			const int64 SplinePointType = SplinePointTypeEnum->GetValueByNameString(*NewValue);

			Mode = ConvertSplinePointTypeToInterpCurveMode(static_cast<ESplinePointType::Type>(SplinePointType));
			check(Mode != CIM_Unknown);

			const FScopedTransaction Transaction(LOCTEXT("SetSplinePointType", "Set spline point type"));
			SplineComp->Modify();

			const int32 NumPoints = SplineComp->GetNumberOfSplinePoints();
			
			for (int32 Index : SelectedKeys)
			{
				if (Index < 0 || Index >= NumPoints)
				{
					UE_LOGF(LogSplineComponentDetails, Error, "Set spline point type: invalid index %d in selected points for spline component %ls which contains %d spline points.",
						Index, *SplineComp->GetPathName(), NumPoints);
					continue;
				}

				if (SplineComp->GetSplinePointType(Index) != ConvertInterpCurveModeToSplinePointType(Mode))
				{
					SplineComp->SetSplinePointType(Index, ConvertInterpCurveModeToSplinePointType(Mode), false);
					bWasModified = true;
				}
			}
		}
	}

	if (bWasModified)
	{
		SplineComp->UpdateSpline();
		SplineComp->bSplineHasBeenEdited = true;
		FComponentVisualizer::NotifyPropertiesModified(SplineComp, SplineProperties);
		if (AActor* Owner = SplineComp->GetOwner())
		{
			Owner->PostEditMove(true);
		}
		UpdateValues();

		GEditor->RedrawLevelEditingViewports(true);
	}
}

USplineComponent* FSplinePointDetails::GetSplineComponentToVisualize() const
{
	if (SplineCompArchetype) 
	{
		check(SplineCompArchetype->IsTemplate());

		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");

		const UClass* BPClass;
		if (const AActor* OwningCDO = SplineCompArchetype->GetOwner())
		{
			// Native component template
			BPClass = OwningCDO->GetClass();
		}
		else
		{
			// Non-native component template
			BPClass = Cast<UClass>(SplineCompArchetype->GetOuter());
		}

		if (BPClass)
		{
			if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(BPClass))
			{
				if (FBlueprintEditor* BlueprintEditor = StaticCast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, false)))
				{
					const AActor* PreviewActor = BlueprintEditor->GetPreviewActor();
					TArray<UObject*> Instances;
					SplineCompArchetype->GetArchetypeInstances(Instances);

					for (UObject* Instance : Instances)
					{
						USplineComponent* SplineCompInstance = Cast<USplineComponent>(Instance);
						if (SplineCompInstance->GetOwner() == PreviewActor)
						{
							return SplineCompInstance;
						}
					}
				}
			}
		}

		// If we failed to find an archetype instance, must return nullptr 
		// since component visualizer cannot visualize the archetype.
		return nullptr;
	}

	return SplineComp;
}

FReply FSplinePointDetails::OnSelectFirstLastSplinePoint(bool bFirst)
{
	TryAcquireDetailsInterface();
	
	if (SplineDetailsInterface)
	{
		if (!SplineComp)
		{
			SplineComp = GetSplineComponentToVisualize();
		}

		if (SplineComp)
		{
			if (SplineDetailsInterface->HandleSelectFirstLastSplinePoint(SplineComp, bFirst))
			{
				// SplineComp can be cached here while the component visualizer is not the active visualizer,
				// so always activate -- the previous "only on fresh acquisition" gate was unsound.
				SplineDetailsInterface->ActivateVisualization();
			}
		}
	}
	return FReply::Handled();
}

FReply FSplinePointDetails::OnSelectPrevNextSplinePoint(bool bNext, bool bAddToSelection)
{
	TryAcquireDetailsInterface();
	
	if (SplineDetailsInterface)
	{
		SplineDetailsInterface->HandleSelectPrevNextSplinePoint(bNext, bAddToSelection);
	}
	return FReply::Handled();
}

FReply FSplinePointDetails::OnSelectAllSplinePoints()
{
	TryAcquireDetailsInterface();

	if (SplineDetailsInterface)
	{
		if (!SplineComp)
		{
			SplineComp = GetSplineComponentToVisualize();
		}

		if (SplineComp)
		{
			if (SplineDetailsInterface->HandleSelectAllSplinePoints(SplineComp))
			{
				SplineDetailsInterface->ActivateVisualization();
			}
		}
	}
	return FReply::Handled();
}

TSharedRef<SWidget> FSplinePointDetails::OnGenerateComboWidget(TSharedPtr<FString> InComboString)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InComboString))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

TSharedRef<SWidget> FSplinePointDetails::BuildSplinePointPropertyLabel(ESplinePointProperty SplinePointProp)
{
	FText Label;
	switch (SplinePointProp)
	{
	case ESplinePointProperty::Rotation:
		Label = LOCTEXT("RotationLabel", "Rotation");
		break;
	case ESplinePointProperty::Location:
		Label = LOCTEXT("LocationLabel", "Location");
		break;
	default:
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, NULL, NULL);

	FUIAction SetRelativeLocationAction
	(
		FExecuteAction::CreateSP(this, &FSplinePointDetails::OnSetTransformEditingAbsolute, SplinePointProp, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplinePointDetails::IsTransformEditingRelative, SplinePointProp)
	);

	FUIAction SetWorldLocationAction
	(
		FExecuteAction::CreateSP(this, &FSplinePointDetails::OnSetTransformEditingAbsolute, SplinePointProp, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FSplinePointDetails::IsTransformEditingAbsolute, SplinePointProp)
	);

	MenuBuilder.BeginSection(TEXT("TransformType"), FText::Format(LOCTEXT("TransformType", "{0} Type"), Label));

	MenuBuilder.AddMenuEntry
	(
		FText::Format(LOCTEXT("RelativeLabel", "Relative"), Label),
		FText::Format(LOCTEXT("RelativeLabel_ToolTip", "{0} is relative to its parent"), Label),
		FSlateIcon(),
		SetRelativeLocationAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		FText::Format(LOCTEXT("WorldLabel", "World"), Label),
		FText::Format(LOCTEXT("WorldLabel_ToolTip", "{0} is relative to the world"), Label),
		FSlateIcon(),
		SetWorldLocationAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.EndSection();


	return
		SNew(SComboButton)
		.ContentPadding(0.f)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ForegroundColor(FSlateColor::UseForeground())
		.MenuContent()
		[
			MenuBuilder.MakeWidget()
		]
	.ButtonContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(this, &FSplinePointDetails::GetSplinePointPropertyText, SplinePointProp)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		];
}

void FSplinePointDetails::OnSetTransformEditingAbsolute(ESplinePointProperty SplinePointProp, bool bIsAbsolute)
{
	if (SplinePointProp == ESplinePointProperty::Location)
	{
		bEditingLocationAbsolute = bIsAbsolute;
	}
	else if (SplinePointProp == ESplinePointProperty::Rotation)
	{
		bEditingRotationAbsolute = bIsAbsolute;
	}
	else
	{
		return;
	}

	UpdateValues();
}

bool FSplinePointDetails::IsTransformEditingAbsolute(ESplinePointProperty SplinePointProp) const
{
	if (SplinePointProp == ESplinePointProperty::Location)
	{
		return bEditingLocationAbsolute;
	}
	else if (SplinePointProp == ESplinePointProperty::Rotation)
	{
		return bEditingRotationAbsolute;
	}

	return false;
}

bool FSplinePointDetails::IsTransformEditingRelative(ESplinePointProperty SplinePointProp) const
{
	if (SplinePointProp == ESplinePointProperty::Location)
	{
		return !bEditingLocationAbsolute;
	}
	else if (SplinePointProp == ESplinePointProperty::Rotation)
	{
		return !bEditingRotationAbsolute;
	}

	return false;
}


FText FSplinePointDetails::GetSplinePointPropertyText(ESplinePointProperty SplinePointProp) const
{
	if (SplinePointProp == ESplinePointProperty::Location)
	{
		return bEditingLocationAbsolute ? LOCTEXT("AbsoluteLocation", "Absolute Location") : LOCTEXT("Location", "Location");
	}
	else if (SplinePointProp == ESplinePointProperty::Rotation)
	{
		return bEditingRotationAbsolute ? LOCTEXT("AbsoluteRotation", "Absolute Rotation") : LOCTEXT("Rotation", "Rotation");
	}

	return FText::GetEmpty();
}

void FSplinePointDetails::SetSplinePointProperty(ESplinePointProperty SplinePointProp, FVector3f NewValue, EAxisList::Type Axis, bool bCommitted)
{
	switch (SplinePointProp)
	{
	case ESplinePointProperty::Location:
		OnSetPosition(NewValue.X, ETextCommit::Default, EAxis::X);
		OnSetPosition(NewValue.Y, ETextCommit::Default, EAxis::Y);
		OnSetPosition(NewValue.Z, ETextCommit::OnEnter, EAxis::Z);
		break;
	case ESplinePointProperty::Rotation:
		OnSetRotation(NewValue.X, ETextCommit::Default, EAxis::X);
		OnSetRotation(NewValue.Y, ETextCommit::Default, EAxis::Y);
		OnSetRotation(NewValue.Z, ETextCommit::OnEnter, EAxis::Z);
		break;
	case ESplinePointProperty::Scale:
		OnSetScale(NewValue.X, ETextCommit::Default, EAxis::X);
		OnSetScale(NewValue.Y, ETextCommit::Default, EAxis::Y);
		OnSetScale(NewValue.Z, ETextCommit::OnEnter, EAxis::Z);
		break;
	case ESplinePointProperty::ArriveTangent:
		OnSetArriveTangent(NewValue.X, ETextCommit::Default, EAxis::X);
		OnSetArriveTangent(NewValue.Y, ETextCommit::Default, EAxis::Y);
		OnSetArriveTangent(NewValue.Z, ETextCommit::OnEnter, EAxis::Z);
		break;
	case ESplinePointProperty::LeaveTangent:
		OnSetLeaveTangent(NewValue.X, ETextCommit::Default, EAxis::X);
		OnSetLeaveTangent(NewValue.Y, ETextCommit::Default, EAxis::Y);
		OnSetLeaveTangent(NewValue.Z, ETextCommit::OnEnter, EAxis::Z);
		break;
	case ESplinePointProperty::Type:
		checkf(false, TEXT("SetSplinePointProperty shouldn't be called for non-vector types"));
		break;
	default:
		break;
	}
}

FUIAction FSplinePointDetails::CreateCopyAction(ESplinePointProperty SplinePointProp)
{
	return
		FUIAction
		(
			FExecuteAction::CreateSP(this, &FSplinePointDetails::OnCopy, SplinePointProp),
			FCanExecuteAction::CreateSP(this, &FSplinePointDetails::OnCanCopy, SplinePointProp)
		);
}

FUIAction FSplinePointDetails::CreatePasteAction(ESplinePointProperty SplinePointProp)
{
	return
		FUIAction(FExecuteAction::CreateSP(this, &FSplinePointDetails::OnPaste, SplinePointProp));
}

bool FSplinePointDetails::OnCanCopy(ESplinePointProperty SplinePointProp) const
{
	// Can't copy if at least one of spline point's values is different (we're editing multiple values) : 
	switch (SplinePointProp)
	{
	case ESplinePointProperty::Location:
		return Position.IsValid() && !Position.HasMultipleValues();
	case ESplinePointProperty::Rotation:
		return Rotation.IsValid() && !Rotation.HasMultipleValues();
	case ESplinePointProperty::Scale:
		return Scale.IsValid() && !Scale.HasMultipleValues();
	case ESplinePointProperty::ArriveTangent:
		return ArriveTangent.IsValid() && !ArriveTangent.HasMultipleValues();
	case ESplinePointProperty::LeaveTangent:
		return LeaveTangent.IsValid() && !LeaveTangent.HasMultipleValues();
	case ESplinePointProperty::Type:
		return PointType.IsValid() && !PointType.HasMultipleValues();
	default:
		return false;
	}
}

void FSplinePointDetails::OnCopy(ESplinePointProperty SplinePointProp)
{
	FString CopyStr;
	switch (SplinePointProp)
	{
	case ESplinePointProperty::Location:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), Position.X.GetValue(), Position.Y.GetValue(), Position.Z.GetValue());
		break;
	case ESplinePointProperty::Rotation:
		CopyStr = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), Rotation.Pitch.GetValue(), Rotation.Yaw.GetValue(), Rotation.Roll.GetValue());
		break;
	case ESplinePointProperty::Scale:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), Scale.X.GetValue(), Scale.Y.GetValue(), Scale.Z.GetValue());
		break;
	case ESplinePointProperty::ArriveTangent:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), ArriveTangent.X.GetValue(), ArriveTangent.Y.GetValue(), ArriveTangent.Z.GetValue());
		break;
	case ESplinePointProperty::LeaveTangent:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), LeaveTangent.X.GetValue(), LeaveTangent.Y.GetValue(), LeaveTangent.Z.GetValue());
		break;
	case ESplinePointProperty::Type:
	{
		FString TypeString = UEnum::GetValueAsString(PointType.Value.GetValue());
		int32 LastColonPos;
		if (TypeString.FindLastChar(':', LastColonPos))
		{
			check((LastColonPos + 1) < TypeString.Len());
			CopyStr = TypeString.RightChop(LastColonPos + 1);
		}
		break;
	}
	default:
		break;
	}

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FSplinePointDetails::OnPaste(ESplinePointProperty SplinePointProp)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	PasteFromText(TEXT(""), PastedText, SplinePointProp);
}

void FSplinePointDetails::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId,
	ESplinePointProperty SplinePointProp)
{
	PasteFromText(InTag, InText, SplinePointProp);
}

void FSplinePointDetails::PasteFromText(
	const FString& InTag,
	const FString& InText,
	ESplinePointProperty SplinePointProp)
{
	FString PastedText = InText;
	switch (SplinePointProp)
	{
	case ESplinePointProperty::Location:
		{
			FVector3f NewLocation;
			if (NewLocation.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteLocation", "Paste Location"));
				SetSplinePointProperty(ESplinePointProperty::Location, NewLocation, EAxisList::All, true);
			}
			break;
		}
	case ESplinePointProperty::Rotation:
		{
			FVector3f NewRotation;
			PastedText.ReplaceInline(TEXT("Pitch="), TEXT("X="));
			PastedText.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
			PastedText.ReplaceInline(TEXT("Roll="), TEXT("Z="));
			if (NewRotation.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
				SetSplinePointProperty(ESplinePointProperty::Rotation, NewRotation, EAxisList::All, true);
			}
			break;
		}
	case ESplinePointProperty::Scale:
		{
			FVector3f NewScale;
			if (NewScale.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteScale", "Paste Scale"));
				SetSplinePointProperty(ESplinePointProperty::Scale, NewScale, EAxisList::All, true);
			}
			break;
		}
	case ESplinePointProperty::ArriveTangent:
		{
			FVector3f NewArrive;
			if (NewArrive.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteArriveTangent", "Paste Arrive Tangent"));
				SetSplinePointProperty(ESplinePointProperty::ArriveTangent, NewArrive, EAxisList::All, true);
			}
			break;
		}
	case ESplinePointProperty::LeaveTangent:
		{
			FVector3f NewLeave;
			if (NewLeave.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteLeaveTangent", "Paste Leave Tangent"));
				SetSplinePointProperty(ESplinePointProperty::LeaveTangent, NewLeave, EAxisList::All, true);
			}
			break;
		}
	case ESplinePointProperty::Type:
	{
		ESelectInfo::Type DummySelectInfo = ESelectInfo::Direct;
		OnSplinePointTypeChanged(MakeShared<FString>(InText), DummySelectInfo);
		break;
	}
	default:
		break;
	}
}

void FSplinePointDetails::OnBeginPositionSlider()
{
	bInSliderTransaction = true;
	SplineComp->Modify();
	GEditor->BeginTransaction(LOCTEXT("SetSplinePointPosition", "Set spline point position"));
}

void FSplinePointDetails::OnBeginScaleSlider()
{
	bInSliderTransaction = true;
	SplineComp->Modify();
	GEditor->BeginTransaction(LOCTEXT("SetSplinePointScale", "Set spline point scale"));
}

void FSplinePointDetails::OnEndSlider(FVector::FReal)
{
	bInSliderTransaction = false;
	GEditor->EndTransaction();
}

FVector FSplinePointDetails::ConvertToLUFEuler(const FQuat& RotationQuaternion) const
{
	const TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> VerseEulerRads = RotationQuaternion.GetNormalized().ToLUFEuler();

	// Since the value is converted from quaternion, will likely have denormals.  Clamp those values.
	auto SanitizeFloat = [](FQuat::FReal Val)->FQuat::FReal
		{
			if (FMath::IsNearlyZero(Val))
			{
				return 0.0;
			}
			return Val;
		};

	const FVector VerseEulerRadsV
	{
		SanitizeFloat(VerseEulerRads.Get<0>()),
		SanitizeFloat(VerseEulerRads.Get<1>()),
		SanitizeFloat(VerseEulerRads.Get<2>())
	};
	return FMath::RadiansToDegrees(VerseEulerRadsV);
}

FQuat FSplinePointDetails::ConvertFromLUFEuler(const FVector& RotationEuler) const
{
	const FVector RotationRads = FMath::DegreesToRadians(RotationEuler);

	const TTuple<FQuat::FReal, FQuat::FReal, FQuat::FReal> RotationRadsT =
	{
		RotationRads.X,
		RotationRads.Y,
		RotationRads.Z
	};


	FQuat Quat = FQuat::MakeFromLUFEuler(RotationRadsT);
	Quat.Normalize();
	return Quat;
}

////////////////////////////////////

TSharedRef<IDetailCustomization> FSplineComponentDetails::MakeInstance()
{
	return MakeShareable(new FSplineComponentDetails);
}

void FSplineComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Hide all spline properties
	for (FName& PropertyName : USplineComponent::GetSplinePropertyNames())
	{
		TSharedPtr<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyName);
		Property->MarkHiddenByCustomization();
	}

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() == 1)
	{
		if (USplineComponent* SplineComp = Cast<USplineComponent>(ObjectsBeingCustomized[0]))
		{
			// Set the spline points details as important in order to have it on top 
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Selected Points", FText::GetEmpty(), ECategoryPriority::Important);
			TSharedRef<FSplinePointDetails> SplinePointDetails = MakeShareable(new FSplinePointDetails(SplineComp));
			Category.AddCustomBuilder(SplinePointDetails);
		}
	}
}

#undef LOCTEXT_NAMESPACE
