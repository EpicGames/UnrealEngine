// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshActorDetails.h"

#include "Containers/Array.h"
#include "Components/StaticMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/StaticMeshActor.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IMeshMergeUtilities.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "MeshMergeModule.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class FUICommandList;
class UStaticMeshComponent;

#define LOCTEXT_NAMESPACE "StaticMeshActorDetails"

TSharedRef<IDetailCustomization> FStaticMeshActorDetails::MakeInstance()
{
	return MakeShareable( new FStaticMeshActorDetails );
}

void FStaticMeshActorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );

	const FLevelEditorCommands& Commands = LevelEditor.GetLevelEditorCommands();
	TSharedRef<const FUICommandList> CommandBindings = LevelEditor.GetGlobalLevelEditorActions();

	FMenuBuilder BlockingVolumeBuilder( true, CommandBindings );
	{
		BlockingVolumeBuilder.BeginSection("StaticMeshActorDetailsBlockingVolume");
		{
			BlockingVolumeBuilder.AddMenuEntry( Commands.CreateBoundingBoxVolume, NAME_None, LOCTEXT("CreateBlockingVolume", "Blocking Volume")  );
		}
		BlockingVolumeBuilder.EndSection();

		BlockingVolumeBuilder.BeginSection("StaticMeshActorDetailsBlockingVolume2");
		{
			BlockingVolumeBuilder.AddMenuEntry( Commands.CreateHeavyConvexVolume, NAME_None, LOCTEXT("CreateHeavyConvexVolume", "Heavy Convex Volume") );
			BlockingVolumeBuilder.AddMenuEntry( Commands.CreateNormalConvexVolume, NAME_None,LOCTEXT("CreateNormalConvexVolume", "Normal Convex Volume") );
			BlockingVolumeBuilder.AddMenuEntry( Commands.CreateLightConvexVolume, NAME_None, LOCTEXT("CreateLightConvexVolume", "Light Convex Volume") );
			BlockingVolumeBuilder.AddMenuEntry( Commands.CreateRoughConvexVolume, NAME_None, LOCTEXT("CreateRoughConvexVolume", "Rough Convex Volume") );
		}
		BlockingVolumeBuilder.EndSection();
	}

	IDetailCategoryBuilder& StaticMeshCategory = DetailBuilder.EditCategory( "StaticMesh" );

	// The blocking volume menu is advanced
	const bool bForAdvanced = true;

	const FText CreateBlockingVolumeString = LOCTEXT("BlockingVolumeMenu", "Create Blocking Volume");

	StaticMeshCategory.AddCustomRow( CreateBlockingVolumeString, bForAdvanced )
	.NameContent()
	[
		SNullWidget::NullWidget
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	.MaxDesiredWidth(250)
	[
		SNew(SComboButton)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("CreateBlockingVolumeTooltip", "Creates a blocking volume from the static mesh"))
		.ButtonContent()
		[
			SNew( STextBlock )
			.Text( CreateBlockingVolumeString ) 
			.Font( IDetailLayoutBuilder::GetDetailFont() )
		]
		.MenuContent()
		[
			BlockingVolumeBuilder.MakeWidget()
		]
	];


	// Retrieve all currently selected static mesh components
	auto RetrieveSelectedStaticMeshComponents = [](const TArray<TWeakObjectPtr<UObject>>& SelectedObjects)
	{		
		TArray<UStaticMeshComponent*> StaticMeshComponents;
		for (TWeakObjectPtr<UObject> WeakObject : SelectedObjects)
		{
			if (WeakObject.IsValid())
			{
				UObject* Object = WeakObject.Get();
				if (AStaticMeshActor* Actor = Cast<AStaticMeshActor>(Object))
				{
					if (UStaticMeshComponent* SMC = Actor->GetStaticMeshComponent())
					{
						if (SMC->GetStaticMesh())
						{
							StaticMeshComponents.Add(SMC);
						}
					}
				}
			}
		}

		return StaticMeshComponents;
	};

	// This allows baking out the materials for the given instance data	
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	IDetailCategoryBuilder& MaterialsCategory = DetailBuilder.EditCategory("Materials");
	FDetailWidgetRow& ButtonRow = MaterialsCategory.AddCustomRow(LOCTEXT("RowLabel", "BakeMaterials"), true);
	ButtonRow.ValueWidget
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SButton)
			.Text(LOCTEXT("BakeLabel", "Bake Materials"))
			.OnClicked_Lambda([RetrieveSelectedStaticMeshComponents, Objects]() -> FReply
			{
				const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
				TArray<UStaticMeshComponent*> StaticMeshComponents = RetrieveSelectedStaticMeshComponents(Objects);
				for (UStaticMeshComponent* Component : StaticMeshComponents)
				{
					MeshMergeUtilities.BakeMaterialsForComponent(Component);
				}
				
				return FReply::Handled();
			})
			.IsEnabled_Lambda([RetrieveSelectedStaticMeshComponents, Objects]() -> bool
			{
				return !RetrieveSelectedStaticMeshComponents(Objects).IsEmpty();
			})
		]
	];		
	
}

#undef LOCTEXT_NAMESPACE
