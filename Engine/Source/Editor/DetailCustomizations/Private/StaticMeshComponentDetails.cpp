// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshComponentDetails.h"

#include "Components/StaticMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/StaticMesh.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "StaticMeshComponentDetails"


TSharedRef<IDetailCustomization> FStaticMeshComponentDetails::MakeInstance()
{
	return MakeShareable( new FStaticMeshComponentDetails );
}

void FStaticMeshComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Create a category so this is displayed early in the properties
	IDetailCategoryBuilder& StaticMeshCategory = DetailBuilder.EditCategory("StaticMesh", FText::GetEmpty(), ECategoryPriority::Important);

	// Customize the StaticMesh property to add a refresh button via CustomButtonSlot
	StaticMeshProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStaticMeshComponent, StaticMesh));
	if (StaticMeshProperty->IsValidHandle())
	{
		IDetailPropertyRow& StaticMeshRow = StaticMeshCategory.AddProperty(StaticMeshProperty.ToSharedRef());

		TSharedPtr<IPropertyHandle> StaticMeshHandle = StaticMeshProperty;
		TSharedRef<SObjectPropertyEntryBox> ObjectEntryBox =
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(StaticMeshProperty)
			.AllowedClass(UStaticMesh::StaticClass())
			.ThumbnailPool(DetailBuilder.GetThumbnailPool())
			.CustomButtonSlot()
			[
				PropertyCustomizationHelpers::MakeCustomButton(
					FAppStyle::Get().GetBrush("Icons.WarningWithColor"),
					FSimpleDelegate::CreateLambda([StaticMeshHandle]()
					{
						if (!StaticMeshHandle.IsValid())
						{
							return;
						}

						TArray<UObject*> OuterObjects;
						StaticMeshHandle->GetOuterObjects(OuterObjects);

						TArray<UMeshComponent*> ComponentsToClean;
						for (UObject* Obj : OuterObjects)
						{
							if (UMeshComponent* MeshComp = Cast<UMeshComponent>(Obj))
							{
								if (MeshComp->GetNumOverrideMaterials() > MeshComp->GetNumMaterials())
								{
									ComponentsToClean.Add(MeshComp);
								}
							}
						}

						if (ComponentsToClean.Num() > 0)
						{
							FScopedTransaction Transaction(LOCTEXT("CleanUpOverrideMaterials", "Clean Up Override Materials"));

							for (UMeshComponent* MeshComp : ComponentsToClean)
							{
								MeshComp->Modify();
								MeshComp->CleanUpOverrideMaterials();
							}
						}
					}),
					LOCTEXT("CleanUpOverrideMaterialsToolTip", "Clean up override materials that exceed the mesh material count"),
					true,
					TAttribute<EVisibility>::CreateLambda([StaticMeshHandle]()
					{
						if (!StaticMeshHandle.IsValid())
						{
							return EVisibility::Collapsed;
						}

						TArray<UObject*> OuterObjects;
						StaticMeshHandle->GetOuterObjects(OuterObjects);

						for (UObject* Obj : OuterObjects)
						{
							if (UMeshComponent* MeshComp = Cast<UMeshComponent>(Obj))
							{
								if (MeshComp->GetNumOverrideMaterials() > MeshComp->GetNumMaterials())
								{
									return EVisibility::Visible;
								}
							}
						}

						return EVisibility::Collapsed;
					}))
			];

		float MinWidth, MaxWidth;
		ObjectEntryBox->GetDesiredWidth(MinWidth, MaxWidth);

		StaticMeshRow.CustomWidget()
			.NameContent()
			[
				StaticMeshProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(MinWidth)
			.MaxDesiredWidth(MaxWidth)
			[
				ObjectEntryBox
			];
	}

	TSharedRef<IPropertyHandle> UseDefaultCollision = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStaticMeshComponent, bUseDefaultCollision));
	UseDefaultCollision->MarkHiddenByCustomization();

	IDetailCategoryBuilder& LightingCategory = DetailBuilder.EditCategory("Lighting");

	// Store off the properties for analysis in later function calls.
	OverrideLightResProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStaticMeshComponent, OverriddenLightMapRes));

	// Add the row that we will be customizing below.
	IDetailPropertyRow& OverrideLightResRow = LightingCategory.AddProperty(OverrideLightResProperty);

	// Create the default property value widget, we'll use this in certain circumstances outlined below.
	TSharedRef<SWidget> ValueWidget = OverrideLightResProperty->CreatePropertyValueWidget();

	// Store off the objects that we are editing for analysis in later function calls.
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	// We use similar logic here to where it is ultimately used:
	//   UStaticMeshComponent::GetLightMapResolution
	// If there is a static mesh and the override is enabled, use the real value of the property.
	// If there is a static mesh and the override is disabled, use the static mesh's resolution value.
	// If no static mesh is assigned, use 0.
	// Ultimately, the last case is an error and we need to warn the user. We also need to handle multiple selection appropriately, 
	// thus the widget switcher below.
	
	OverrideLightResRow.CustomWidget()
		.NameContent()
		[
			// Use the default lightmap property name
			OverrideLightResProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FStaticMeshComponentDetails::HandleNoticeSwitcherWidgetIndex)
			+ SWidgetSwitcher::Slot() // Slot 0, the single common value for static mesh lightmap resolutions.
			[
				SNew(SNumericEntryBox<int32>)
				.ToolTipText(OverrideLightResProperty->GetToolTipText())
				.IsEnabled(false)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Value(this, &FStaticMeshComponentDetails::GetStaticMeshLightResValue)
			]
			+ SWidgetSwitcher::Slot() // Slot 1, the editor for when overrides are enabled
			[
				ValueWidget
			]
			+ SWidgetSwitcher::Slot() // Slot 2, the editor for when we are missing one or more static meshes.
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTipText(OverrideLightResProperty->GetToolTipText())
				.IsEnabled(false)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("DetailsMissingStaticMesh", "Missing StaticMesh!"))
			]
			+ SWidgetSwitcher::Slot() // Slot 3, the editor for when we have multiple heterogenous values.
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ToolTipText(OverrideLightResProperty->GetToolTipText())
				.IsEnabled(false)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("DetailsMultipleValues", "Multiple Values"))
			]
		];

}

TOptional<int32> FStaticMeshComponentDetails::GetStaticMeshLightResValue() const
{
	// Go ahead and grab the current value for the static mesh resolution as this shouldn't change during the lifetime
	// of this customization.
	TOptional<int32> DefaultResolution = 0;
	for (int32 i = 0; i < ObjectsCustomized.Num() && DefaultResolution == 0; i++)
	{
		UStaticMeshComponent*  Component = Cast<UStaticMeshComponent>(ObjectsCustomized[i].Get());
		if (Component != nullptr && Component->GetStaticMesh() != nullptr)
		{
			DefaultResolution = Component->GetStaticMesh()->GetLightMapResolution();
		}
	}
	return DefaultResolution;
}

int FStaticMeshComponentDetails::HandleNoticeSwitcherWidgetIndex() const
{
	// Determine if the override enabled is set homogenously or not.
	bool bOverrideEnabled = false;
	FPropertyAccess::Result AccessResultOverrideEnabled = FPropertyAccess::Fail;
	bool bOverrideFound = false;

	// Loop through all the components being edited to see if any are missing static meshes
	// and to see if their lightmap resolutions are all the same on those static meshes.
	FPropertyAccess::Result AccessResultOverride = FPropertyAccess::Success;
	bool bHasMissingStaticMeshes = false;
	int32 OverrideLightRes = -1;
	check(ObjectsCustomized.Num() > 0);
	for (int32 i = 0; i < ObjectsCustomized.Num(); i++)
	{
		UStaticMeshComponent*  Component = Cast<UStaticMeshComponent>(ObjectsCustomized[i].Get());
		if (Component != nullptr)
		{
			if (Component->GetStaticMesh() == nullptr)
			{
				bHasMissingStaticMeshes = true;
				AccessResultOverride = FPropertyAccess::MultipleValues;
			}
			else
			{
				if (bOverrideFound == false)
				{
					AccessResultOverrideEnabled = FPropertyAccess::Success;
					bOverrideEnabled = Component->bOverrideLightMapRes;
					bOverrideFound = true;
				}
				else if (bOverrideEnabled != Component->bOverrideLightMapRes)
				{
					AccessResultOverrideEnabled = FPropertyAccess::MultipleValues;
				}

				int32 CurrentStaticLightRes = Component->GetStaticMesh()->GetLightMapResolution();
				if (OverrideLightRes == -1)
				{
					OverrideLightRes = CurrentStaticLightRes;
				}
				else if (CurrentStaticLightRes != OverrideLightRes)
				{
					AccessResultOverride = FPropertyAccess::MultipleValues;
				}
			}
		}
	}

	// This gets a little hairy. The desired behavior is as follows:
	// For single item selection:
	//     a) if override is enabled and we have a valid static mesh, display the general editor for the property.
	//     b) if override is disabled and we have a valid static mesh, display the static mesh's value for the property.
	//     c) otherwise, warn the user if they are missing the static mesh.
	// For multiple selection:
	//     d) if all overrides are enabled, all have valid meshes, and all have the same value, display the general editor.
	//     e) if all overrides are enabled, all have valid meshes, and all have different values, display the general editor (which should say Multiple Values).
	//     f) if all overrides are disabled, all have valid meshes, and all static meshes have the same value for the resolution, display that common resolution
	//     g) if all overrides are disabled, all have valid meshes, and all static meshes have heterogenous values for the resolution, display the multiple values read-only text.
	//     h) if overrides are heterogenous, display the multiple values read-only text.
	//     i) if any of the above have invalid static meshes, warn the user.
	if (bHasMissingStaticMeshes) // Covers cases c and i above.
	{
		return 2;
	}
	else if (AccessResultOverrideEnabled == FPropertyAccess::MultipleValues) // covers case h above
	{
		return 3;
	}
	else if (AccessResultOverride == FPropertyAccess::MultipleValues && bOverrideEnabled == false) // covers case g above
	{
		return 3;
	}
	else if (bOverrideEnabled == false) // covers cases b and f above
	{
		return 0;
	}
	else // covers cases a, d, and e above
	{
		return 1;
	}
}

#undef LOCTEXT_NAMESPACE
