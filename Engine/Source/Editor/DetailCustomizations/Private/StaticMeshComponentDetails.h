// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class UObject;

class FStaticMeshComponentDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	/**
	 * @return Which UI to show whether or not override lightmap res is enabled
	 */
	int HandleNoticeSwitcherWidgetIndex() const;

	/**
	* @return Accessor for the static mesh's light resolution value.
	*/
	TOptional<int32> GetStaticMeshLightResValue() const;
private:
	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
	TSharedPtr<IPropertyHandle> OverrideLightResProperty;
	TSharedPtr<IPropertyHandle> StaticMeshProperty;
};
