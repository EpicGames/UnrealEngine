// Copyright Epic Games, Inc. All Rights Reserved.

#include "StereoLayerComponentCustomization.h"

#include "DetailLayoutBuilder.h"
#include "Components/StereoLayerComponent.h"

TSharedRef<IDetailCustomization> FStereoLayerComponentCustomization::MakeInstance()
{
	return MakeShareable(new FStereoLayerComponentCustomization);
}

void FStereoLayerComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() != 1)
	{
		return;
	}
	
	const UStereoLayerComponent* StereoLayerComponent = Cast<UStereoLayerComponent>(ObjectsBeingCustomized[0]);
	TSharedRef<IPropertyHandle> SupportsDepthHandle = DetailBuilder.GetProperty(TEXT("bSupportsDepth"));
	if (!ensure(StereoLayerComponent) || !ensure(SupportsDepthHandle->IsValidHandle()))
	{
		return;
	}
	
	WeakComponent = MakeWeakObjectPtr(StereoLayerComponent);

	DetailBuilder.EditDefaultProperty(SupportsDepthHandle)->EditCondition(TAttribute<bool>(this, &FStereoLayerComponentCustomization::ShouldEnableDepthSupport), NULL);
}

bool FStereoLayerComponentCustomization::ShouldEnableDepthSupport() const
{
	return WeakComponent.IsValid() && WeakComponent->GetStereoLayerType() == EStereoLayerType::SLT_WorldLocked;
}
