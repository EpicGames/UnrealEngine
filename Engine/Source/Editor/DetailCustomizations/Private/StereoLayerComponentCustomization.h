// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class UStereoLayerComponent;
class IDetailPropertyRow;

class FStereoLayerComponentCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	[[nodiscard]] bool ShouldEnableDepthSupport() const;
	
	TWeakObjectPtr<const UStereoLayerComponent> WeakComponent;
};
