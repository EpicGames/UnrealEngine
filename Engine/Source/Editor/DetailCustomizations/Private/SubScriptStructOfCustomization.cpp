// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubScriptStructOfCustomization.h"

#include "DetailWidgetRow.h"
#include "EditorClassUtils.h"
#include "HAL/PlatformCrt.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SubScriptStructOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

class IDetailChildrenBuilder;

void FSubScriptStructOfCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	const FString& MetaStructName1 = PropertyHandle->GetMetaData("MetaStruct");
	const FString& MetaStructName2 = PropertyHandle->GetMetaData("BaseStruct");
	const bool bAllowNone = !(PropertyHandle->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);
	const bool bShowTreeView = PropertyHandle->HasMetaData("ShowTreeView");
	const bool bHideViewOptions = PropertyHandle->HasMetaData("HideViewOptions");
	const bool bShowDisplayNames = PropertyHandle->HasMetaData("ShowDisplayNames");
	

	const UScriptStruct* MetaScriptStruct = nullptr;
	if (!MetaStructName1.IsEmpty())
	{
		TArray<const UScriptStruct*> Structs = PropertyCustomizationHelpers::GetStructsFromMetadataString(MetaStructName1);
		if (Structs.Num() >= 1)
		{
			MetaScriptStruct = Structs[0];
		}
	}
	else if (!MetaStructName2.IsEmpty())
	{
		TArray<const UScriptStruct*> Structs = PropertyCustomizationHelpers::GetStructsFromMetadataString(MetaStructName2);
		if (Structs.Num() >= 1)
		{
			MetaScriptStruct = Structs[0];
		}
	}

	HeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0.0f)
	[
		SNew(SStructPropertyEntryBox)
			.MetaStruct(MetaScriptStruct)
			.AllowNone(bAllowNone)
			.HideViewOptions(bHideViewOptions)
			.ShowDisplayNames(bShowDisplayNames)
			.ShowTreeView(bShowTreeView)
			.SelectedStruct(this, &FSubScriptStructOfCustomization::HandleGetScriptStruct)
			.OnSetStruct(this, &FSubScriptStructOfCustomization::HandleSetScriptStruct)
	];
}

void FSubScriptStructOfCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

const UScriptStruct* FSubScriptStructOfCustomization::HandleGetScriptStruct() const
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		check(StructProperty->Struct == TBaseStructure<FSubScriptStructOf>::Get());

		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);

		if (RawData.Num() >= 1 && RawData[0])
		{
			return static_cast<FSubScriptStructOf*>(RawData[0])->Get();
		}
	}
	return nullptr;
}

void FSubScriptStructOfCustomization::HandleSetScriptStruct(const UScriptStruct* NewStruct)
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		check(StructProperty->Struct == TBaseStructure<FSubScriptStructOf>::Get());

		FSubScriptStructOf DefaultSubScriptStructOf;

		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);
		FSubScriptStructOf* PreviousValue = RawData.Num() == 1 ? static_cast<FSubScriptStructOf*>(RawData[0]) : &DefaultSubScriptStructOf;

		FSubScriptStructOf NewValue = const_cast<UScriptStruct*>(NewStruct);

		FString TextValue;
		StructProperty->Struct->ExportText(TextValue, &NewValue, PreviousValue, nullptr, EPropertyPortFlags::PPF_None, nullptr);
		ensure(PropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
	}
}
