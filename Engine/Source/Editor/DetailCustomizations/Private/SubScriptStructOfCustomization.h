// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class UClass;

/**
 * Customizes a SubScriptStructOf reference to look like a UScriptStruct* property
 */
class FSubScriptStructOfCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable(new FSubScriptStructOfCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	/** @return The struct currently set on this reference */
	const UScriptStruct* HandleGetScriptStruct() const;
	/** Set the struct used by this reference */
	void HandleSetScriptStruct(const UScriptStruct* newStruct);

	/** Handle to the property being customized */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
