// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateStringStructCustomization.h"

#include "DetailWidgetRow.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "STemplateStringEditableTextBox.h"
#include "UObject/TemplateString.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define LOCTEXT_NAMESPACE "TemplateStringStructCustomization"

void FTemplateStringStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FTemplateStringStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TemplateStringProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTemplateString, Template));
	check(TemplateStringProperty);

	ResolvedStringProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTemplateString, Resolved));
	check(ResolvedStringProperty);

	StructPropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &FTemplateStringStructCustomization::OnResetToDefault));

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0.0f)
	.MinDesiredWidth(125.0f)
	[
		SNew(STemplateStringEditableTextBox)
		.ToolTipText_Raw(this, &FTemplateStringStructCustomization::GetToolTip)
		.Text_Raw(this, &FTemplateStringStructCustomization::GetText)
		.ResolvedText_Raw(this, &FTemplateStringStructCustomization::GetResolvedText)
		.OnTextChanged(this, &FTemplateStringStructCustomization::SetText)
	];
}

FText FTemplateStringStructCustomization::GetText() const
{
	if(TemplateStringProperty.IsValid())
	{
		FString TemplateString;
		TemplateStringProperty->GetValue(TemplateString);

		return FText::FromString(TemplateString);
	}

	return FText::GetEmpty();
}

FText FTemplateStringStructCustomization::GetResolvedText() const
{
	if(ResolvedStringProperty.IsValid())
	{
		FText ResolvedString;
		ResolvedStringProperty->GetValue(ResolvedString);

		if(!ResolvedString.IsEmpty())
		{
			return ResolvedString;
		}
	}

	// If the ResolvedText isn't available, or is empty, we fallback to the original template/tokenized string
	return GetText();
}

void FTemplateStringStructCustomization::SetText(const FText& InNewText) const
{
	if(TemplateStringProperty.IsValid())
	{
		TemplateStringProperty->SetValue(InNewText.ToString());
	}
}

FText FTemplateStringStructCustomization::GetToolTip() const
{
	static const FText EmptyText = FText::GetEmpty();

	if(TemplateStringProperty.IsValid())
	{
		if(CachedTooltip.IsEmpty())
		{
			FTextBuilder TextBuilder;
			if(TemplateStringProperty->GetParentHandle().IsValid())
			{
				TextBuilder.AppendLine(TemplateStringProperty->GetParentHandle()->GetToolTipText());
			}

			const TArray<FString>& ValidArgs = GetValidArguments();
			if(!ValidArgs.IsEmpty())
			{
				TextBuilder.AppendLine(LOCTEXT("ValidArgs_ToolTipHeading", "Valid Arguments:"));
				TextBuilder.Indent();

				for(const FString& Arg : ValidArgs)
				{
					TextBuilder.AppendLine(Arg);
				}
			}

			CachedTooltip = TextBuilder.ToText();
		}
		
		return CachedTooltip;
	}

	return EmptyText;
}

const TArray<FString>& FTemplateStringStructCustomization::GetValidArguments() const
{
	static TArray<FString> EmptyArray;

	if(TemplateStringProperty.IsValid())
	{
		if(ValidArguments.IsEmpty())
		{
			if(TemplateStringProperty->GetParentHandle()->HasMetaData(TEXT("ValidArgs")))
			{
				const FString ValidArgStr = TemplateStringProperty->GetParentHandle()->GetMetaData(TEXT("ValidArgs"));
				ValidArgStr.ParseIntoArrayWS(ValidArguments, TEXT(","), true);
			}
		}

		return ValidArguments;
	}

	return EmptyArray;
}

void FTemplateStringStructCustomization::OnResetToDefault()
{
	if (TemplateStringProperty.IsValid())
	{
		TemplateStringProperty->ResetToDefault();
	}

	if (ResolvedStringProperty.IsValid())
	{
		ResolvedStringProperty->ResetToDefault();
	}
}

#undef LOCTEXT_NAMESPACE
