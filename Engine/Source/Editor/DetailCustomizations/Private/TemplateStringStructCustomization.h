// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

/**
 * Implements a details view customization for the FTemplateString structure.
 */
class FTemplateStringStructCustomization
	: public IPropertyTypeCustomization
{
public:
	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance( )
	{
		return MakeShareable(new FTemplateStringStructCustomization());
	}

public:
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:
	/** Get the property string value (as FText). */
	FText GetText() const;

	/** Get the property string value (as FText). */
	FText GetResolvedText() const;

	/** Set the property string value (from FText). */
	void SetText(const FText& InNewText) const;

	/** Get/Create tooltip, with list of valid arguments. */	
	FText GetToolTip() const;

	/** Get the list of valid arguments. */
	const TArray<FString>& GetValidArguments() const;

	/** Custom behavior that clears both the Template and Resolved strings. */
	void OnResetToDefault();

private:
	/** Pointer to the string that will be set when changing the path. */
	TSharedPtr<IPropertyHandle> TemplateStringProperty;

	/** Pointer to the stored result of the template string after it's resolved and all args are replaced. */
	TSharedPtr<IPropertyHandle> ResolvedStringProperty;

	/** Cached tooltip, created by GetToolTip(). */
	mutable FText CachedTooltip;

	/** Store the valid arguments for this property. */
	mutable TArray<FString> ValidArguments;
};
