// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextCustomization.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "STextPropertyEditableTextBox.h"

namespace
{
	/** Allows STextPropertyEditableTextBox to edit a property handle */
	class FEditableTextPropertyHandle : public IEditableTextProperty
	{
	public:
		FEditableTextPropertyHandle(const TSharedRef<IPropertyHandle>& InPropertyHandle, const TSharedPtr<IPropertyUtilities>& InPropertyUtilities)
			: PropertyHandle(InPropertyHandle)
			, PropertyUtilities(InPropertyUtilities)
		{
			static const FName NAME_MaxLength = "MaxLength";
			MaxLength = PropertyHandle->IsValidHandle() ? PropertyHandle->GetIntMetaData(NAME_MaxLength) : 0;
		}

		virtual bool IsMultiLineText() const override
		{
			static const FName NAME_MultiLine = "MultiLine";
			return PropertyHandle->IsValidHandle() && PropertyHandle->GetBoolMetaData(NAME_MultiLine);
		}

		virtual bool IsPassword() const override
		{
			static const FName NAME_PasswordField = "PasswordField";
			return PropertyHandle->IsValidHandle() && PropertyHandle->GetBoolMetaData(NAME_PasswordField);
		}

		virtual bool IsReadOnly() const override
		{
			return !PropertyHandle->IsValidHandle() || PropertyHandle->IsEditConst();
		}

		virtual bool IsDefaultValue() const override
		{
			return PropertyHandle->IsValidHandle() && !PropertyHandle->DiffersFromDefault();
		}

		virtual FText GetToolTipText() const override
		{
			return (PropertyHandle->IsValidHandle())
				? PropertyHandle->GetToolTipText()
				: FText::GetEmpty();
		}

		virtual int32 GetNumTexts() const override
		{
			return (PropertyHandle->IsValidHandle())
				? PropertyHandle->GetNumPerObjectValues() 
				: 0;
		}

		virtual FText GetText(const int32 InIndex) const override
		{
			if (PropertyHandle->IsValidHandle())
			{
				FString ObjectValue;
				if (PropertyHandle->GetPerObjectValue(InIndex, ObjectValue) == FPropertyAccess::Success)
				{
					FText TextValue;
					if (FTextStringHelper::ReadFromBuffer(*ObjectValue, TextValue))
					{
						return TextValue;
					}
				}
			}

			return FText::GetEmpty();
		}

		virtual void SetText(const int32 InIndex, const FText& InText) override
		{
			if (PropertyHandle->IsValidHandle())
			{
				FString ObjectValue;
				FTextStringHelper::WriteToBuffer(ObjectValue, InText);
				PropertyHandle->SetPerObjectValue(InIndex, ObjectValue);
			}
		}

		virtual bool IsValidText(const FText& InText, FText& OutErrorMsg) const override
		{
			if (MaxLength > 0 && InText.ToString().Len() > MaxLength)
			{
				OutErrorMsg = FText::Format(NSLOCTEXT("PropertyEditor", "PropertyTextTooLongError", "This value is too long ({0}/{1} characters)"), InText.ToString().Len(), MaxLength);
				return false;
			}

			return true;
		}

#if USE_STABLE_LOCALIZATION_KEYS
		virtual void GetStableTextId(const int32 InIndex, const ETextPropertyEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey) const override
		{
			if (PropertyHandle->IsValidHandle())
			{
				static const FName NAME_UniqueIdWhenEditedOnDefault  = "UniqueIdWhenEditedOnDefault";
				static const FName NAME_UniqueIdWhenEditedOnInstance = "UniqueIdWhenEditedOnInstance";

				UPackage* PropertyPackage = nullptr;
				bool bForceUniqueId = false;

				// We can't query if this property handle is under an object or an external struct, so try and get the objects first before falling back to using the packages
				// Note: We want to use the object where possible so that we can tell if we're editing a CDO/archetype or an instance
				{
					TArray<UObject*> PropertyObjects;
					PropertyHandle->GetOuterObjects(PropertyObjects);

					if (PropertyObjects.IsValidIndex(InIndex) && PropertyObjects[InIndex])
					{
						PropertyPackage = PropertyObjects[InIndex]->GetPackage();
						bForceUniqueId = PropertyHandle->HasMetaData(PropertyObjects[InIndex]->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ? NAME_UniqueIdWhenEditedOnDefault : NAME_UniqueIdWhenEditedOnInstance);
					}
				}
				if (!PropertyPackage || PropertyPackage == GetTransientPackage())
				{
					TArray<UPackage*> PropertyPackages;
					PropertyHandle->GetOuterPackages(PropertyPackages);

					check(PropertyPackages.IsValidIndex(InIndex));
					PropertyPackage = PropertyPackages[InIndex];
				}

				ensureAlwaysMsgf(PropertyPackage, TEXT("A package must be available for key stabilization to work correctly. Did you forget to set the package on a FStructOnScope?"));
				StaticStableTextId(PropertyPackage, InEditAction, InTextSource, InProposedNamespace, bForceUniqueId ? FString() : InProposedKey, OutStableNamespace, OutStableKey);
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

	private:
		TSharedRef<IPropertyHandle> PropertyHandle;
		TSharedPtr<IPropertyUtilities> PropertyUtilities;

		/** The maximum length of the value that can be edited, or <=0 for unlimited */
		int32 MaxLength = 0;
	};
}

void FTextCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils )
{
	TSharedRef<IEditableTextProperty> EditableTextProperty = MakeShareable(new FEditableTextPropertyHandle(InPropertyHandle, PropertyTypeCustomizationUtils.GetPropertyUtilities()));
	const bool bIsMultiLine = EditableTextProperty->IsMultiLineText();

	HeaderRow.FilterString(InPropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(bIsMultiLine ? 250.f : 125.f)
		.MaxDesiredWidth(600.f)
		[
			SNew(STextPropertyEditableTextBox, EditableTextProperty)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		];
}

void FTextCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils )
{

}
