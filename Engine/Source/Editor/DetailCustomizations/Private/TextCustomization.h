// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FTextCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable( new FTextCustomization );
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils ) override;
};
