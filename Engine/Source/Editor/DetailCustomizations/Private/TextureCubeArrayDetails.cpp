// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureCubeArrayDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Engine/TextureCubeArray.h"
#include "Engine/TextureCube.h"
#include "Widgets/Layout/SBox.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "TextureCubeArrayDetails"

TSharedRef<IDetailCustomization> FTextureCubeArrayDetails::MakeInstance()
{
	return MakeShareable(new FTextureCubeArrayDetails);
}

void FTextureCubeArrayDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.Num() != 1)
	{
		return;
	}

	TextureCubeArrayPtr = Cast<UTextureCubeArray>(Objects[0].Get());
	if (!TextureCubeArrayPtr.IsValid())
	{
		return;
	}

	IDetailCategoryBuilder& SourceCubeCategory = DetailBuilder.EditCategory("SourceCube");

	// Error banner for size/layout mismatches
	SourceCubeCategory.AddCustomRow(LOCTEXT("MismatchWarningFilter", "Mismatch Warning"), false)
		.WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.f, 4.f))
			[
				SNew(SWarningOrErrorBox)
				.MessageStyle(EMessageStyle::Error)
				.Message_Lambda([this]() -> FText { return GetErrorText(); })
			]
		]
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility
		{
			return GetErrorVisibility();
		}));

	// Custom array builder with filtered asset pickers per element
	TSharedPtr<IPropertyHandle> SourceTexturesHandle = DetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UTextureCubeArray, SourceTextures));

	if (SourceTexturesHandle.IsValid() && SourceTexturesHandle->IsValidHandle())
	{
		TSharedRef<FDetailArrayBuilder> ArrayBuilder = MakeShareable(new FDetailArrayBuilder(SourceTexturesHandle.ToSharedRef()));

		ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FTextureCubeArrayDetails::OnGenerateArrayElementWidget));

		SourceCubeCategory.AddCustomBuilder(ArrayBuilder, false);
	}
}

void FTextureCubeArrayDetails::OnGenerateArrayElementWidget(
	TSharedRef<IPropertyHandle> ElementHandle, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(ElementHandle);

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	SAssignNew(ValueWidget, SObjectPropertyEntryBox)
		.PropertyHandle(ElementHandle)
		.AllowedClass(UTextureCube::StaticClass())
		.AllowClear(true)
		.DisplayThumbnail(true)
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(this, &FTextureCubeArrayDetails::ShouldFilterCubeAsset, ElementIndex));

	PropertyRow.CustomWidget()
		.NameContent()
		.MinDesiredWidth(Row.NameWidget.MinWidth)
		.MaxDesiredWidth(Row.NameWidget.MaxWidth)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.MinDesiredWidth(Row.ValueWidget.MinWidth)
		.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
		[
			ValueWidget.ToSharedRef()
		];
}

const FTextureSource* FindFirstValidSource(const UTextureCubeArray* Tex)
{
	const FTextureSource* FirstValidSource = nullptr;
	for (const TObjectPtr<UTextureCube>& SourceTexture : Tex->SourceTextures)
	{
		if (SourceTexture && SourceTexture->Source.IsValid())
		{
			FirstValidSource = &SourceTexture->Source;
			break;
		}
	}
	return FirstValidSource;
}

bool FTextureCubeArrayDetails::ShouldFilterCubeAsset(const FAssetData& AssetData, int32 ElementIndex) const
{
	if (!TextureCubeArrayPtr.IsValid())
	{
		return false;
	}

	const TArray<TObjectPtr<UTextureCube>>& SourceTextures = TextureCubeArrayPtr->SourceTextures;

	// Find the first valid source that is NOT the element being edited.  If there are no other valid entries,
	// the user is free to pick anything — there's nothing else for it to conflict with.  This logic is designed
	// to specifically allow edits on the first element to achieve a valid state matching later elements, which
	// wouldn't work if we always chose the first valid element as ReferenceSource.  You could work around this
	// limitation by resetting the element before editing, but that's not very intuitive if you don't know how
	// the filtering code works under the hood.
	const FTextureSource* ReferenceSource = nullptr;
	for (int32 i = 0; i < SourceTextures.Num(); ++i)
	{
		if (i != ElementIndex && SourceTextures[i] && SourceTextures[i]->Source.IsValid())
		{
			ReferenceSource = &SourceTextures[i]->Source;
			break;
		}
	}

	if (!ReferenceSource)
	{
		return false;
	}

	// Parse the "Dimensions" asset registry tag (format: "512x512x6" or "512x512")
	FString DimensionsStr;
	if (!AssetData.GetTagValue(FName("Dimensions"), DimensionsStr))
	{
		return false;		// No tag available, allow it and let PostEditChange handle validation
	}

	// Parse SizeX and SizeY from the dimensions string
	int32 CandidateSizeX = 0;
	int32 CandidateSizeY = 0;

	TArray<FString> Parts;
	DimensionsStr.ParseIntoArray(Parts, TEXT("x"), true);
	if (Parts.Num() >= 2)
	{
		CandidateSizeX = FCString::Atoi(*Parts[0]);
		CandidateSizeY = FCString::Atoi(*Parts[1]);
	}

	// Filter out if sizes don't match
	if (CandidateSizeX != ReferenceSource->GetSizeX() || CandidateSizeY != ReferenceSource->GetSizeY())
	{
		return true;	// Filter out
	}

	return false;	// Allow
}

EVisibility FTextureCubeArrayDetails::GetErrorVisibility() const
{
	if (!TextureCubeArrayPtr.IsValid())
	{
		return EVisibility::Collapsed;
	}

	const FTextureSource* FirstValidSource;
	if (!(FirstValidSource = FindFirstValidSource(TextureCubeArrayPtr.Get())))
	{
		return EVisibility::Collapsed;		// Nothing valid in the array, no mismatch errors possible
	}

	for (const TObjectPtr<UTextureCube>& Tex : TextureCubeArrayPtr->SourceTextures)
	{
		if (!Tex || !Tex->Source.IsValid())
		{
			continue;
		}

		if (Tex->Source.GetSizeX() != FirstValidSource->GetSizeX() ||
			Tex->Source.GetSizeY() != FirstValidSource->GetSizeY() ||
			Tex->Source.GetNumSlices() != FirstValidSource->GetNumSlices())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FText FTextureCubeArrayDetails::GetErrorText() const
{
	if (!TextureCubeArrayPtr.IsValid())
	{
		return FText::GetEmpty();
	}

	const TArray<TObjectPtr<UTextureCube>>& SourceTextures = TextureCubeArrayPtr->SourceTextures;

	// Find first valid texture as reference
	const UTextureCube* FirstValid = nullptr;
	for (const TObjectPtr<UTextureCube>& Tex : SourceTextures)
	{
		if (Tex && Tex->Source.IsValid())
		{
			FirstValid = Tex;
			break;
		}
	}

	if (!FirstValid)
	{
		return FText::GetEmpty();
	}

	const int32 ExpectedSizeX = (int32)FirstValid->Source.GetSizeX();
	const int32 ExpectedSizeY = (int32)FirstValid->Source.GetSizeY();
	const int32 ExpectedSlices = FirstValid->Source.GetNumSlices();
	const TCHAR* ExpectedType = ExpectedSlices == 6 ? TEXT("Cubemap") : TEXT("LongLat");

	// Get first mismatched texture, and indices of additional mismatches.  The dialog could get huge and become difficult
	// to read, if we displayed full information for each mismatch where there are many, and in common cases, it's most
	// likely there will be only one mismatch.
	const UTextureCube* FirstMismatch = nullptr;
	TStringBuilder<256> MismatchedIndices;
	for (int32 SourceTextureIndex = 0; SourceTextureIndex < SourceTextures.Num(); SourceTextureIndex++)
	{
		const TObjectPtr<UTextureCube>& Tex = SourceTextures[SourceTextureIndex];
		if (!Tex || !Tex->Source.IsValid())
		{
			continue;
		}

		if (Tex->Source.GetSizeX() != ExpectedSizeX ||
			Tex->Source.GetSizeY() != ExpectedSizeY ||
			Tex->Source.GetNumSlices() != ExpectedSlices)
		{
			if (!FirstMismatch)
			{
				FirstMismatch = Tex;
			}

			if (MismatchedIndices.Len() == 0)
			{
				MismatchedIndices.Appendf(TEXT("%d"), SourceTextureIndex);
			}
			else
			{
				MismatchedIndices.Appendf(TEXT(", %d"), SourceTextureIndex);
			}
		}
	}

	// FirstMismatch can be null, in cases where GetErrorText is called before GetErrorVisibility has a chance to hide the error control.
	// Check for that and return.
	if (!FirstMismatch)
	{
		return FText::GetEmpty();
	}

	return FText::Format(
		LOCTEXT("SizeMismatchError",
			"Texture size or layout mismatch:\nExpected:  {0}x{1} ({2})  '{3}'\nMismatch:  {4}x{5} ({6})  '{7}'.\nMismatch indices: {8}"),
		FText::AsNumber(ExpectedSizeX),
		FText::AsNumber(ExpectedSizeY),
		FText::FromString(ExpectedType),
		FText::FromString(FirstValid->GetName()),
		FText::AsNumber(FirstMismatch->Source.GetSizeX()),
		FText::AsNumber(FirstMismatch->Source.GetSizeY()),
		FText::FromString(FirstMismatch->Source.GetNumSlices() == 6 ? TEXT("Cubemap") : TEXT("LongLat")),
		FText::FromString(FirstMismatch->GetName()),
		FText::FromString(MismatchedIndices.GetData()));
}

#undef LOCTEXT_NAMESPACE
