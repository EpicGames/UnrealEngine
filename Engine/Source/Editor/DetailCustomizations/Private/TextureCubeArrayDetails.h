// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Layout/Visibility.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class UTextureCubeArray;
struct FAssetData;

class FTextureCubeArrayDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	/** Filter for array element asset pickers - blocks cubemaps that don't match the first entry's size/format. */
	bool ShouldFilterCubeAsset(const FAssetData& AssetData, int32 ElementIndex) const;

	/** Generates custom widget per array element with the filtered asset picker. */
	void OnGenerateArrayElementWidget(TSharedRef<IPropertyHandle> ElementHandle, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/** Dynamic error text describing which textures mismatch. */
	FText GetErrorText() const;

	/** Visible only when there's a size/format mismatch in the array. */
	EVisibility GetErrorVisibility() const;

	TWeakObjectPtr<UTextureCubeArray> TextureCubeArrayPtr;
};
