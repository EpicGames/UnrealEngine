// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureLODSettingsDetails.h"

#include "Containers/BitArray.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureLODSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "RHIDefinitions.h"
#include "Serialization/Archive.h"
#include "SlateOptMacros.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/STextBlock.h"

class FString;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "TextureLODSettingsDetails"


////////////////////////////////////////////////
// FDeviceProfileTextureLODSettingsDetails


FDeviceProfileTextureLODSettingsDetails::FDeviceProfileTextureLODSettingsDetails(IDetailLayoutBuilder* InDetailBuilder)
	: DetailBuilder(InDetailBuilder)
{
	TextureLODSettingsPropertyNameHandle = DetailBuilder->GetProperty("TextureLODGroups", UTextureLODSettings::StaticClass());
	LODGroupsArrayHandle = TextureLODSettingsPropertyNameHandle->AsArray();

	TArray<UObject*> OuterObjects;
	TextureLODSettingsPropertyNameHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() == 1)
	{
		DeviceProfile = CastChecked<UDeviceProfile>(OuterObjects[0]);
	}
}


void FDeviceProfileTextureLODSettingsDetails::CreateTextureGroupEntryRow(int32 GroupId, IDetailCategoryBuilder& DetailCategoryBuilder)
{
	TSharedRef<IPropertyHandle> LODGroupElementHandle = LODGroupsArrayHandle->GetElement(GroupId);

	DetailCategoryBuilder.AddCustomBuilder(MakeShareable(new FTextureLODGroupLayout(DeviceProfile, (TextureGroup)GroupId)));
}


void FDeviceProfileTextureLODSettingsDetails::CreateTextureLODSettingsPropertyView()
{
	DetailBuilder->HideProperty(TextureLODSettingsPropertyNameHandle);

	IDetailCategoryBuilder& TextureLODSettingsDetailCategory = DetailBuilder->EditCategory("Texture LOD Settings");

#define SETUPLODGROUP(GroupId) CreateTextureGroupEntryRow(GroupId, TextureLODSettingsDetailCategory);
	FOREACH_ENUM_TEXTUREGROUP(SETUPLODGROUP)
#undef SETUPLODGROUP
}


////////////////////////////////////////////////
// FTextureLODGroupLayout

FTextureLODGroupLayout::FTextureLODGroupLayout(const UDeviceProfile* InDeviceProfile, TextureGroup InGroupId)
{
	LodGroup = &InDeviceProfile->GetTextureLODSettings()->GetTextureLODGroup(InGroupId);

#define POPULATE_MIPGENSETTINGS(MipGenSettingsId) AddToAvailableMipGenSettings(MipGenSettingsId);
	FOREACH_ENUM_TEXTUREMIPGENSETTINGS(POPULATE_MIPGENSETTINGS)
#undef POPULATE_MIPGENSETTINGS
}


FTextureLODGroupLayout::~FTextureLODGroupLayout()
{
}


void FTextureLODGroupLayout::AddToAvailableMipGenSettings(TextureMipGenSettings MipGenSettingsId)
{
	MipGenSettingsComboList.Add(MakeShareable(new TextureMipGenSettings(MipGenSettingsId)));
}

void FTextureLODGroupLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	UEnum* TextureGroupEnum = StaticEnum<TextureGroup>();
	const FString& LODGroupName = TextureGroupEnum->GetMetaData(TEXT("DisplayName"), LodGroup->Group);

	NodeRow.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(LODGroupName))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FTextureLODGroupLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	// Min and Max LOD properties
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MinLODSize", "Min LOD Size"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MinLODSize", "Min LOD Size"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<uint32>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(1)
				.MaxValue(8192)
				.Value(this, &FTextureLODGroupLayout::GetMinLODSize)
				.OnValueChanged(this, &FTextureLODGroupLayout::OnMinLODSizeChanged)
				.OnValueCommitted(this, &FTextureLODGroupLayout::OnMinLODSizeCommitted)
			];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MaxLODSize", "Max LOD Size"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaxLODSize", "Max LOD Size"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<uint32>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(1)
				.MaxValue(8192)
				.Value(this, &FTextureLODGroupLayout::GetMaxLODSize)
				.OnValueChanged(this, &FTextureLODGroupLayout::OnMaxLODSizeChanged)
				.OnValueCommitted(this, &FTextureLODGroupLayout::OnMaxLODSizeCommitted)
			];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MaxLODSizeVT", "Max LOD Size VT"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaxLODSizeVT", "Max LOD Size VT"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<uint32>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(0)
				.MaxValue(32768)
				.Value(this, &FTextureLODGroupLayout::GetMaxLODSizeVT)
				.OnValueChanged(this, &FTextureLODGroupLayout::OnMaxLODSizeVTChanged)
				.OnValueCommitted(this, &FTextureLODGroupLayout::OnMaxLODSizeVTCommitted)
			];
	}

	// LOD Bias
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("LODBias", "LOD Bias"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("LODBias", "LOD Bias"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<int32>)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.MinValue(-MAX_TEXTURE_MIP_COUNT)
				.MaxValue(MAX_TEXTURE_MIP_COUNT)
				.Value(this, &FTextureLODGroupLayout::GetLODBias)
				.OnValueChanged(this, &FTextureLODGroupLayout::OnLODBiasChanged)
				.OnValueCommitted(this, &FTextureLODGroupLayout::OnLODBiasCommitted)
			];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("LODBiasVT_Row", "LOD Bias VT"))
			.NameContent()
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("LODBiasVT_Name", "LOD Bias VT"))
			]
			.ValueContent()
			[
				SNew(SSpinBox<int32>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.MinValue(-MAX_TEXTURE_MIP_COUNT)
					.MaxValue(MAX_TEXTURE_MIP_COUNT)
					.Value(this, &FTextureLODGroupLayout::GetLODBiasVT)
					.OnValueChanged(this, &FTextureLODGroupLayout::OnLODBiasVTChanged)
					.OnValueCommitted(this, &FTextureLODGroupLayout::OnLODBiasVTCommitted)
			];
	}


	// Filter properties

	TSharedPtr<FName> NamePointPtr = MakeShareable(new FName(NAME_Point));
	FilterComboList.Add(NamePointPtr);
	TSharedPtr<FName> NameLinearPtr = MakeShareable(new FName(NAME_Linear));
	FilterComboList.Add(NameLinearPtr);
	TSharedPtr<FName> NameAnisoPtr = MakeShareable(new FName(NAME_Aniso));
	FilterComboList.Add(NameAnisoPtr);
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MinMagFilter", "MinMag Filter"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MinMagFilter", "MinMag Filter"))
			]
			.ValueContent()
			[
				SNew(SComboBox< TSharedPtr<FName> >)
				.OptionsSource(&FilterComboList)
				.OnGenerateWidget(this, &FTextureLODGroupLayout::MakeMinMagFilterComboWidget)
				.OnSelectionChanged(this, &FTextureLODGroupLayout::OnMinMagFilterChanged)
				.ContentPadding(0.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FTextureLODGroupLayout::GetMinMagFilterComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FTextureLODGroupLayout::GetMinMagFilterComboBoxToolTip)
				]
			];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MipFilter", "Mip Filter"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MipFilter", "Mip Filter"))
			]
		.ValueContent()
			[
				SNew(SComboBox< TSharedPtr<FName> >)
				.OptionsSource(&FilterComboList)
				.OnGenerateWidget(this, &FTextureLODGroupLayout::MakeMipFilterComboWidget)
				.OnSelectionChanged(this, &FTextureLODGroupLayout::OnMipFilterChanged)
				.ContentPadding(0.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FTextureLODGroupLayout::GetMipFilterComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FTextureLODGroupLayout::GetMipFilterComboBoxToolTip)
				]
			];
	}


	// Mip Gen Settings
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("MipGenSettings", "Mip Gen Settings"))
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MipGenSettings", "Mip Gen Settings"))
			]
		.ValueContent()
			[
				SNew(SComboBox< TSharedPtr<TextureMipGenSettings> >)
				.OptionsSource(&MipGenSettingsComboList)
				.OnGenerateWidget(this, &FTextureLODGroupLayout::MakeMipGenSettingsComboWidget)
				.OnSelectionChanged(this, &FTextureLODGroupLayout::OnMipGenSettingsChanged)
				.ContentPadding(0.f)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FTextureLODGroupLayout::GetMipGenSettingsComboBoxContent)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &FTextureLODGroupLayout::GetMipGenSettingsComboBoxToolTip)
				]
			];
	}

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


uint32 FTextureLODGroupLayout::GetMinLODSize() const
{
	return LodGroup->MinLODSize;
}

void FTextureLODGroupLayout::OnMinLODSizeChanged(uint32 NewValue)
{
	LodGroup->MinLODSize = NewValue;
}

void FTextureLODGroupLayout::OnMinLODSizeCommitted(uint32 NewValue, ETextCommit::Type TextCommitType)
{
	//if (FEngineAnalytics::IsAvailable())
	//{
//		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("MinLODSize"), FString::Printf(TEXT("%.1f"), NewValue));
	//}
	OnMinLODSizeChanged(NewValue);
}


uint32 FTextureLODGroupLayout::GetMaxLODSize() const
{
	return LodGroup->MaxLODSize;
}

void FTextureLODGroupLayout::OnMaxLODSizeChanged(uint32 NewValue)
{
	LodGroup->MaxLODSize = NewValue;
}

void FTextureLODGroupLayout::OnMaxLODSizeCommitted(uint32 NewValue, ETextCommit::Type TextCommitType)
{
	//if (FEngineAnalytics::IsAvailable())
	//{
		//		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("MaxLODSize"), FString::Printf(TEXT("%.1f"), NewValue));
	//}
	OnMaxLODSizeChanged(NewValue);
}


uint32 FTextureLODGroupLayout::GetMaxLODSizeVT() const
{
	return LodGroup->MaxLODSize_VT;
}

void FTextureLODGroupLayout::OnMaxLODSizeVTChanged(uint32 NewValue)
{
	LodGroup->MaxLODSize_VT = NewValue;
}

void FTextureLODGroupLayout::OnMaxLODSizeVTCommitted(uint32 NewValue, ETextCommit::Type TextCommitType)
{
	OnMaxLODSizeVTChanged(NewValue);
}


int32 FTextureLODGroupLayout::GetLODBias() const
{
	return LodGroup->LODBias;
}

void FTextureLODGroupLayout::OnLODBiasChanged(int32 NewValue)
{
	LodGroup->LODBias = NewValue;
}

void FTextureLODGroupLayout::OnLODBiasCommitted(int32 NewValue, ETextCommit::Type TextCommitType)
{
	//if (FEngineAnalytics::IsAvailable())
	//{
		//		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.StaticMesh.ReductionSettings"), TEXT("LODBias"), FString::Printf(TEXT("%.1f"), NewValue));
	//}
	OnLODBiasChanged(NewValue);
}

int32 FTextureLODGroupLayout::GetLODBiasVT() const
{
	return LodGroup->LODBias_VT;
}

void FTextureLODGroupLayout::OnLODBiasVTChanged(int32 NewValue)
{
	LodGroup->LODBias_VT = NewValue;
}

void FTextureLODGroupLayout::OnLODBiasVTCommitted(int32 NewValue, ETextCommit::Type TextCommitType)
{
	OnLODBiasVTChanged(NewValue);
}


TSharedRef<SWidget> FTextureLODGroupLayout::MakeMinMagFilterComboWidget(TSharedPtr<FName> InItem)
{
	return SNew(STextBlock).Text(FText::FromName(*InItem)).Font(IDetailLayoutBuilder::GetDetailFont());
}

void FTextureLODGroupLayout::OnMinMagFilterChanged(TSharedPtr<FName> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FName NewValue = *NewSelection.Get();
		LodGroup->MinMagFilter = NewValue;
	}
}

FText FTextureLODGroupLayout::GetMinMagFilterComboBoxToolTip() const
{
	return FText::GetEmpty();
}

FText FTextureLODGroupLayout::GetMinMagFilterComboBoxContent() const
{
	return FText::FromName(LodGroup->MinMagFilter);
}


TSharedRef<SWidget> FTextureLODGroupLayout::MakeMipFilterComboWidget(TSharedPtr<FName> InItem)
{
	return SNew(STextBlock).Text(FText::FromName(*InItem)).Font(IDetailLayoutBuilder::GetDetailFont());
}

void FTextureLODGroupLayout::OnMipFilterChanged(TSharedPtr<FName> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FName NewValue = *NewSelection.Get();
		LodGroup->MipFilter = NewValue;
	}
}

FText FTextureLODGroupLayout::GetMipFilterComboBoxToolTip() const
{
	return FText::GetEmpty();
}

FText FTextureLODGroupLayout::GetMipFilterComboBoxContent() const
{
	return FText::FromName(LodGroup->MipFilter);
}


TSharedRef<SWidget> FTextureLODGroupLayout::MakeMipGenSettingsComboWidget(TSharedPtr<TextureMipGenSettings> InItem)
{
	UEnum* TextureGroupEnum = StaticEnum<TextureMipGenSettings>();
	const FString& MipGenSettingsName = TextureGroupEnum->GetMetaData(TEXT("DisplayName"), TextureGroupEnum->GetIndexByValue(*InItem.Get()));

	return SNew(STextBlock).Text(FText::FromString(MipGenSettingsName)).Font(IDetailLayoutBuilder::GetDetailFont());
}

void FTextureLODGroupLayout::OnMipGenSettingsChanged(TSharedPtr<TextureMipGenSettings> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		TextureMipGenSettings NewValue = *NewSelection.Get();
		LodGroup->MipGenSettings = NewValue;
	}
}

FText FTextureLODGroupLayout::GetMipGenSettingsComboBoxToolTip() const
{
	return FText::GetEmpty();
}

FText FTextureLODGroupLayout::GetMipGenSettingsComboBoxContent() const
{
	UEnum* TextureGroupEnum = StaticEnum<TextureMipGenSettings>();
	const FString& MipGenSettingsName = TextureGroupEnum->GetMetaData(TEXT("DisplayName"), TextureGroupEnum->GetIndexByValue(LodGroup->MipGenSettings));
	
	return FText::FromString(MipGenSettingsName);
}

#undef LOCTEXT_NAMESPACE
