// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Engine/TextureDefines.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"

class FDetailWidgetRow;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IPropertyHandleArray;
class SWidget;
struct FTextureLODGroup;

/**
 * Texture Group layout for property editor views.
 */
class FTextureLODGroupLayout : public IDetailCustomNodeBuilder, public TSharedFromThis < FTextureLODGroupLayout >
{
public:
	FTextureLODGroupLayout(const class UDeviceProfile* InDeviceProfile, TextureGroup GroupId);
	virtual ~FTextureLODGroupLayout();
private:
	/** Begin IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override{}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName TextureLODGroup("TextureLODGroup"); return TextureLODGroup; }
	virtual bool InitiallyCollapsed() const override { return true; }
	/** End IDetailCustomNodeBuilder Interface*/

private:

	// Controls for the Min LOD Size property editing
	uint32 GetMinLODSize() const;
	void OnMinLODSizeChanged(uint32 NewValue);
	void OnMinLODSizeCommitted(uint32 NewValue, ETextCommit::Type TextCommitType);

	// Controls for the Max LOD Size property editing
	uint32 GetMaxLODSize() const;
	void OnMaxLODSizeChanged(uint32 NewValue);
	void OnMaxLODSizeCommitted(uint32 NewValue, ETextCommit::Type TextCommitType);

	// Controls for the Max LOD Size VT property editing
	uint32 GetMaxLODSizeVT() const;
	void OnMaxLODSizeVTChanged(uint32 NewValue);
	void OnMaxLODSizeVTCommitted(uint32 NewValue, ETextCommit::Type TextCommitType);

	// Controls for the LOD Bias property editing
	int32 GetLODBias() const;
	void OnLODBiasChanged(int32 NewValue);
	void OnLODBiasCommitted(int32 NewValue, ETextCommit::Type TextCommitType);

	// Controls for the LOD Bias VT property editing
	int32 GetLODBiasVT() const;
	void OnLODBiasVTChanged(int32 NewValue);
	void OnLODBiasVTCommitted(int32 NewValue, ETextCommit::Type TextCommitType);

	// Controls for the MinMag Filter property editing
	TSharedRef<SWidget> MakeMinMagFilterComboWidget(TSharedPtr<FName> InItem);
	void OnMinMagFilterChanged(TSharedPtr<FName> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetMinMagFilterComboBoxContent() const;
	FText GetMinMagFilterComboBoxToolTip() const;

	// Controls for the Mip Filter property editing
	TSharedRef<SWidget> MakeMipFilterComboWidget(TSharedPtr<FName> InItem);
	void OnMipFilterChanged(TSharedPtr<FName> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetMipFilterComboBoxContent() const;
	FText GetMipFilterComboBoxToolTip() const;

	// Controls for the MipGenSettings property editing
	TSharedRef<SWidget> MakeMipGenSettingsComboWidget(TSharedPtr<TextureMipGenSettings> InItem);
	void OnMipGenSettingsChanged(TSharedPtr<TextureMipGenSettings> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetMipGenSettingsComboBoxContent() const;
	FText GetMipGenSettingsComboBoxToolTip() const;

private:
	// The LOD group we are creating an editor for.
	FTextureLODGroup* LodGroup;

	// The available filter names for the MinMag and Mip Filter selection
	TArray<TSharedPtr<FName>> FilterComboList;

private:

	// Populate the Mip Gen Settings Combo Options
	void AddToAvailableMipGenSettings(TextureMipGenSettings MipGenSettingsId);

	// The available MipGenSettingsavailable for selection
	TArray<TSharedPtr<TextureMipGenSettings>> MipGenSettingsComboList;
};

/**
* Details panel for Texture LOD Settings
*/
class FDeviceProfileTextureLODSettingsDetails
	: public TSharedFromThis<FDeviceProfileTextureLODSettingsDetails>
{

public:

	/**
	 * Constructor for the parent property details view
	 *
	 * @param InDetailsBuilder Where we are adding our property view to
	 */
	FDeviceProfileTextureLODSettingsDetails(IDetailLayoutBuilder* InDetailBuilder);


	/**
	 * Create the parent property view for the device profile
	 */
	void CreateTextureLODSettingsPropertyView();

	/**
	 * Create an editor for the provided LOD Group
	 */
	void CreateTextureGroupEntryRow(int32 GroupId, IDetailCategoryBuilder& DetailGroup);
	
private:

	/**
	 * Delegate used when the device profiles parent is updated from any source.
	 */
	void OnTextureLODSettingsPropertyChanged();

private:

	/** A handle to the detail view builder */
	IDetailLayoutBuilder* DetailBuilder;

	/** Access to the Parent Property */
	TSharedPtr<IPropertyHandle> TextureLODSettingsPropertyNameHandle;

	/** Access to the LOD Groups array */
	TSharedPtr<IPropertyHandleArray> LODGroupsArrayHandle;

	/** A reference to the object we are showing these properties for */
	class UDeviceProfile* DeviceProfile;
};
