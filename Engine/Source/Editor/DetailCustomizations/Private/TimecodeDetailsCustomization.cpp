// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimecodeDetailsCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/Timecode.h"
#include "PropertyHandle.h"
#include "Trace/Detail/Channel.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "Timecode"

void FTimecodeDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TimecodeProperty = PropertyHandle;

	ChildBuilder.AddProperty(PropertyHandle)
		.CustomWidget()
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FTimecodeDetailsCustomization::OnGetTimecodeText)
			.OnTextCommitted(this, &FTimecodeDetailsCustomization::OnTimecodeTextCommitted)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsReadOnly_Lambda([this]()
			{
				return TimecodeProperty.IsValid() ? !TimecodeProperty->IsEditable() : false;
			})
		];
}

FText FTimecodeDetailsCustomization::OnGetTimecodeText() const
{
	TArray<void*> RawData;
	TimecodeProperty->AccessRawData(RawData);

	if (RawData.Num())
	{
		FString CurrentValue = ((FTimecode*)RawData[0])->ToString();
		return FText::FromString(CurrentValue);
	}

	return FText::GetEmpty();
}

void FTimecodeDetailsCustomization::OnTimecodeTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	TArray<void*> RawData;
	TimecodeProperty->AccessRawData(RawData);

	if (RawData.Num())
	{
		TArray<FString> Splits;
		InText.ToString().ParseIntoArray(Splits, TEXT(":"));

		const int NumSplits = Splits.Num();
		if (NumSplits > 0 && NumSplits <= 4)
		{
			GEditor->BeginTransaction(FText::Format(LOCTEXT("SetTimecodeProperty", "Edit {0}"), TimecodeProperty->GetPropertyDisplayName()));
			
			TimecodeProperty->NotifyPreChange();
			((FTimecode*)RawData[0])->Hours = FCString::Atoi(*Splits[0]);
			((FTimecode*)RawData[0])->Minutes = NumSplits > 1 ? FCString::Atoi(*Splits[1]) : 0;

			if (NumSplits > 2)
			{
				TArray<FString> SubSplits;
				Splits[2].ParseIntoArray(SubSplits, TEXT(";"));

				const int NumSubSplits = SubSplits.Num();

				((FTimecode*) RawData[0])->bDropFrameFormat = NumSubSplits != 1 ? true : false;

				if (((FTimecode*) RawData[0])->bDropFrameFormat)
				{
					((FTimecode*) RawData[0])->Seconds = FCString::Atoi(*SubSplits[0]);
					((FTimecode*) RawData[0])->Frames = NumSubSplits > 1 ? FCString::Atoi(*SubSplits[1]) : 0;
				}
				else
				{
					((FTimecode*) RawData[0])->Seconds = FCString::Atoi(*Splits[2]);
					((FTimecode*) RawData[0])->Frames = NumSplits > 3 ? FCString::Atoi(*Splits[3]) : 0;
				}
			}
			
			TimecodeProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			TimecodeProperty->NotifyFinishedChangingProperties();

			GEditor->EndTransaction();
		}
		else
		{
			UE_LOGF(LogTemp, Error, "Unexpected timecode format. Expected 4 values, got %d", Splits.Num());
		}
	}
}

#undef LOCTEXT_NAMESPACE
