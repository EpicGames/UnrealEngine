// Copyright Epic Games, Inc. All Rights Reserved.

#include "TimespanStructCustomization.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "DetailWidgetRow.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Attribute.h"
#include "Misc/Timespan.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "TimespanStructCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FTimespanStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	/* do nothing */
}


void FTimespanStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyHandle = StructPropertyHandle;
	InputValid = true;

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		.MinDesiredWidth(125.0f)
		[
			SNew(SEditableTextBox)
				.ClearKeyboardFocusOnCommit(false)
				.IsEnabled(!PropertyHandle->IsEditConst())
				.ForegroundColor(this, &FTimespanStructCustomization::HandleTextBoxForegroundColor)
				.OnTextChanged(this, &FTimespanStructCustomization::HandleTextBoxTextChanged)
				.OnTextCommitted(this, &FTimespanStructCustomization::HandleTextBoxTextCommited)
				.SelectAllTextOnCommit(true)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text(this, &FTimespanStructCustomization::HandleTextBoxText)
		];
}


/* FTimespanStructCustomization callbacks
 *****************************************************************************/

FSlateColor FTimespanStructCustomization::HandleTextBoxForegroundColor() const
{
	if (InputValid)
	{
		static const FName DefaultForeground("Colors.Foreground");
		return FAppStyle::Get().GetSlateColor(DefaultForeground);
	}

	static const FName Red("Colors.AccentRed");
	return FAppStyle::Get().GetSlateColor(Red);
}


FText FTimespanStructCustomization::HandleTextBoxText() const
{
	TArray<void*> RawData;
	PropertyHandle->AccessRawData(RawData);
	
	if (RawData.Num() != 1)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	
	if(RawData[0] == nullptr)
	{
		return FText::GetEmpty();
	}

	FString ValueString;

	if (!((FTimespan*)RawData[0])->ExportTextItem(ValueString, FTimespan::Zero(), NULL, 0, NULL))
	{
		return FText::GetEmpty();
	}
	
	return FText::FromString(ValueString);
}


void FTimespanStructCustomization::HandleTextBoxTextChanged(const FText& NewText)
{
	FTimespan Timespan;
	InputValid = FTimespan::Parse(NewText.ToString(), Timespan);
}


void FTimespanStructCustomization::HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo)
{
	FTimespan ParsedTimespan;
	InputValid = FTimespan::Parse(NewText.ToString(), ParsedTimespan);

	if (InputValid && PropertyHandle.IsValid())
	{
		TArray<void*> RawData;

		PropertyHandle->AccessRawData(RawData);
		PropertyHandle->NotifyPreChange();

		for (auto RawDataInstance : RawData)
		{
			*(FTimespan*)RawDataInstance = ParsedTimespan;
		}

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();
	}
}


#undef LOCTEXT_NAMESPACE
