// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class SEditableTextBox;

/**
 * Implements a details view customization for the FTimespan structure.
 */
class FTimespanStructCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FTimespanStructCustomization());
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	/** Handles getting the text color of the editable text box. */
	FSlateColor HandleTextBoxForegroundColor() const;

	/** Handles getting the text to be displayed in the editable text box. */
	FText HandleTextBoxText() const;

	/** Handles changing the value in the editable text box. */
	void HandleTextBoxTextChanged(const FText& NewText);

	/** Handles committing the text in the editable text box. */
	void HandleTextBoxTextCommited(const FText& NewText, ETextCommit::Type CommitInfo);

private:

	/** Holds a flag indicating whether the current input is a valid GUID. */
	bool InputValid;

	/** Holds a handle to the property being edited. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Holds the text box for editing the Guid. */
	TSharedPtr<SEditableTextBox> TextBox;
};
