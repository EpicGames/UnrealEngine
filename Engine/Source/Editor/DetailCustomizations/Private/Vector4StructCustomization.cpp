// Copyright Epic Games, Inc. All Rights Reserved.
#include "Vector4StructCustomization.h"

#include "ColorGradingVectorCustomization.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "PropertyHandle.h"
#include "UObject/UnrealType.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;

TSharedRef<IPropertyTypeCustomization> FVector4StructCustomization::MakeInstance()
{
	return MakeShareable(new FVector4StructCustomization);
}

FVector4StructCustomization::FVector4StructCustomization()
	: FMathStructCustomization()
	, ColorGradingVectorCustomization(nullptr)
{
}

FVector4StructCustomization::~FVector4StructCustomization()
{
	//Release all resources
	ColorGradingVectorCustomization = nullptr;
}

void FVector4StructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FProperty* Property = StructPropertyHandle->GetProperty();
	if (Property)
	{
		const FString& ColorGradingModeString = Property->GetMetaData(TEXT("ColorGradingMode"));
		if (!ColorGradingModeString.IsEmpty())
		{
			//Create our color grading customization shared pointer
			TSharedPtr<FColorGradingVectorCustomization> ColorGradingCustomization = GetOrCreateColorGradingVectorCustomization(StructPropertyHandle);

			//Customize the childrens
			ColorGradingVectorCustomization->CustomizeChildren(StructBuilder, StructCustomizationUtils);
			
			// We handle the customize Children so just return here
			return;
		}
	}

	//Use the base class customize children
	FMathStructCustomization::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
}

TSharedPtr<FColorGradingVectorCustomization> FVector4StructCustomization::GetOrCreateColorGradingVectorCustomization(TSharedRef<IPropertyHandle>& StructPropertyHandle)
{
	//Create our color grading customization shared pointer
	if (!ColorGradingVectorCustomization.IsValid())
	{
		TArray<TWeakPtr<IPropertyHandle>> WeakChildArray;
		WeakChildArray.Append(SortedChildHandles);

		ColorGradingVectorCustomization = MakeShareable(new FColorGradingVectorCustomization(StructPropertyHandle, WeakChildArray));
	}

	return ColorGradingVectorCustomization;
}

void FVector4StructCustomization::MakeHeaderRow(TSharedRef<IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
{
	FProperty* Property = StructPropertyHandle->GetProperty();
	if (Property)
	{
		const FString& ColorGradingModeString = Property->GetMetaData(TEXT("ColorGradingMode"));
		if (!ColorGradingModeString.IsEmpty())
		{
			//Create our color grading customization shared pointer
			TSharedPtr<FColorGradingVectorCustomization> ColorGradingCustomization = GetOrCreateColorGradingVectorCustomization(StructPropertyHandle);

			ColorGradingVectorCustomization->MakeHeaderRow(Row, StaticCastSharedRef<FVector4StructCustomization>(AsShared()));

			// We handle the customize Children so just return here
			return;
		}
	}

	//Use the base class header row
	FMathStructCustomization::MakeHeaderRow(StructPropertyHandle, Row);
}
