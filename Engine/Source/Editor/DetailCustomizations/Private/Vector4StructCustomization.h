// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MathStructCustomizations.h"
#include "Templates/SharedPointer.h"

class FColorGradingVectorCustomization;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;

/**
 * Customizes FVector4 structs.
 * We override Vector4 because the color grading controls are made with vector4
 */
class FVector4StructCustomization
	: public FMathStructCustomization
{
public:
	FVector4StructCustomization();
	~FVector4StructCustomization();
	/** @return A new instance of this class */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//////////////////////////////////////////////////////////////////////////
	/** FMathStructCustomization interface */

	virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;	

private:
	// We specialize the detail display of color grading vector property. The color grading mode is specified inside the metadata of the FProperty
	TSharedPtr<FColorGradingVectorCustomization> GetOrCreateColorGradingVectorCustomization(TSharedRef<IPropertyHandle>& StructPropertyHandle);

	TSharedPtr<FColorGradingVectorCustomization> ColorGradingVectorCustomization;
};

