// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/VectorStructCustomization.h"

#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AxisDisplayInfo.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

class IPropertyTypeCustomization;

namespace UE::Editor::DetailCustomizations::Private
{
	static TAutoConsoleVariable<bool> ShowVector3Children(
		TEXT("Editor.DetailCustomizations.ShowVector3Children"),
		true,
		TEXT("When true, the detail customizations for Vector3 variants expand to show children")
	);
}

TSharedRef<IPropertyTypeCustomization> FVectorStructCustomization::MakeInstance() 
{
	return MakeShareable(new FVectorStructCustomization);
}

void FVectorStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (UE::Editor::DetailCustomizations::Private::ShowVector3Children.GetValueOnGameThread())
	{
		FMathStructCustomization::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
	}
}

void FVectorStructCustomization::GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren)
{
	static const FName X("X");
	static const FName Y("Y");
	static const FName Z("Z");

	FStructProperty* StructProperty = CastField<FStructProperty>(StructPropertyHandle->GetProperty());
	if (!ensureMsgf(StructProperty, TEXT("Vector customization only supports UScriptStruct.")))
	{
		return;
	}

	constexpr int32 NumberOfElements = 3;
	TSharedPtr<IPropertyHandle> VectorChildren[NumberOfElements];

	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		if (PropertyName == X)
		{
			ChildHandle->SetPropertyDisplayName(AxisDisplayInfo::GetAxisDisplayName(EAxisList::X));
			ChildHandle->SetToolTipText(AxisDisplayInfo::GetAxisDisplayName(EAxisList::X));

			VectorChildren[0] = ChildHandle;
		}
		else if (PropertyName == Y)
		{
			ChildHandle->SetPropertyDisplayName(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Y));
			ChildHandle->SetToolTipText(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Y));

			VectorChildren[1] = ChildHandle;
		}
		else if(PropertyName == Z)
		{
			ChildHandle->SetPropertyDisplayName(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Z));
			ChildHandle->SetToolTipText(AxisDisplayInfo::GetAxisDisplayName(EAxisList::Z));

			VectorChildren[2] = ChildHandle;
		}
		else
		{
			ensureMsgf(false, TEXT("The property doesn't exist. Vector customization supports X,Y,Z properties. %s"), *StructProperty->Struct->GetFullName());
		}
	}

	for (int32 Index = 0; Index < NumberOfElements; ++Index)
	{
		if (ensureMsgf(VectorChildren[Index], TEXT("Missing a property. Vector customization supports X,Y,Z properties. %s"), *StructProperty->Struct->GetFullName()))
		{
			OutChildren.Add(VectorChildren[Index].ToSharedRef());
		}
	}
}
