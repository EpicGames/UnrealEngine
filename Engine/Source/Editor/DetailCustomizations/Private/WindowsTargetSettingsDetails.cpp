// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsTargetSettingsDetails.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorDirectories.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "SExternalImageReference.h"
#include "UnrealEngine.h"
#include "RHIShaderFormatDefinitions.inl"
#include "ShaderFormatsPropertyDetails.h"
#include "ShaderPlatformConfigEditor.h"

#if WITH_ENGINE
#include "AudioDevice.h"
#include "ContentStreaming.h"
#endif 

#define LOCTEXT_NAMESPACE "WindowsTargetSettingsDetails"

namespace WindowsTargetSettingsDetailsConstants
{
	/** The filename for the game splash screen */
	const FString GameSplashFileName(TEXT("Splash/Splash.bmp"));

	/** The filename for the editor splash screen */
	const FString EditorSplashFileName(TEXT("Splash/EdSplash.bmp"));

	/** ToolTip used when an option is not available to binary users. */
	const FText DisabledTip = LOCTEXT("GitHubSourceRequiredToolTip", "This requires GitHub source.");
}

static FText GetFriendlyNameFromWindowsShaderPlatform(FName InShaderPlatformName)
{
	const EShaderPlatform ShaderPlatform = ShaderFormatNameToShaderPlatform(InShaderPlatformName);

	FText FriendlyRHIName;
	switch (ShaderPlatform)
	{
	case SP_PCD3D_SM6:
		FriendlyRHIName = LOCTEXT("DirectX12", "DirectX 12 (SM6)");
		break;
	case SP_PCD3D_SM5:
		FriendlyRHIName = LOCTEXT("DirectX11", "DirectX 11 & 12 (SM5)");
		break;
	case SP_PCD3D_ES3_1:
		FriendlyRHIName = LOCTEXT("DirectXMobile", "DirectX Mobile Emulation (Mobile)");
		break;
	case SP_VULKAN_SM5:
		FriendlyRHIName = LOCTEXT("VulkanSM5", "Vulkan (SM5)");
		break;
	case SP_VULKAN_SM6:
		FriendlyRHIName = LOCTEXT("VulkanSM6", "Vulkan (SM6)");
		break;

	case SP_OPENGL_PCES3_1:
	case SP_VULKAN_PCES3_1:
		// Explicitly remove these formats as they are obsolete/not quite supported; users can still target them by adding them as +TargetedRHIs in the TargetPlatform ini.
		FriendlyRHIName = FText::GetEmpty();
		break;

	default:
		UE_LOGF(LogEngine, Warning, "Unknown Windows target RHI %ls", *InShaderPlatformName.ToString());
		FriendlyRHIName = LOCTEXT("UnknownRHI", "UnknownRHI");
		break;
	}

	return FriendlyRHIName;
}

static FText GetFriendlyNameForWindowsShaderPlatformCheckbox(FName InShaderPlatformName)
{
	const EShaderPlatform ShaderPlatform = ShaderFormatNameToShaderPlatform(InShaderPlatformName);

	FText FriendlyName;

	switch (ShaderPlatform)
	{
	case SP_PCD3D_SM6:
	case SP_VULKAN_SM6:
		FriendlyName = LOCTEXT("SM6", "SM6");
		break;
	case SP_PCD3D_SM5:
	case SP_VULKAN_SM5:
		FriendlyName = LOCTEXT("SM5", "SM5");
		break;
	case SP_PCD3D_ES3_1:
		FriendlyName = LOCTEXT("Mobile", "Mobile");
		break;
	default:
		break;
	}

	return FriendlyName;
}

static bool FilterShaderPlatform_D3D12(FName InShaderPlatform)
{
	return InShaderPlatform == NAME_PCD3D_SM6 || InShaderPlatform == NAME_PCD3D_SM5 || InShaderPlatform == NAME_PCD3D_ES3_1;
}

static bool FilterShaderPlatform_D3D11(FName InShaderPlatform)
{
	return InShaderPlatform == NAME_PCD3D_SM5 || InShaderPlatform == NAME_PCD3D_ES3_1;
}

static bool FilterShaderPlatform_Vulkan(FName InShaderPlatform)
{
	return InShaderPlatform == NAME_VULKAN_SM5 || InShaderPlatform == NAME_VULKAN_SM6;
}

TSharedRef<IDetailCustomization> FWindowsTargetSettingsDetails::MakeInstance()
{
	return MakeShareable(new FWindowsTargetSettingsDetails);
}

namespace EWindowsImageScope
{
	enum Type
	{
		Engine,
		GameOverride
	};
}

/* Helper function used to generate filenames for splash screens */
static FString GetWindowsSplashFilename(EWindowsImageScope::Type Scope, bool bIsEditorSplash)
{
	FString Filename;

	if (Scope == EWindowsImageScope::Engine)
	{
		Filename = FPaths::EngineContentDir();
	}
	else
	{
		Filename = FPaths::ProjectContentDir();
	}

	if(bIsEditorSplash)
	{
		Filename /= WindowsTargetSettingsDetailsConstants::EditorSplashFileName;
	}
	else
	{
		Filename /= WindowsTargetSettingsDetailsConstants::GameSplashFileName;
	}

	Filename = FPaths::ConvertRelativePathToFull(Filename);

	return Filename;
}

/* Helper function used to generate filenames for icons */
static FString GetWindowsIconFilename(EWindowsImageScope::Type Scope)
{
	if (Scope == EWindowsImageScope::Engine)
	{
		FString Filename = FPaths::EngineDir() / FString(TEXT("Build/Windows/Resources/Default.ico"));
		return FPaths::ConvertRelativePathToFull(Filename);
	}
	else
	{
		FString Filename = FPaths::ProjectDir() / TEXT("Build/Windows/Application.ico");
		if(!FPaths::FileExists(Filename))
		{
			FString LegacyFilename = FPaths::GameSourceDir() / FString(FApp::GetProjectName()) / FString(TEXT("Resources/Windows")) / FString(FApp::GetProjectName()) + TEXT(".ico");
			if(FPaths::FileExists(LegacyFilename))
			{
				Filename = LegacyFilename;
			}
		}
		return FPaths::ConvertRelativePathToFull(Filename);
	}
}

void FWindowsTargetSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	// Setup the supported/targeted RHI property view
	ITargetPlatformSettings* TargetPlatformSettings = FModuleManager::GetModuleChecked<ITargetPlatformSettingsModule>("WindowsTargetPlatformSettings").GetTargetPlatformSettings()[0];

	D3D12TargetShaderFormatsDetails = MakeShareable(new FShaderFormatsPropertyDetails(&DetailBuilder, TEXT("D3D12TargetedShaderFormats"), TEXT("D3D12 Targeted Shader Formats")));
	D3D12TargetShaderFormatsDetails->CreateTargetShaderFormatsPropertyView(TargetPlatformSettings, &GetFriendlyNameForWindowsShaderPlatformCheckbox, &FilterShaderPlatform_D3D12, ECategoryPriority::Important);

	D3D11TargetShaderFormatsDetails = MakeShareable(new FShaderFormatsPropertyDetails(&DetailBuilder, TEXT("D3D11TargetedShaderFormats"), TEXT("D3D11 Targeted Shader Formats")));
	D3D11TargetShaderFormatsDetails->CreateTargetShaderFormatsPropertyView(TargetPlatformSettings, &GetFriendlyNameForWindowsShaderPlatformCheckbox, &FilterShaderPlatform_D3D11, ECategoryPriority::Important);

	VulkanTargetShaderFormatsDetails = MakeShareable(new FShaderFormatsPropertyDetails(&DetailBuilder, TEXT("VulkanTargetedShaderFormats"), TEXT("Vulkan Targeted Shader Formats")));
	VulkanTargetShaderFormatsDetails->CreateTargetShaderFormatsPropertyView(TargetPlatformSettings, &GetFriendlyNameForWindowsShaderPlatformCheckbox, &FilterShaderPlatform_Vulkan, ECategoryPriority::Important);

	UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigs(DetailBuilder, TargetPlatformSettings);

	// Next add the splash image customization
	const FText EditorSplashDesc(LOCTEXT("EditorSplashLabel", "Editor Splash"));
	IDetailCategoryBuilder& SplashCategoryBuilder = DetailBuilder.EditCategory(TEXT("Splash"));
	FDetailWidgetRow& EditorSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(EditorSplashDesc);

	const FString EditorSplash_TargetImagePath = GetWindowsSplashFilename(EWindowsImageScope::GameOverride, true);
	const FString EditorSplash_DefaultImagePath = GetWindowsSplashFilename(EWindowsImageScope::Engine, true);

	TArray<FString> ImageExtensions;
	ImageExtensions.Add(TEXT("png"));
	ImageExtensions.Add(TEXT("jpg"));
	ImageExtensions.Add(TEXT("bmp"));

	EditorSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(EditorSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, EditorSplash_DefaultImagePath, EditorSplash_TargetImagePath)
			.FileDescription(EditorSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FWindowsTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FWindowsTargetSettingsDetails::HandlePostExternalIconCopy))
			.DeleteTargetWhenDefaultChosen(true)
			.FileExtensions(ImageExtensions)
			.DeletePreviousTargetWhenExtensionChanges(true)
		]
	];

	const FText GameSplashDesc(LOCTEXT("GameSplashLabel", "Game Splash"));
	FDetailWidgetRow& GameSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(GameSplashDesc);
	const FString GameSplash_TargetImagePath = GetWindowsSplashFilename(EWindowsImageScope::GameOverride, false);
	const FString GameSplash_DefaultImagePath = GetWindowsSplashFilename(EWindowsImageScope::Engine, false);

	GameSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(GameSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GameSplash_DefaultImagePath, GameSplash_TargetImagePath)
			.FileDescription(GameSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FWindowsTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FWindowsTargetSettingsDetails::HandlePostExternalIconCopy))
			.DeleteTargetWhenDefaultChosen(true)
			.FileExtensions(ImageExtensions)
			.DeletePreviousTargetWhenExtensionChanges(true)
		]
	];

	IDetailCategoryBuilder& IconsCategoryBuilder = DetailBuilder.EditCategory(TEXT("Icon"));	
	FDetailWidgetRow& GameIconWidgetRow = IconsCategoryBuilder.AddCustomRow(LOCTEXT("GameIconLabel", "Game Icon"));
	GameIconWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("GameIconLabel", "Game Icon"))
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GetWindowsIconFilename(EWindowsImageScope::Engine), GetWindowsIconFilename(EWindowsImageScope::GameOverride))
			.FileDescription(GameSplashDesc)
			.OnPreExternalImageCopy(FOnPreExternalImageCopy::CreateSP(this, &FWindowsTargetSettingsDetails::HandlePreExternalIconCopy))
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FWindowsTargetSettingsDetails::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FWindowsTargetSettingsDetails::HandlePostExternalIconCopy))
		]
	];

	AudioPluginWidgetManager.BuildAudioCategory(DetailBuilder, TEXT("Windows"));
}

bool FWindowsTargetSettingsDetails::HandlePreExternalIconCopy(const FString& InChosenImage)
{
	return true;
}


FString FWindowsTargetSettingsDetails::GetPickerPath()
{
	return FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
}


bool FWindowsTargetSettingsDetails::HandlePostExternalIconCopy(const FString& InChosenImage)
{
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InChosenImage));
	return true;
}

void FWindowsTargetSettingsDetails::HandleAudioStreamCachingToggled(ECheckBoxState EnableStreamCaching, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(EnableStreamCaching == ECheckBoxState::Checked);

#if WITH_ENGINE
	IStreamingManager::Get().OnAudioStreamingParamsChanged();
#endif
}

ECheckBoxState FWindowsTargetSettingsDetails::GetAudioStreamCachingToggled(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	bool bEnabled = false;
	PropertyHandle->GetValue(bEnabled);

	if (bEnabled)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}

#undef LOCTEXT_NAMESPACE
