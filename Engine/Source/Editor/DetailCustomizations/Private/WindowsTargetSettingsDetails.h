// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "TargetPlatformAudioCustomization.h"
#include "Templates/SharedPointer.h"

class FShaderFormatsPropertyDetails;
class IDetailLayoutBuilder;
class IPropertyHandle;

enum class ECheckBoxState : uint8;

/**
 * Manages the Transform section of a details view                    
 */
class FWindowsTargetSettingsDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	/** Delegate handler for before an icon is copied */
	bool HandlePreExternalIconCopy(const FString& InChosenImage);

	/** Delegate handler to get the path to start picking from */
	FString GetPickerPath();

	/** Delegate handler to set the path to start picking from */
	bool HandlePostExternalIconCopy(const FString& InChosenImage);

	/** Handles when Stream Caching is toggled. */
	void HandleAudioStreamCachingToggled(ECheckBoxState bEnableStreamCaching, TSharedPtr<IPropertyHandle> PropertyHandle);

	/** This gets the current value of the audio stream caching bool property. */
	ECheckBoxState GetAudioStreamCachingToggled(TSharedPtr<IPropertyHandle> PropertyHandle) const;

private:
	/** Reference to the target shader formats property view */
	TSharedPtr<FShaderFormatsPropertyDetails> D3D12TargetShaderFormatsDetails;
	TSharedPtr<FShaderFormatsPropertyDetails> D3D11TargetShaderFormatsDetails;
	TSharedPtr<FShaderFormatsPropertyDetails> VulkanTargetShaderFormatsDetails;

	FAudioPluginWidgetManager AudioPluginWidgetManager;
};
