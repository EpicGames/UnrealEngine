// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldSettingsDetails.h"

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "GameModeInfoCustomizer.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailChildrenBuilder.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "ScopedTransaction.h"
#include "Settings/EditorExperimentalSettings.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "WorldSettingsDetails"


FWorldSettingsDetails::~FWorldSettingsDetails()
{
}

/* IDetailCustomization overrides
 *****************************************************************************/

void FWorldSettingsDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("GameMode");
	CustomizeGameInfoProperty("DefaultGameMode", DetailBuilder, Category);

	AddLightmapCustomization(DetailBuilder);

	AddWorldCustomization(DetailBuilder);
	
	DetailBuilder.HideProperty(AActor::GetHiddenPropertyName(), AActor::StaticClass());
}


/* FWorldSettingsDetails implementation
 *****************************************************************************/


void FWorldSettingsDetails::CustomizeGameInfoProperty( const FName& PropertyName, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder )
{
	// Get the object that we are viewing details of. Expect to only edit one WorldSettings object at a time!
	TArray< TWeakObjectPtr<UObject> > ObjectsCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);
	UObject* ObjectCustomized = (ObjectsCustomized.Num() > 0) ? ObjectsCustomized[0].Get() : NULL;

	// Allocate customizer object
	GameInfoModeCustomizer = MakeShareable(new FGameModeInfoCustomizer(ObjectCustomized, PropertyName));

	// Then use it to customize
	GameInfoModeCustomizer->CustomizeGameModeSetting(DetailBuilder, CategoryBuilder);
}


void FWorldSettingsDetails::AddLightmapCustomization( IDetailLayoutBuilder& DetailBuilder )
{
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Lightmass");

	TSharedRef<FLightmapCustomNodeBuilder> LightMapGroupBuilder = MakeShareable(new FLightmapCustomNodeBuilder(DetailBuilder.GetThumbnailPool()));
	const bool bForAdvanced = true;
	Category.AddCustomBuilder(LightMapGroupBuilder, bForAdvanced);
}

void FWorldSettingsDetails::AddWorldCustomization(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	ULevel* CustomizedLevel = nullptr;
	if (CustomizedObjects.Num() > 0)
	{
		if (AWorldSettings* WorldSettings = Cast<AWorldSettings>(CustomizedObjects[0]))
		{
			CustomizedLevel = WorldSettings->GetLevel();
			SelectedWorldSettings = WorldSettings;
		}
	}

	// Hide some of the WorldPartition properties found in AActor
	TArray<TSharedRef<IPropertyHandle>> PropertiesToHide;
	PropertiesToHide.Add(DetailBuilder.GetProperty(AActor::GetRuntimeGridPropertyName(), AActor::StaticClass()));
	PropertiesToHide.Add(DetailBuilder.GetProperty(AActor::GetIsSpatiallyLoadedPropertyName(), AActor::StaticClass()));
	for (TSharedRef<IPropertyHandle> Property : PropertiesToHide)
	{
		DetailBuilder.HideProperty(Property);
	}

	if (CustomizedLevel)
	{
		IDetailCategoryBuilder& WorldCategory = DetailBuilder.EditCategory("World");
		
		WorldCategory.AddCustomRow(LOCTEXT("LevelUseExternalActorsRow", "LevelUseExternalActors"), true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LevelUseExternalActors", "Use External Actors"))
				.ToolTipText(LOCTEXT("ActorPackagingMode_ToolTip", "Use external actors, new actor spawned in this level will be external and existing external actors will be loaded on load."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(this, &FWorldSettingsDetails::IsUseExternalActorsEnabled, CustomizedLevel)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FWorldSettingsDetails::OnUseExternalActorsChanged, CustomizedLevel)
				.IsChecked(this, &FWorldSettingsDetails::IsUseExternalActorsChecked, CustomizedLevel)
				.IsEnabled(this, &FWorldSettingsDetails::IsUseExternalActorsEnabled, CustomizedLevel)
			];
		

		const bool bIsUsingActorFolders = CustomizedLevel->IsUsingActorFolders();
		WorldCategory.AddCustomRow(LOCTEXT("LevelUseActorFoldersRow", "LevelUseActorFolders"), true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LevelUseActorFolders", "Use Actor Folder Objects"))
				.ToolTipText(LOCTEXT("LevelUseActorFolders_ToolTip", "Use actor folder objects, actor folders of this level will be persistent in their own object."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(this, &FWorldSettingsDetails::IsUsingActorFoldersEnabled, CustomizedLevel)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FWorldSettingsDetails::OnUseActorFoldersChanged, CustomizedLevel)
				.IsChecked(this, &FWorldSettingsDetails::IsUsingActorFoldersChecked, CustomizedLevel)
				.IsEnabled(this, &FWorldSettingsDetails::IsUsingActorFoldersEnabled, CustomizedLevel)
			];
		

		if (IsPartitionedWorld(CustomizedLevel))
		{
			IDetailCategoryBuilder& WorldPartitionCategory = DetailBuilder.EditCategory("WorldPartition");

			WorldPartitionCategory.AddCustomRow(LOCTEXT("DefaultWorldPartitionSettingsRow", "DefaultWorldPartitionSettings"), true)
				.NameContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DefaultWorldPartitionSettings", "Default World Partition Settings"))
					.ToolTipText(LOCTEXT("DefaultWorldPartitionSettings_ToolTip", "Save or Reset the current World Partition default editor state"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.OnClicked_Lambda([CustomizedLevel]()
							{
								FScopedTransaction Transaction(LOCTEXT("ResetDefaultWorldPartitionSettings", "Reset Default World Partition Settings"));
								CustomizedLevel->GetWorldSettings()->ResetDefaultWorldPartitionSettings();
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ResetButtonText", "Reset"))
							.ToolTipText(LOCTEXT("ResetButtonToolTip", "Reset World Partition default editor state"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.OnClicked_Lambda([CustomizedLevel]()
							{
								FScopedTransaction Transaction(LOCTEXT("SaveDefaultWorldPartitionSettings", "Save Default World Partition Settings"));
								CustomizedLevel->GetWorldSettings()->SaveDefaultWorldPartitionSettings();
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SaveButtonText", "Save"))
							.ToolTipText(LOCTEXT("SaveButtonToolTip", "Save current World Partition editor state as map default"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						]
					]
				];
		}
	}
}

void FWorldSettingsDetails::OnUseActorFoldersChanged(ECheckBoxState BoxState, ULevel* Level)
{
	if (Level && (BoxState == ECheckBoxState::Checked))
	{
		Level->SetUseActorFolders(true, /*bInteractiveMode*/ true);
	}
}

ECheckBoxState FWorldSettingsDetails::IsUsingActorFoldersChecked(ULevel* Level) const
{
	return Level->IsUsingActorFolders() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FWorldSettingsDetails::IsUsingActorFoldersEnabled(ULevel* Level) const
{
	return !Level->IsUsingActorFolders();
}

void FWorldSettingsDetails::OnUseExternalActorsChanged(ECheckBoxState BoxState, ULevel* Level)
{
	if (Level != nullptr)
	{
		// Validate we have a saved map
		UPackage* LevelPackage = Level->GetOutermost();
		if (LevelPackage == GetTransientPackage()
			|| LevelPackage->HasAnyFlags(RF_Transient)
			|| !FPackageName::IsValidLongPackageName(LevelPackage->GetName()))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UseExternalActorsSaveMap", "You need to save the level before enabling the `Use External Actors` option."));
			return;
		}

		FText MessageTitle(LOCTEXT("ConvertActorPackagingDialog", "Convert Actors Packaging"));
		FText PackagingMode = !Level->IsUsingExternalActors() ? LOCTEXT("ExternalActors", "External") : LOCTEXT("InternalActors", "Internal");
		FText Message = FText::Format(LOCTEXT("ConvertActorPackagingMsg", "Do you want to convert all actors to {0} packaging as well?"), PackagingMode);
		EAppReturnType::Type ConvertAnswer = FMessageDialog::Open(EAppMsgType::YesNo, Message, MessageTitle);
		// if the user accepts, convert all actors to what the new packaging mode will be
		if (ConvertAnswer == EAppReturnType::Yes)
		{
			Level->Modify();
			Level->ConvertAllActorsToPackaging(BoxState == ECheckBoxState::Checked);

			if (Level->IsUsingExternalActors())
			{
				Level->SetUseActorFolders(true);
			}
		}
	}
}

ECheckBoxState FWorldSettingsDetails::IsUseExternalActorsChecked(ULevel* Level) const
{
	return Level->IsUsingExternalActors() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FWorldSettingsDetails::IsPartitionedWorld(ULevel* Level) const
{
	return UWorld::IsPartitionedWorld(Level->GetWorld());
}

bool FWorldSettingsDetails::IsUseExternalActorsEnabled(ULevel* Level) const
{
	return !IsPartitionedWorld(Level);
}

FLightmapCustomNodeBuilder::FLightmapCustomNodeBuilder(const TSharedPtr<FAssetThumbnailPool>& InThumbnailPool)
{
	ThumbnailPool = InThumbnailPool;
}


FLightmapCustomNodeBuilder::~FLightmapCustomNodeBuilder()
{
	FEditorDelegates::OnLightingBuildKept.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}


void FLightmapCustomNodeBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;

	FEditorDelegates::OnLightingBuildKept.AddSP(this, &FLightmapCustomNodeBuilder::HandleLightingBuildKept);
	FEditorDelegates::NewCurrentLevel.AddSP(this, &FLightmapCustomNodeBuilder::HandleNewCurrentLevel);
}


void FLightmapCustomNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew( STextBlock )
		.Text( LOCTEXT("LightmapHeaderRowContent", "Lightmaps") )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];

	NodeRow.ValueContent()
	[
		SNew( STextBlock )
		.Text( this, &FLightmapCustomNodeBuilder::GetLightmapCountText )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];
}


void FLightmapCustomNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	RefreshLightmapItems();

	for(TSharedPtr<FLightmapItem>& Item : LightmapItems)
	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("LightMapsFilter", "Lightmaps"))
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		[
			MakeLightMapList(Item)
		];
	}
}


FText FLightmapCustomNodeBuilder::GetLightmapCountText() const
{
	return FText::Format( LOCTEXT("LightmapHeaderRowCount", "{0} Lightmap(s)"), FText::AsNumber(LightmapItems.Num()) );
}


void FLightmapCustomNodeBuilder::HandleLightingBuildKept()
{
	OnRegenerateChildren.ExecuteIfBound();
}

void FLightmapCustomNodeBuilder::HandleNewCurrentLevel()
{
	OnRegenerateChildren.ExecuteIfBound();
}


TSharedRef<SWidget> FLightmapCustomNodeBuilder::MakeLightMapList(TSharedPtr<FLightmapItem> LightMapItem)
{
	if ( !ensure(LightMapItem.IsValid()) )
	{
		return SNullWidget::NullWidget;
	}

	const uint32 ThumbnailResolution = 64;
	const uint32 ThumbnailBoxPadding = 4;
	UObject* LightMapObject = FindObject<UObject>(NULL, *LightMapItem->ObjectPath);
	FAssetData LightMapAssetData(LightMapObject);

	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.bAllowFadeIn = true;

	TWeakPtr<FLightmapItem> LightmapWeakPtr = LightMapItem;
	return
		SNew(SBorder)
		.BorderImage(nullptr)
		.Padding(0.0f)
		.OnMouseButtonUp(this, &FLightmapCustomNodeBuilder::OnMouseButtonUp, LightmapWeakPtr)
		.OnMouseDoubleClick(this, &FLightmapCustomNodeBuilder::OnLightMapListMouseButtonDoubleClick, LightmapWeakPtr)
		[
			SNew(SHorizontalBox)
			// Viewport
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.WidthOverride( static_cast<float>(ThumbnailResolution + ThumbnailBoxPadding * 2) )
				.HeightOverride( static_cast<float>(ThumbnailResolution + ThumbnailBoxPadding * 2) )
				[
					// Drop shadow border
					SNew(SBorder)
					.Padding(ThumbnailBoxPadding)
					.BorderImage( FAppStyle::GetBrush("ContentBrowser.ThumbnailShadow") )
					[
						LightMapItem->Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
					]
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 1.f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromName(LightMapAssetData.AssetName) )
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 1.f)
				[
					// Class
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromString(LightMapAssetData.AssetClassPath.ToString()))
				]
			]
		];
}


TSharedPtr<SWidget> FLightmapCustomNodeBuilder::OnGetLightMapContextMenuContent(TSharedPtr<FLightmapItem> Lightmap)
{
	if (Lightmap.IsValid() )
	{
		FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, NULL);

		MenuBuilder.BeginSection("LightMapsContextMenuSection", LOCTEXT("LightMapsContextMenuHeading", "Options") );
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ViewLightmapLabel", "View Lightmap"),
				LOCTEXT("ViewLightmapTooltip", "Opens the texture editor with this lightmap."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FLightmapCustomNodeBuilder::ExecuteViewLightmap, Lightmap->ObjectPath))
				);
		}
		MenuBuilder.EndSection(); //LightMapsContextMenuSection

		return MenuBuilder.MakeWidget();
	}

	return nullptr;
}


FReply FLightmapCustomNodeBuilder::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr<FLightmapItem> Lightmap)
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		TSharedPtr<SWidget> MenuContent = OnGetLightMapContextMenuContent(Lightmap.Pin());

		if(MenuContent.IsValid() && MouseEvent.GetEventPath() != nullptr)
		{
			const FVector2D& SummonLocation = MouseEvent.GetScreenSpacePosition();
			FWidgetPath WidgetPath = *MouseEvent.GetEventPath();
			FSlateApplication::Get().PushMenu(WidgetPath.Widgets.Last().Widget, WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FLightmapCustomNodeBuilder::OnLightMapListMouseButtonDoubleClick(const FGeometry& MyGeom, const FPointerEvent& PointerEvent, TWeakPtr<FLightmapItem> SelectedLightmap)
{
	if ( ensure(SelectedLightmap.IsValid()) )
	{
		ExecuteViewLightmap(SelectedLightmap.Pin()->ObjectPath);
	}

	return FReply::Handled();
}


void FLightmapCustomNodeBuilder::ExecuteViewLightmap(FString SelectedLightmapPath)
{
	UObject* LightMapObject = FindObject<UObject>(NULL, *SelectedLightmapPath);
	if ( LightMapObject )
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LightMapObject);
	}
}

void FLightmapCustomNodeBuilder::RefreshLightmapItems()
{
	LightmapItems.Empty();

	FWorldContext& Context = GEditor->GetEditorWorldContext();
	UWorld* World = Context.World();
	if ( World )
	{
		TArray<UTexture2D*> LightMapsAndShadowMaps;
		World->GetLightMapsAndShadowMaps(World->GetCurrentLevel(), LightMapsAndShadowMaps, false);

		for ( auto ObjIt = LightMapsAndShadowMaps.CreateConstIterator(); ObjIt; ++ObjIt )
		{
			UTexture2D* CurrentObject = *ObjIt;
			if (CurrentObject)
			{
				FAssetData AssetData = FAssetData(CurrentObject);
				const uint32 ThumbnailResolution = 64;
				TSharedPtr<FAssetThumbnail> LightMapThumbnail = MakeShareable( new FAssetThumbnail( AssetData, ThumbnailResolution, ThumbnailResolution, ThumbnailPool ) );
				TSharedPtr<FLightmapItem> NewItem = MakeShareable( new FLightmapItem(CurrentObject->GetPathName(), LightMapThumbnail) );
				LightmapItems.Add(NewItem);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
