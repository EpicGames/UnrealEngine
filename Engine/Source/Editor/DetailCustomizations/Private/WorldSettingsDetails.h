// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAssetThumbnailPool;
class FDetailWidgetRow;
class FGameModeInfoCustomizer;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SWidget;
class ULevel;
class UObject;
struct FGeometry;
struct FPointerEvent;

/**
 * Implements details panel customizations for AWorldSettings fields.
 */
class FWorldSettingsDetails
	: public IDetailCustomization
{
public:

	virtual ~FWorldSettingsDetails();

	// IDetailCustomization interface

	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

public:

	/**
	 * Makes a new instance of this detail layout class.
	 *
	 * @return The created instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance( )
	{
		return MakeShareable(new FWorldSettingsDetails);
	}

protected:

	/**
	 * Customizes an AGameInfo property with the given name.
	 *
	 * @param PropertyName The property to customize.
	 * @param DetailBuilder The detail builder.
	 * @param CategoryBuilder The category builder
	 */
	void CustomizeGameInfoProperty( const FName& PropertyName, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder );

	/**
	 * Adds the lightmap customization to the Lightmass section
	 *
	 * @param DetailBuilder The detail builder.
	 */
	void AddLightmapCustomization( IDetailLayoutBuilder& DetailBuilder );

	/**
	 * Add customization to the World section
	 * @param DetailBuilder the detail builder.
	 */
	void AddWorldCustomization(IDetailLayoutBuilder& DetailBuilder);

private:
	// return true if level's owning world is partitioned.
	bool IsPartitionedWorld(ULevel* Level) const;
	
	// Called when `ULevel::bUseExternalActors` changes.
	void OnUseExternalActorsChanged(ECheckBoxState State, ULevel* Level);

	// return the state of `ULevel::bUseExternalActors`.
	ECheckBoxState IsUseExternalActorsChecked(ULevel* Level) const;

	// return true if the state of 'ULevel::bUseExternalActors' can be changed.
	bool IsUseExternalActorsEnabled(ULevel* Level) const;

	// Called when `ULevel::bUseActorFolders` changes.
	void OnUseActorFoldersChanged(ECheckBoxState BoxState, ULevel* Level);

	// return the state of `ULevel::bUseActorFolders`.
	ECheckBoxState IsUsingActorFoldersChecked(ULevel* Level) const;

	// return true if the state of 'ULevel::bUseActorFolders' can be changed.
	bool IsUsingActorFoldersEnabled(ULevel* Level) const;

	// Handles checking whether a given asset is acceptable for drag-and-drop.
	bool HandleAssetDropTargetIsAssetAcceptableForDrop( const UObject* InObject ) const;

	// Handles dropping an asset.
	void HandleAssetDropped( UObject* Object, TSharedRef<IPropertyHandle> GameInfoProperty );

	/** Helper class to customizer GameMode property */
	TSharedPtr<FGameModeInfoCustomizer>	GameInfoModeCustomizer;

	TWeakObjectPtr<class AWorldSettings> SelectedWorldSettings;
};


/** Custom struct for each group of arguments in the function editing details */
class FLightmapCustomNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FLightmapCustomNodeBuilder>
{
public:
	FLightmapCustomNodeBuilder(const TSharedPtr<FAssetThumbnailPool>& InThumbnailPool);
	~FLightmapCustomNodeBuilder();

protected:

	// IDetailCustomNodeBuilder interface
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return FName(TEXT("Lightmaps")); }
	virtual bool InitiallyCollapsed() const override { return false; }

private:

	struct FLightmapItem
	{
		FString ObjectPath;
		TSharedPtr<class FAssetThumbnail> Thumbnail;

		FLightmapItem(const FString& InObjectPath, const TSharedPtr<class FAssetThumbnail>& InThumbnail)
			: ObjectPath(InObjectPath)
			, Thumbnail(InThumbnail)
		{}
	};

	/** Handler for the lightmap count text in the right hand column */
	FText GetLightmapCountText() const;

	/** Handler for when lighting has been rebuilt and kept */
	void HandleLightingBuildKept();

	/** Handler for when the current level changes */
	void HandleNewCurrentLevel();

	/** Handler for light map list view widget creation */
	TSharedRef<SWidget> MakeLightMapList(TSharedPtr<FLightmapItem> AssetItem);

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetLightMapContextMenuContent(TSharedPtr<FLightmapItem> Lightmap);

	/** Handler for right clicking an item */
	FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr<FLightmapItem> Lightmap);

	/** Handler for double clicking an item */
	FReply OnLightMapListMouseButtonDoubleClick(const FGeometry& MyGeom, const FPointerEvent& PointerEvent, TWeakPtr<FLightmapItem> SelectedLightmap);

	/** Handler for when "View" is selected in the light map list */
	void ExecuteViewLightmap(FString SelectedLightmapPath);

	/** Refreshes the list of lightmaps to display */
	void RefreshLightmapItems();

private:

	/** Delegate to handle refreshing this group */
	FSimpleDelegate OnRegenerateChildren;

	/** The list view showing light maps in this world */
	TArray<TSharedPtr<FLightmapItem>> LightmapItems;
	TSharedPtr<class FAssetThumbnailPool> ThumbnailPool;
};
