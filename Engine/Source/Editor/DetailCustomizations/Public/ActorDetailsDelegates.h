// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AActor;

/** Delegate that returns a const& to an array of actors this view is inspecting */
DECLARE_DELEGATE_RetVal( const TArray< TWeakObjectPtr<AActor> >&, FGetSelectedActors );

DECLARE_MULTICAST_DELEGATE_TwoParams(FExtendActorDetails, class IDetailLayoutBuilder&, const FGetSelectedActors&);

/** Delegate that is invoked to extend the actor details view */
extern DETAILCUSTOMIZATIONS_API FExtendActorDetails OnExtendActorDetails;
