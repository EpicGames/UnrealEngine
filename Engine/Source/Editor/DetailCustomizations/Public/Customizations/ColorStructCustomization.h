// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "MathStructCustomizations.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;
class SBorder;
class SColorPicker;
class SWidget;
class SWindow;
struct FGeometry;
struct FPointerEvent;

/**
 * Base class for color struct customization (FColor,FLinearColor).
 */
class FColorStructCustomization
	: public FMathStructCustomization
{
public:

	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();
	UE_API virtual ~FColorStructCustomization() override;

protected:

	FColorStructCustomization()
		: bIsLinearColor(false)
		, bIgnoreAlpha(false)
		, bIsInlineColorPickerVisible(false)
		, bIsInteractive(false)
		, bIsInBlueprint(false)
		, bDontUpdateWhileEditing(false)
	{
	}

protected:

	struct FLinearOrSrgbColor;

	/**
	 * Get the color used by this struct as a linear color value
	 * @param InColor To be filled with the color value used by this struct, or white if this struct is being used to edit multiple values
	 * @return The result of trying to get the color value
	 */
	UE_API virtual FPropertyAccess::Result GetColorAsLinear(FLinearColor& InColor) const;

	/**
	 * Does the type have Alpha Support
	 * @return true if it does
	*/
	virtual bool TypeSupportsAlpha() const { return true; }

	/**
	 * Stores the colors from the property into SavedPreColorPickerColors
	*/
	UE_API virtual void GatherSavedPreColorPickerColors();

	/**
	 * Stores the color as a string in LastPickerColorString
	*/
	UE_API virtual void SetLastPickerColorString(const FLinearColor NewColor);

	/**
	 * Converts Colors into strings
	 * @param Colors Array of colors to convert
	 * @return Array of colors converted to string
	*/
	UE_API virtual TArray<FString> ConvertToPerObjectColors(const TArray<FLinearOrSrgbColor>& Colors) const;

protected:

	/** Creates the color widget that when clicked spawns the color picker window. */
	UE_API TSharedRef<SWidget> CreateColorWidget(TWeakPtr<IPropertyHandle>);

	/**
	 * Does this struct have multiple values?
	 * @return EVisibility::Visible if it does, EVisibility::Collapsed otherwise
	 */
	UE_API EVisibility GetMultipleValuesTextVisibility() const;

	/**
	 * Creates a new color picker for interactively selecting the color
	 *
	 * @param bUseAlpha If true alpha will be displayed, otherwise it is ignored
	 * @param bOnlyRefreshOnOk If true the value of the property will only be refreshed when the user clicks OK in the color picker
	 */
	UE_API void CreateColorPicker(bool bUseAlpha);
	
	/** Creates a new color picker for interactively selecting color */
	UE_API TSharedRef<SColorPicker> CreateInlineColorPicker(TWeakPtr<IPropertyHandle>);

	/**
	 * Called when the property is set from the color picker 
	 * 
	 * @param NewColor The new color value
	 */
	UE_API void OnSetColorFromColorPicker(FLinearColor NewColor);
	
	/**
	 * Called to reset all colors to before the color picker spawned
	 */
	UE_API void ResetColors();

	/**
	 * Called when the user clicks cancel in the color picker
	 * The values are reset to their original state when this happens
	 *
	 * @param OriginalColor Original color of the property
	 */
	UE_API void OnColorPickerCancelled(FLinearColor OriginalColor);

	/**
	 * Called when the color picker window is clsoed
	 */
	UE_API void OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window);

	/**
	 * Called when the user enters an interactive color change (dragging something in the picker)
	 */
	UE_API void OnColorPickerInteractiveBegin();

	/**
	 * Called when the user completes an interactive color change (dragging something in the picker)
	 */
	UE_API void OnColorPickerInteractiveEnd();

	/**
	 * @return The color that should be displayed in the color block                                                              
	 */
	UE_API FLinearColor OnGetColorForColorBlock() const;
	
	/**
	 * @return The color that should be displayed in the color block in slate color format                                                           
	 */
	UE_API FSlateColor  OnGetSlateColorForBlock() const;

	/**
	 * @return The border color encompassing the entire color block                                                         
	 */
	UE_API FSlateColor GetColorWidgetBorderColor() const;
	/**
	 * Called when the user clicks in the color block (opens inline color picker)
	 */
	UE_API FReply OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	
	/** Called when the user clicks on the the button to get the full color picker */
	UE_API FReply OnOpenFullColorPickerClicked();

protected:

	// FMathStructCustomization interface

	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override;
	UE_API virtual void GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren) override;

protected:
	/** Stores a linear or srb color without converting between the two. Only one is valid at a time */
	struct FLinearOrSrgbColor
	{
		FLinearOrSrgbColor(const FLinearColor& InLinearColor)
			: LinearColor(InLinearColor)
		{}

		FLinearOrSrgbColor(const FColor& InSrgbColor)
			: SrgbColor(InSrgbColor)
		{}

		FLinearColor GetLinear() const { return LinearColor; }
		FColor GetSrgb() const { return SrgbColor; }
	private:
		FLinearColor LinearColor;
		FColor SrgbColor;
	};

	/** Saved per struct colors in case the user clicks cancel in the color picker */
	TArray<FLinearOrSrgbColor> SavedPreColorPickerColors;

	/** Color struct handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Cached widget for the color picker to use as a parent */
	TSharedPtr<SWidget> ColorPickerParentWidget;

	TSharedPtr<SWidget > ColorWidgetBackgroundBorder;

	/** Overrides the default state of the sRGB check box */
	TOptional<bool> sRGBOverride;

	/** True if the property is a linear color property */
	bool bIsLinearColor;

	/** True if the property wants to ignore the alpha component */
	bool bIgnoreAlpha;

	/** True if the inline color picker is visible */
	bool bIsInlineColorPickerVisible;

	/** True if the user is performing an interactive color change */
	bool bIsInteractive;
	
	/** True if the property we are editing is contained within a blueprint */
	bool bIsInBlueprint;

	/** Last color set from color picker as string*/
	FString LastPickerColorString;


	/** The value won;t be updated while editing */
	bool bDontUpdateWhileEditing;

	TOptional<int32> TransactionIndex;	
};

#undef UE_API
