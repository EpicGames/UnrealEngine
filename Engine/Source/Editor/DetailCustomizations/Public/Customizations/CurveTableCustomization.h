// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "DetailWidgetRow.h"
#include "DetailWidgetRow.h"
#include "Engine/CurveTable.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "IPropertyUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class ITableRow;
class STableViewBase;
class SWidget;
class UObject;
struct FRealCurve;

/**
 * Customizes a DataTable asset to use a dropdown
 */
class FCurveTableCustomizationLayout : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{
		return MakeShareable( new FCurveTableCustomizationLayout );
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override
	{
		this->StructPropertyHandle = InStructPropertyHandle;

		HeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget( FText::GetEmpty(), FText::GetEmpty() )
			];
	}

	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override
	{
		/** Get all the existing property handles */
		CurveTablePropertyHandle = InStructPropertyHandle->GetChildHandle( "CurveTable" );
		RowNamePropertyHandle = InStructPropertyHandle->GetChildHandle( "RowName" );

		if( CurveTablePropertyHandle->IsValidHandle() && RowNamePropertyHandle->IsValidHandle() )
		{
			/** Queue up a refresh of the selected item, not safe to do from here */
			StructCustomizationUtils.GetPropertyUtilities()->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FCurveTableCustomizationLayout::OnInitialRefresh));

			CreateCurveTableChildProperty(StructBuilder);

			FSimpleDelegate OnCurveTableChangedDelegate = FSimpleDelegate::CreateSP( this, &FCurveTableCustomizationLayout::OnCurveTableChanged );
			CurveTablePropertyHandle->SetOnPropertyValueChanged( OnCurveTableChangedDelegate );

			/** Construct a combo box widget to select from a list of valid options */
			StructBuilder.AddCustomRow( NSLOCTEXT("CurveTable", "RowNameLabel", "Row Name" ) )
			.NameContent()
				[
					SNew( STextBlock )
					.Text( NSLOCTEXT("CurveTable", "RowNameLabel", "Row Name" ) )
					.Font( StructCustomizationUtils.GetRegularFont() )
				]
			.ValueContent()
				[
					SAssignNew( RowNameComboButton, SComboButton )
					.OnGetMenuContent( this, &FCurveTableCustomizationLayout::GetListContent )
					.ContentPadding( FMargin( 2.0f, 2.0f ) )
					.ButtonContent()
					[
						SNew( STextBlock )
						.Text( this, &FCurveTableCustomizationLayout::GetRowNameComboBoxContentText )
						.ToolTipText( this, &FCurveTableCustomizationLayout::GetRowNameComboBoxContentText )
					]
				];
		}
	}
	
	virtual void CreateCurveTableChildProperty(IDetailChildrenBuilder& StructBuilder)
	{
		/** Edit the data table uobject as normal */
		StructBuilder.AddProperty( CurveTablePropertyHandle.ToSharedRef() );
	}

protected:
	/** Init the contents the combobox sources its data off */
	TSharedPtr<FString> InitWidgetContent(bool bShouldModifyData)
	{
		TSharedPtr<FString> InitialValue;
		
		FName RowName;
		const FPropertyAccess::Result RowResult = RowNamePropertyHandle->GetValue( RowName );
		RowNames.Empty();

		if ( RowResult != FPropertyAccess::Fail )
		{
			/** Get the properties we wish to work with */
			UCurveTable* CurveTable = nullptr;
			CurveTablePropertyHandle->GetValue((UObject*&)CurveTable);

			if ( CurveTable != nullptr )
			{
				/** Extract all the row names from the RowMap */
				for ( TMap<FName, FRealCurve*>::TConstIterator Iterator(CurveTable->GetRowMap()); Iterator; ++Iterator )
				{
					/** Create a simple array of the row names */
					TSharedRef<FString> RowNameItem = MakeShareable(new FString(Iterator.Key().ToString()));
					RowNames.Add(RowNameItem);

					/** Set the initial value to the currently selected item */
					if ( Iterator.Key() == RowName )
					{
						InitialValue = RowNameItem;
					}
				}
			}

			if (!InitialValue.IsValid() && RowResult != FPropertyAccess::MultipleValues)
			{
				// Only set initial value to none if it's not a multiple value case
				InitialValue = MakeShareable(new FString(TEXT("None")));
			}
				

			/** Reset the initial value to ensure a valid entry is set */
			if ( InitialValue.IsValid() && bShouldModifyData )
			{
				TArray<void*> RawData;

				// This raw data access is necessary to avoid setting the value during details customization setup which would cause infinite recursion repetedly reinitializing this customization
				RowNamePropertyHandle->AccessRawData(RawData);
				if ( RawData.Num() == 1 )
				{
					FName* RawFName = (FName*)RawData[0];
					*RawFName = **InitialValue;
				}
			}
		}

		return InitialValue;
	}

	/** Returns the ListView for the ComboButton */
	UE_API TSharedRef<SWidget> GetListContent();

	/** Initial refresh, will not modify data */
	void OnInitialRefresh()
	{
		CurrentSelectedItem = InitWidgetContent(false);
		if (RowNameComboListView.IsValid() && CurrentSelectedItem.IsValid())
		{
			RowNameComboListView->SetSelection(CurrentSelectedItem);
			RowNameComboListView->RequestListRefresh();
		}
	}

	/** Delegate to refresh the drop down when the datatable changes */
	void OnCurveTableChanged()
	{
		CurrentSelectedItem = InitWidgetContent(true);
		if (RowNameComboListView.IsValid())
		{
			RowNameComboListView->SetSelection(CurrentSelectedItem);
			RowNameComboListView->RequestListRefresh();
		}
	}

	/** Return the representation of the the row names to display */
	TSharedRef<ITableRow>  HandleRowNameComboBoxGenarateWidget( TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( STableRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNew( STextBlock ).Text( FText::FromString(*InItem) )
			];
	}

	/** Display the current selection */
	FText GetRowNameComboBoxContentText( ) const
	{
		FName RowNameValue;
		const FPropertyAccess::Result RowResult = RowNamePropertyHandle->GetValue( RowNameValue );
		if ( RowResult != FPropertyAccess::MultipleValues )
		{
			TSharedPtr<FString> SelectedRowName = CurrentSelectedItem;
			if ( SelectedRowName.IsValid() )
			{
				return FText::FromString(*SelectedRowName);
			}
			else
			{
				return NSLOCTEXT( "CurveTableCustomization", "None", "None" );
			}
		}
		return NSLOCTEXT( "CurveTableCustomization", "MultipleValues", "Multiple Values" );
	}

	/** Update the root data on a change of selection */
	void OnSelectionChanged( TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo )
	{
		if( SelectedItem.IsValid() )
		{
			CurrentSelectedItem = SelectedItem; 
			FName NewValue = FName( **SelectedItem );
			RowNamePropertyHandle->SetValue( NewValue );

			// Close the combo
			RowNameComboButton->SetIsOpen( false );
		}
	}

	/** Called by Slate when the filter box changes text. */
	void OnFilterTextChanged( const FText& InFilterText )
	{
		FString CurrentFilterText = InFilterText.ToString();

		FName RowName;
		const FPropertyAccess::Result RowResult = RowNamePropertyHandle->GetValue( RowName );
		RowNames.Empty();

		/** Get the properties we wish to work with */
		UCurveTable* CurveTable = nullptr;
		CurveTablePropertyHandle->GetValue( ( UObject*& )CurveTable );

		if( CurveTable != nullptr )
		{
			/** Extract all the row names from the RowMap */
			for( TMap<FName, FRealCurve*>::TConstIterator Iterator( CurveTable->GetRowMap() ); Iterator; ++Iterator )
			{
				/** Create a simple array of the row names */
				FString RowString = Iterator.Key().ToString();
				if( CurrentFilterText == TEXT("") || RowString.Contains(CurrentFilterText) )
				{
					TSharedRef<FString> RowNameItem = MakeShareable( new FString( RowString ) );				
					RowNames.Add( RowNameItem );					
				}			
			}
		}		
		RowNameComboListView->RequestListRefresh();
	}

	/** The comboButton objects */
	TSharedPtr<SComboButton> RowNameComboButton;
	TSharedPtr<SListView<TSharedPtr<FString> > > RowNameComboListView;
	TSharedPtr<FString> CurrentSelectedItem;	
	/** Handle to the struct properties being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyHandle> CurveTablePropertyHandle;
	TSharedPtr<IPropertyHandle> RowNamePropertyHandle;
	/** A cached copy of strings to populate the combo box */
	TArray<TSharedPtr<FString> > RowNames;
};

#undef UE_API
