// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

/** Arguments for the linear color ramp customization */
struct FLinearColorRampCustomizationArgs
{
	/** When true, the alpha channel is displayed and editable */
	bool bWithAlphaChannel = true;
};

class FLinearColorRampCustomization : public IPropertyTypeCustomization
{
public:
	FLinearColorRampCustomization() = default;
	FLinearColorRampCustomization(const FLinearColorRampCustomizationArgs InArgs);

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	DETAILCUSTOMIZATIONS_API static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it, with arguments */
	DETAILCUSTOMIZATIONS_API static TSharedRef<IPropertyTypeCustomization> MakeInstance(const FLinearColorRampCustomizationArgs InArgs);

	//~ Begin - IPropertyTypeCustomization interface
	DETAILCUSTOMIZATIONS_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*CustomizationUtils*/) override {}
	//~ End - IPropertyTypeCustomization interface

private:
	/** Arguments for this customization */
	const FLinearColorRampCustomizationArgs Args;
};
