// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Math/UnitConversion.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SWidget.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class SWidget;
struct FSlateBrush;

/**
 * Base class for math struct customization (e.g, vector, rotator, color)                                                              
 */
class FMathStructCustomization
	: public IPropertyTypeCustomization
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Notification when the max/min slider values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnNumericEntryBoxDynamicSliderMinMaxValueChanged, float, TWeakPtr<SWidget>, bool, bool);

	FMathStructCustomization()
		: bIsUsingSlider(false)
		, bPreserveScaleRatio(false)
	{ }

	virtual ~FMathStructCustomization() = default;

	/** IPropertyTypeCustomization instance */
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** Return max/min slider value changed delegate (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMaxValueChanged; }
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMinValueChanged; }

	/** Callback when the max/min spinner value are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	template <typename NumericType>
	void OnDynamicSliderMaxValueChanged(NumericType NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher);

	template <typename NumericType>
	void OnDynamicSliderMinValueChanged(NumericType NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower);

	/**
	* Called to see if the value is enabled for editing
	*
	* @param WeakHandlePtr	Handle to the property that the new value is for
	*/
	UE_API bool IsValueEnabled(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;
	
	// Argument struct for ExtractNumericMetadata, to make it easier to add new metadata to extract.
	template <typename NumericType>
	struct FNumericMetadata
	{
		TOptional<NumericType> MinValue;
		TOptional<NumericType> MaxValue;
		TOptional<NumericType> SliderMinValue;
		TOptional<NumericType> SliderMaxValue;
		TSharedPtr<INumericTypeInterface<NumericType>> TypeInterface;
		NumericType SliderExponent;
		NumericType Delta;
		int32 LinearDeltaSensitivity;
		float ShiftMultiplier;
		float CtrlMultiplier;
		bool bSupportDynamicSliderMaxValue;
		bool bSupportDynamicSliderMinValue;
		bool bAllowSpinBox;
	};

	/** Utility function that will extract common Math related numeric metadata */	
	template <typename NumericType>
	DETAILCUSTOMIZATIONS_API static void ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, FNumericMetadata<NumericType>& MetadataOut);

	template <typename NumericType>
	UE_DEPRECATED(5.0, "Use ExtractNumericMetadata overload with struct argument instead.")
		DETAILCUSTOMIZATIONS_API static void ExtractNumericMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, TOptional<NumericType>& MinValue,
			TOptional<NumericType>& MaxValue, TOptional<NumericType>& SliderMinValue, TOptional<NumericType>& SliderMaxValue,
			NumericType& SliderExponent, NumericType& Delta, int32& ShiftMouseMovePixelPerDelta,
			bool& bSupportDynamicSliderMaxValue, bool& bSupportDynamicSliderMinValue);

protected:

	/**
	 * Makes the header row for the customization
	 *
	 * @param StructPropertyHandle	Handle to the struct property
	 * @param Row	The header row to add widgets to
	 */
	UE_API virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row);

	/**
	 * Gets the sorted children for the struct
	 *
	 * @param StructPropertyHandle	The handle to the struct property
	 * @param OutChildren			The child array that should be populated in the order that children should be displayed
	 */
	UE_API virtual void GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray< TSharedRef<IPropertyHandle> >& OutChildren);

	/**
	 * Constructs a widget for the property handle 
	 *
	 * @param StructurePropertyHandle	handle of the struct property
	 * @param PropertyHandle	Child handle of the struct property
	 */
	UE_API virtual TSharedRef<SWidget> MakeChildWidget(
		TSharedRef<IPropertyHandle>& StructurePropertyHandle,
		TSharedRef<IPropertyHandle>& PropertyHandle);

	/**
	 * Gets the value as a float for the provided property handle
	 *
	 * @param WeakHandlePtr	Handle to the property to get the value from
	 * @return The value or unset if it could not be accessed
	 */
	template<typename NumericType>
	TOptional<NumericType> OnGetValue(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;

	/**
	 * Called when the value is committed from the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param CommitType	How the value was committed (unused)
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 */
	template<typename NumericType>
	void OnValueCommitted(NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr);
	

	/**
	 * Called when the value is changed in the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 */
	template<typename NumericType>
	void OnValueChanged(NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr);

	/**
	 * Called to set the value of the property handle.
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param Flags         The flags to pass when setting the value on the property handle.
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 */
	template<typename NumericType>
	void SetValue(NumericType NewValue, EPropertyValueSetFlags::Type Flags, TWeakPtr<IPropertyHandle> WeakHandlePtr);

	/**
	 * Gets the tooltip for the value. Displays the property name and the current value
	 * 
	 * @praram WeakHandlePtr Handle to the property to get the value from
	 */
	template <typename NumericType>
	UE_DEPRECATED(5.6, "OnGetValueToolTip is deprecated and no longer called.")
	UE_API FText OnGetValueToolTip(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;

private:

	UE_API TOptional<FTextFormat> OnGetValueToolTipTextFormat(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;

	/** Gets the brush to use for the lock icon. */
	UE_API const FSlateBrush* GetPreserveScaleRatioImage() const;

	/** Gets the checked value of the preserve scale ratio option */
	UE_API ECheckBoxState IsPreserveScaleRatioChecked() const;

	/** Called when the user toggles preserve ratio. */
	UE_API void OnPreserveScaleRatioToggled(ECheckBoxState NewState, TWeakPtr<IPropertyHandle> PropertyHandle);

	UE_API FReply OnNormalizeClicked(TWeakPtr<IPropertyHandle> PropertyHandle);

private:

	/** Called when a value starts to be changed by a slider */
	UE_API void OnBeginSliderMovement();

	/** Called when a value stops being changed by a slider */
	template<typename NumericType>
	void OnEndSliderMovement(NumericType NewValue);

	template<typename NumericType>
	TSharedRef<SWidget> MakeNumericWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle);

protected:

	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMaxValueChanged;
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMinValueChanged;

	/** All the sorted children of the struct that should be displayed */
	TArray< TSharedRef<IPropertyHandle> > SortedChildHandles;

	/** All created numeric entry box widget for this customization */
	TArray<TWeakPtr<SWidget>> NumericEntryBoxWidgetList;

	/** True if a value is being changed by dragging a slider */
	bool bIsUsingSlider;

	/** True if the ratio is locked when scaling occurs (uniform scaling) */
	bool bPreserveScaleRatio;
};

#undef UE_API
