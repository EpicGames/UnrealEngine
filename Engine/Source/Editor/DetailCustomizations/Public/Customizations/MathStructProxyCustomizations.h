// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "DetailLayoutBuilder.h"
#include "Customizations/MathStructCustomizations.h"
#include "Delegates/Delegate.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"
#include "Math/UnitConversion.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDetailWidgetRow;
class IPropertyHandle;
class IPropertyTypeCustomization;
class IPropertyTypeCustomizationUtils;
class IPropertyUtilities;
class SWidget;

/** 
 * Helper class used to track the dirty state of a proxy value
 */
template< typename ObjectType >
class TProxyValue
{
public:
	TProxyValue()
		: Value()
		, bIsSet(false)
	{
	}

	TProxyValue(const ObjectType& InValue)
		: Value(InValue)
		, bIsSet(false)
	{
	}

	/** 
	 * Set the wrapped value
	 * @param	InValue	The value to set.
	 */
	void Set(const ObjectType& InValue)
	{
		Value = InValue;
		bIsSet = true;
	}

	/** 
	 * Get the wrapped value
	 * @return the wrapped value
	 */
	const ObjectType& Get() const
	{
		return Value;
	}

	/** 
	 * Get the wrapped value
	 * @return the wrapped value
	 */
	ObjectType& Get()
	{
		return Value;
	}

	/** 
	 * Check to see if the value is set.
	 * @return whether the value is set.
	 */
	bool IsSet() const
	{
		return bIsSet;
	}

	/** 
	 * Mark the value as if it was set.
	 */
	void MarkAsSet()
	{
		bIsSet = true;
	}

private:
	/** The value we are tracking */
	ObjectType Value;

	/** Whether the value is set */
	bool bIsSet;
};

/**
 * Helper class used to track the state of properties of proxy values.
 */
template< typename ObjectType, typename PropertyType >
class TProxyProperty
{
public:
	TProxyProperty(const TSharedRef< TProxyValue<ObjectType> >& InValue, PropertyType& InPropertyValue)
		: Value(InValue)
		, Property(InPropertyValue)
		, bIsSet(false)
	{
	}

	/**
	 * Set the value of this property
	 * @param	InPropertyValue		The value of the property to set
	 */
	void Set(const PropertyType& InPropertyValue)
	{
		Property = InPropertyValue;
		Value->MarkAsSet();
		bIsSet = true;
	}

	/**
	 * Get the value of this property
	 * @return The value of the property
	 */
	const PropertyType& Get() const
	{
		return Property;
	}

	/** 
	 * Check to see if the value is set.
	 * @return whether the value is set.
	 */
	bool IsSet() const
	{
		return bIsSet;
	}

private:
	/** The proxy value we are tracking */
	TSharedRef< TProxyValue<ObjectType> > Value;

	/** The property of the value we are tracking */
	PropertyType& Property;

	/** Whether the value is set */
	bool bIsSet;
};

/** 
 * Helper class to aid representing math structs to the user in an editable form
 * e.g. representing a quaternion as a set of euler angles
 */
class FMathStructProxyCustomization : public FMathStructCustomization
{
public:
	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	/** FMathStructCustomization interface */
	UE_API virtual void MakeHeaderRow( TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row ) override;

protected:

	/**
	 * Cache the values from the property to the proxy.
	 * @param	WeakHandlePtr	The property handle to get values from.
	 * @return true if values(s) were successfully cached
	 */
	virtual bool CacheValues( TWeakPtr<IPropertyHandle> WeakHandlePtr ) const = 0;

	/**
	 * Flush the values from the proxy to the property.
	 * @param	WeakHandlePtr	The property handle to set values to.
	 * @return true if values(s) were successfully flushed
	 */
	virtual bool FlushValues( TWeakPtr<IPropertyHandle> WeakHandlePtr ) const = 0;

	/** 
	 * Helper function to make a numeric property widget to edit a proxy value.
	 * @param	StructPropertyHandle	Property handle to the containing struct
	 * @param	ProxyValue				The value we will be editing in the proxy data.
	 * @param	Label					A label to use for this value.
	 * @param	bRotationInDegrees		Whether this is to be used for a rotation value (configures sliders appropriately).
	 * @return the newly created widget.
	 */
	template<typename ProxyType, typename NumericType>
	TSharedRef<SWidget> MakeNumericProxyWidget(TSharedRef<IPropertyHandle>& StructPropertyHandle, TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue, const FText& Label, bool bRotationInDegrees = false, const FLinearColor& LabelColor = FCoreStyle::Get().GetColor("DefaultForeground"));

	template <typename ProxyType, typename NumericType>
	UE_DEPRECATED(5.6, "OnGetValueToolTip is deprecated and no longer called. Please use the Label in MakeNumericProxyWidget to customize the display name portion of the ToolTip.")
	FText OnGetValueToolTip(TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef<TProxyProperty<ProxyType, NumericType>> ProxyValue, FText Label) const;

	DETAILCUSTOMIZATIONS_API TOptional<FTextFormat> OnGetValueToolTipTextFormat(FText Label) const;

private:

	/**
	 * Gets the value as a float for the provided property handle
	 *
	 * @param WeakHandlePtr	Handle to the property to get the value from
	 * @param ProxyValue	Proxy value to get value from.
	 * @return The value or unset if it could not be accessed
	 */
	template<typename ProxyType, typename NumericType>
	TOptional<NumericType> OnGetValue( TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue ) const;

	/**
	 * Called when the value is committed from the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param CommitType	How the value was committed (unused)
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 */
	template<typename ProxyType, typename NumericType>
	void OnValueCommitted( NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue );
	
	/**
	 * Called when the value is changed in the property editor
	 *
	 * @param NewValue		The new value of the property as a float
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 */
	template<typename ProxyType, typename NumericType>
	void OnValueChanged( NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue );

	/** Called when a value starts to be changed by a slider */
	DETAILCUSTOMIZATIONS_API void OnBeginSliderMovement();

	/** Called when a value stops being changed by a slider */
	template<typename ProxyType, typename NumericType>
	void OnEndSliderMovement( NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue );

protected:
	/** Cached property utilities */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};

template<typename ProxyType, typename NumericType>
inline TSharedRef<SWidget> FMathStructProxyCustomization::MakeNumericProxyWidget(
	TSharedRef<IPropertyHandle>& StructPropertyHandle,
	TSharedRef< TProxyProperty<ProxyType, NumericType> >& ProxyValue,
	const FText& Label,
	bool bRotationInDegrees,
	const FLinearColor& LabelBackgroundColor)
{
	TWeakPtr<IPropertyHandle> WeakHandlePtr = StructPropertyHandle;

	TSharedPtr<INumericTypeInterface<NumericType>> TypeInterface = nullptr;
	if (FUnitConversion::Settings().ShouldDisplayUnits() && bRotationInDegrees)
	{
		TypeInterface = MakeShared<TNumericUnitTypeInterface<NumericType>>(EUnit::Degrees);
	}

	return
		SNew(SNumericEntryBox<NumericType>)
		.IsEnabled(this, &FMathStructProxyCustomization::IsValueEnabled, WeakHandlePtr)
		.Value(this, &FMathStructProxyCustomization::OnGetValue<ProxyType, NumericType>, WeakHandlePtr, ProxyValue)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.OnValueCommitted(this, &FMathStructProxyCustomization::OnValueCommitted<ProxyType, NumericType>, WeakHandlePtr, ProxyValue)
		.OnValueChanged(this, &FMathStructProxyCustomization::OnValueChanged<ProxyType, NumericType>, WeakHandlePtr, ProxyValue)
		.OnBeginSliderMovement(this, &FMathStructProxyCustomization::OnBeginSliderMovement)
		.OnEndSliderMovement(this, &FMathStructProxyCustomization::OnEndSliderMovement<ProxyType, NumericType>, WeakHandlePtr, ProxyValue)
		// Only allow spin on handles with one object.  Otherwise it is not clear what value to spin
		.AllowSpin(StructPropertyHandle->GetNumOuterObjects() == 1)
		.MinValue(TOptional<NumericType>())
		.MaxValue(TOptional<NumericType>())
		.MaxSliderValue(bRotationInDegrees ? 360.0f : TOptional<NumericType>())
		.MinSliderValue(bRotationInDegrees ? 0.0f : TOptional<NumericType>())
		.LabelPadding(FMargin(3))
		.ToolTipTextFormat(this, &FMathStructProxyCustomization::OnGetValueToolTipTextFormat, Label)
		.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
		.TypeInterface(TypeInterface)
		.Label()
		[
			SNumericEntryBox<NumericType>::BuildNarrowColorLabel(LabelBackgroundColor)
		];
}

template<typename ProxyType, typename NumericType>
inline void FMathStructProxyCustomization::OnEndSliderMovement( NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue )
{
	bIsUsingSlider = false;

	ProxyValue->Set(NewValue);
	FlushValues(WeakHandlePtr);
}

template<typename ProxyType, typename NumericType>
inline TOptional<NumericType> FMathStructProxyCustomization::OnGetValue( TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue ) const
{
	if(CacheValues(WeakHandlePtr))
	{
		return ProxyValue->Get();
	}
	return TOptional<NumericType>();
}

template<typename ProxyType, typename NumericType>
inline void FMathStructProxyCustomization::OnValueCommitted( NumericType NewValue, ETextCommit::Type CommitType, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue )
{
	if (!bIsUsingSlider && !GIsTransacting)
	{
		ProxyValue->Set(NewValue);
		FlushValues(WeakHandlePtr);
	}
}	

template<typename ProxyType, typename NumericType>
inline void FMathStructProxyCustomization::OnValueChanged( NumericType NewValue, TWeakPtr<IPropertyHandle> WeakHandlePtr, TSharedRef< TProxyProperty<ProxyType, NumericType> > ProxyValue )
{
	if( bIsUsingSlider )
	{
		ProxyValue->Set(NewValue);
		FlushValues(WeakHandlePtr);
	}
}

struct FTransformField
{
	enum Type
	{
		Location,
		Rotation,
		Scale
	};
};

/** 
 * Proxy struct customization that displays a matrix as a position, rotation & scale.
 */
template<typename T>
class FMatrixStructCustomization : public FMathStructProxyCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

public:
	FMatrixStructCustomization()
		: CachedRotation(MakeShareable( new TProxyValue<UE::Math::TRotator<T>>(UE::Math::TRotator<T>::ZeroRotator)))
		, CachedRotationYaw(MakeShareable( new TProxyProperty<UE::Math::TRotator<T>, T>(CachedRotation, CachedRotation->Get().Yaw)))
		, CachedRotationPitch(MakeShareable( new TProxyProperty<UE::Math::TRotator<T>, T>(CachedRotation, CachedRotation->Get().Pitch)))
		, CachedRotationRoll(MakeShareable( new TProxyProperty<UE::Math::TRotator<T>, T>(CachedRotation, CachedRotation->Get().Roll)))
		, CachedTranslation(MakeShareable( new TProxyValue<UE::Math::TVector<T>>(UE::Math::TVector<T>::ZeroVector)))
		, CachedTranslationX(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedTranslation, CachedTranslation->Get().X)))
		, CachedTranslationY(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedTranslation, CachedTranslation->Get().Y)))
		, CachedTranslationZ(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedTranslation, CachedTranslation->Get().Z)))
		, CachedScale(MakeShareable( new TProxyValue<UE::Math::TVector<T>>(UE::Math::TVector<T>::ZeroVector)))
		, CachedScaleX(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedScale, CachedScale->Get().X)))
		, CachedScaleY(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedScale, CachedScale->Get().Y)))
		, CachedScaleZ(MakeShareable( new TProxyProperty<UE::Math::TVector<T>, T>(CachedScale, CachedScale->Get().Z)))
		, bUseLeftUpForwardAxisDisplayCoordinateSystem( false )
	{
	}

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	/** Customization utility functions */
	void CustomizeLocation(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row);
	void CustomizeRotation(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row, TOptional<FText> OverrideName = TOptional<FText>{});
	void CustomizeScale(TSharedRef<class IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& Row);

	/** FMathStructCustomization interface */
	UE_API virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override;

	/** FMathStructProxyCustomization interface */
	UE_API virtual bool CacheValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
	UE_API virtual bool FlushValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
	UE_API virtual FIntVector4 GetSwizzle() const;

	UE_API void OnCopy(FTransformField::Type Type, TWeakPtr<IPropertyHandle> PropertyHandlePtr);
	UE_API void OnPaste(FTransformField::Type Type, TWeakPtr<IPropertyHandle> PropertyHandlePtr);

	void PasteFromText(
		const FString& InTag,
		const FString& InText,
		FTransformField::Type Type,
		TWeakPtr<IPropertyHandle> PropertyHandlePtr);

	void OnPasteFromText(
		const FString& InTag,
		const FString& InText,
		const TOptional<FGuid>& InOperationId,
		FTransformField::Type Type,
		TWeakPtr<IPropertyHandle> PropertyHandlePtr);

protected:
	/** Cached rotation values */
	mutable TSharedRef< TProxyValue<UE::Math::TRotator<T>> > CachedRotation;
	mutable TSharedRef< TProxyProperty<UE::Math::TRotator<T>, T> > CachedRotationYaw;
	mutable TSharedRef< TProxyProperty<UE::Math::TRotator<T>, T> > CachedRotationPitch;
	mutable TSharedRef< TProxyProperty<UE::Math::TRotator<T>, T> > CachedRotationRoll;

	/** Cached translation values */
	mutable TSharedRef< TProxyValue<UE::Math::TVector<T>> > CachedTranslation;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T > > CachedTranslationX;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T > > CachedTranslationY;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T> > CachedTranslationZ;

	/** Cached scale values */
	mutable TSharedRef< TProxyValue<UE::Math::TVector<T>> > CachedScale;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T> > CachedScaleX;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T> > CachedScaleY;
	mutable TSharedRef< TProxyProperty<UE::Math::TVector<T>, T> > CachedScaleZ;

	/** Whether or not this customization is using the LeftUpForward axis display coordinate system */
	bool bUseLeftUpForwardAxisDisplayCoordinateSystem;
};

/** 
 * Proxy struct customization that displays an FTransform as a position, euler rotation & scale.
 */
template<typename T>
class FTransformStructCustomization : public FMatrixStructCustomization<T>
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	/** FMathStructProxyCustomization interface */
	UE_API virtual bool CacheValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
	UE_API virtual bool FlushValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
	UE_API virtual FIntVector4 GetSwizzle() const override;
};

/**
 * Proxy struct customization that displays an FQuat as an euler rotation
 */
template<typename T>
class FQuatStructCustomization : public FMatrixStructCustomization<T>
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** FMathStructCustomization interface */
	UE_API virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& Row) override;

protected:
	/** FMathStructProxyCustomization interface */
	UE_API virtual bool CacheValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
	UE_API virtual bool FlushValues(TWeakPtr<IPropertyHandle> PropertyHandlePtr) const override;
};

#undef UE_API