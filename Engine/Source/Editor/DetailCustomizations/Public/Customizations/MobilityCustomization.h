// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "IDetailCustomNodeBuilder.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDetailWidgetRow;
class IDetailCategoryBuilder;
class IPropertyHandle;

enum class ECheckBoxState : uint8;

// Helper class to create a Mobility customization for the specified Property in the specified CategoryBuilder.
class FMobilityCustomization : public IDetailCustomNodeBuilder, public TSharedFromThis<FMobilityCustomization>
{
public:
	enum : uint8
	{
		StaticMobilityBitMask = (1u << EComponentMobility::Static),
		StationaryMobilityBitMask = (1u << EComponentMobility::Stationary),
		MovableMobilityBitMask = (1u << EComponentMobility::Movable),
	};

	UE_API FMobilityCustomization(TSharedPtr<IPropertyHandle> InMobilityHandle, uint8 InRestrictedMobilityBits, bool InForLight);

	UE_API virtual void GenerateHeaderRowContent(FDetailWidgetRow& WidgetRow) override;
	UE_API virtual FName GetName() const override;
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override { return MobilityHandle; }

private:

	UE_API EComponentMobility::Type GetActiveMobility() const;
	UE_API FSlateColor GetMobilityTextColor(EComponentMobility::Type InMobility) const;
	UE_API void OnMobilityChanged(EComponentMobility::Type InMobility);
	UE_API FText GetMobilityToolTip() const;

	TSharedPtr<IPropertyHandle> MobilityHandle;
	bool bForLight;
	uint8 RestrictedMobilityBits;
};

#undef UE_API
