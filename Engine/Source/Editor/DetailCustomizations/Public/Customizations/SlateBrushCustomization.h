// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class IPropertyHandle;
class SErrorText;

class FSlateBrushStructCustomization : public IPropertyTypeCustomization
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance(bool bIncludePreview);

	UE_API FSlateBrushStructCustomization(bool bIncludePreview);

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

	UE_API virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;              

private:
	/**
	 * Get the Slate Brush outline settings property row visibility
	 */
	UE_API EVisibility GetOutlineSettingsPropertyVisibility() const;

	/**
	 * Get the Slate Brush tiling property row visibility
	 */
	UE_API EVisibility GetTilingPropertyVisibility() const;

	/**
	 *  Get the Slate Brush margin property row visibility
	 */
	UE_API EVisibility GetMarginPropertyVisibility() const;

	/** Callback for determining image size reset button visibility */
	UE_API bool IsImageSizeResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	/** Callback for clicking the image size reset button */
	UE_API void OnImageSizeResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	/** Gets the current default image size based on the current texture resource */
	UE_API FVector2f GetDefaultImageSize() const;

	/** Slate Brush DrawAs property */
	TSharedPtr<IPropertyHandle> DrawAsProperty;

	/** Slate Brush Image Size property */
	TSharedPtr<IPropertyHandle> ImageSizeProperty;

	/** Slate Brush Resource Object property */
	TSharedPtr<IPropertyHandle> ResourceObjectProperty;

	/** Slate Brush Resource Name property */
	TSharedPtr<IPropertyHandle> ResourceNameProperty;

	/** Slate Brush Image Type property */
	TSharedPtr<IPropertyHandle> ImageTypeProperty;

	/** Error text to display if the resource object is not valid*/
	TSharedPtr<SErrorText> ResourceErrorText;

	/** Should we show the preview portion of the customization? */
	bool bIncludePreview;
};

#undef UE_API
