// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SWidget.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class IPropertyHandle;
class SWidget;
struct FAssetData;
struct FSlateFontInfo;

/** Customize the appearance of an FSlateFontInfo */
class FSlateFontInfoStructCustomization : public IPropertyTypeCustomization
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

protected:
	UE_API void AddFontSizeProperty(IDetailChildrenBuilder& InStructBuilder);
	UE_API bool IsFontSizeEnabled() const;

	/** @return The value or unset if properties with multiple values are viewed */
	UE_API TOptional<float> OnFontSizeGetValue() const;
	UE_API void OnFontSizeValueChanged(float NewDisplayValue);
	UE_API void OnFontSizeValueCommitted(float NewDisplayValue, ETextCommit::Type CommitInfo);

	/**
	 * Called when the slider begins to move.  We create a transaction here to undo the property
	 */
	UE_API void OnFontSizeBeginSliderMovement();

	/**
	 * Called when the slider stops moving.  We end the previously created transaction
	 */
	UE_API void OnFontSizeEndSliderMovement(float NewDisplayValue);

	/** @return a dynamic text explaining what the size does and showing the current Font DPI setting */
	UE_API FText GetFontSizeTooltipText() const;

	static UE_API float ConvertFontSizeFromNativeToDisplay(float FontSize);
	static UE_API float ConvertFontSizeFromDisplayToNative(float FontSize);

	/** Called to filter out invalid font assets */
	static UE_API bool OnFilterFontAsset(const FAssetData& InAssetData);

	/** Called when the font object used by this FSlateFontInfo has been changed */
	UE_API void OnFontChanged(const FAssetData& InAssetData);

	/** Called to see whether the font entry combo should be enabled */
	UE_API bool IsFontEntryComboEnabled() const;

	/** Called before the font entry combo is opened - used to update the list of available font entries */
	UE_API void OnFontEntryComboOpening();

	/** Called when the selection of the font entry combo is changed */
	UE_API void OnFontEntrySelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type);

	/** Make the widget for an entry in the font entry combo */
	UE_API TSharedRef<SWidget> MakeFontEntryWidget(TSharedPtr<FName> InFontEntry);

	/** Get the text to use for the font entry combo button */
	UE_API FText GetFontEntryComboText() const;

	/** Get the name of the currently active font entry (may not be the selected entry if the entry is set to use "None") */
	UE_API FName GetActiveFontEntry() const;

	/** Get the array of FSlateFontInfo instances this customization is currently editing */
	UE_API TArray<FSlateFontInfo*> GetFontInfoBeingEdited();
	UE_API TArray<const FSlateFontInfo*> GetFontInfoBeingEdited() const;

	/** Handle to the struct property being edited */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Handle to the "FontObject" property being edited */
	TSharedPtr<IPropertyHandle> FontObjectProperty;

	/** Handle to the "TypefaceFontName" property being edited */
	TSharedPtr<IPropertyHandle> TypefaceFontNameProperty;

	/** Handle to the "Size" property being edited */
	TSharedPtr<IPropertyHandle> FontSizeProperty;

	/** Font entry combo box widget */
	TSharedPtr<SComboBox<TSharedPtr<FName>>> FontEntryCombo;

	/** Source data for the font entry combo widget */
	TArray<TSharedPtr<FName>> FontEntryComboData;

	/** True if the slider is being used to change the value of the property */
	bool bIsUsingSlider = false;

	/** When using the slider, what was the last committed value */
	float LastSliderFontSizeCommittedValue = 0.0f;

};

#undef UE_API
