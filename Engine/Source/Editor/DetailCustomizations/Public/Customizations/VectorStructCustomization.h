// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Customizations/MathStructCustomizations.h"
#include "Templates/SharedPointer.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class IPropertyHandle;
class IPropertyTypeCustomization;

/**
 * Customizes FVector structs.
 */
class FVectorStructCustomization
	: public FMathStructCustomization
{
public:

	/** @return A new instance of this class */
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** FMathStructCustomization interface */
	UE_API virtual void GetSortedChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, TArray<TSharedRef<IPropertyHandle>>& OutChildren) override;
};

#undef UE_API
