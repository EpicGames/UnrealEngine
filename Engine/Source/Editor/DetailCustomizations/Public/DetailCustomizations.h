// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "PropertyEditorDelegates.h"
#include "UObject/NameTypes.h"

class FDetailCustomizationsModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	/**
	 * Suppresses the DevelopmentStatus warnings for the given class or any derived versions of this class
	 */
	void DETAILCUSTOMIZATIONS_API RegisterDevelopmentStatusWarningSuppression(FName ClassName);
	/**
	 * Removes suppression of DevelopmentStatus warnings for the given class or any derived versions of this class
	 */
	void DETAILCUSTOMIZATIONS_API UnregisterDevelopmentStatusWarningSuppression(FName ClassName);

	bool IsDevelopmentStatusWarningSuppressed(const UClass* Class) const;

private:
	void RegisterPropertyTypeCustomizations();
	void RegisterObjectCustomizations();
	void RegisterSectionMappings();

	/**
	 * Registers a custom class
	 *
	 * @param ClassName				The class name to register for property customization
	 * @param DetailLayoutDelegate	The delegate to call to get the custom detail layout instance
	 */
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate );

	/**
	* Registers a custom struct
	*
	* @param StructName				The name of the struct to register for property customization
	* @param StructLayoutDelegate	The delegate to call to get the custom detail layout instance
	*/
	void RegisterCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate );
private:
	/** List of registered class that we must unregister when the module shuts down */
	TSet< FName > RegisteredClassNames;
	TSet< FName > RegisteredPropertyTypes;
	TSet< FName > SuppressedDevelopmentStatusWarnings;
};
