// Copyright Epic Games, Inc. All Rights Reserved.
#include "IDetailCustomization.h"

/** Used to hide categories in a Details View*/
class FDetailsViewCategoryHiderCustomization : public IDetailCustomization
{
public:
	static DETAILCUSTOMIZATIONS_API TSharedRef<IDetailCustomization> MakeInstance(TArray<FName>&& InCategoriesToHide);
	static DETAILCUSTOMIZATIONS_API TSharedRef<IDetailCustomization> MakeInstance(const TArrayView<FName>& InCategoriesToHide);

private:
	FDetailsViewCategoryHiderCustomization(TArray<FName>&& InCategoriesToHide);
	FDetailsViewCategoryHiderCustomization(const TArrayView<FName>& InCategoriesToHide);

	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailLayoutBuilder) override;

private:
	TArray<FName> CategoriesToHide;

	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
};
