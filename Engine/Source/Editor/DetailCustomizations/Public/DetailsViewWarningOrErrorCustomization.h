// Copyright Epic Games, Inc. All Rights Reserved.
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "SWarningOrErrorBox.h"

/** Used to insert an SWarningOrErrorBox into a Details View at the top of a specified category */
class FDetailsViewWarningOrErrorCustomization : public IDetailCustomization
{
public:
	static DETAILCUSTOMIZATIONS_API TSharedRef<IDetailCustomization> MakeInstance(
		const FName& InCategoryForInsertion,
		const FName& InRowTag,
		const FText& InWarningOrErrorLabel,
		const EMessageStyle InMessageStyle = EMessageStyle::Warning,
		ECategoryPriority::Type InCategoryPriority = ECategoryPriority::Uncommon);

private:
	FDetailsViewWarningOrErrorCustomization(
		const FName& InCategoryForInsertion, 
		const FName& InRowTag, 
		const FText& InWarningOrErrorLabel, 
		const EMessageStyle InMessageStyle, 
		const ECategoryPriority::Type InCategoryPriority);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder) override;
	
private:
	FName CategoryForInsertion;
	FName RowTag;
	FText WarningOrErrorLabel;
	EMessageStyle MessageStyle;
	ECategoryPriority::Type CategoryPriority;

	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
};
