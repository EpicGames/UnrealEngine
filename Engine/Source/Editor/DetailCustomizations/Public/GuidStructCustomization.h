// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IPropertyHandle;
class SEditableTextBox;
struct FGuid;

namespace EPropertyEditorGuidActions
{
	/**
	 * Enumerates quick-set action types.
	 */
	enum Type
	{
		/** Generate a new GUID. */
		Generate,

		/** Set a null GUID. */
		Invalidate
	};
}

// Writes the supplied GUID to the supplied property handle as one undoable transaction
DETAILCUSTOMIZATIONS_API void WriteGuidToProperty(TSharedPtr<IPropertyHandle> GuidPropertyHandle, const FGuid& Guid);

/**
 * Implements a details panel customization for FGuid structures.
 */
class FGuidStructCustomization
	: public IPropertyTypeCustomization
{
public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for Guids.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance( )
	{
		return MakeShareable(new FGuidStructCustomization);
	}

public:

	// IPropertyTypeCustomization interface

	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

protected:

	/**
	 * Sets the property's value.
	 *
	 * @param Guid The value to set.
	 */
	void SetGuidValue( const FGuid& Guid );

private:

	/** Callback for clicking an item in the quick-set menu. */
	void HandleGuidActionClicked( EPropertyEditorGuidActions::Type Action );

	/** Handles getting the text color of the editable text box. */
	FSlateColor HandleTextBoxForegroundColor() const;

	/** Handles getting the text to be displayed in the editable text box. */
	FText HandleTextBoxText( ) const;

	/** Handles changing the value in the editable text box. */
	void HandleTextBoxTextChanged( const FText& NewText );

	/** Handles committing the text in the editable text box. */
	void HandleTextBoxTextCommited( const FText& NewText, ETextCommit::Type CommitInfo );

private:

	/** Holds a flag indicating whether the current input is a valid GUID. */
	bool InputValid;

	/** Holds a handle to the property being edited. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Holds the text box for editing the Guid. */
	TSharedPtr<SEditableTextBox> TextBox;
};
