// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "InputCoreTypes.h"
#include "Misc/Optional.h"
#include "PropertyHandle.h"
#include "SKeySelector.h"
#include "Templates/SharedPointer.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class IPropertyHandle;
class SWidget;
struct FKey;

/**
 * Implements a details panel customization for FKey structures.
 * As  "Key"				<SKeySelector>
 */
class FKeyStructCustomization
	: public IPropertyTypeCustomization
{
public:
	UE_API FKeyStructCustomization();

	UE_NONCOPYABLE(FKeyStructCustomization)

	// IPropertyTypeCustomization interface

	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override { };
	virtual bool ShouldInlineKey() const override { return true; }

	// Helper variant that generates the key struct in the header and appends a single button at the end
	// TODO: Is there a better way?
	UE_API void CustomizeHeaderOnlyWithButton(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TSharedRef<SWidget> Button);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Sets a bool for whether the information Icon for a combo trigger is displayed or not */ 
	UE_DEPRECATED(5.2, "SetDisplayIcon(bShouldDisplayIcon) is deprecated and the icon referenced has been removed")
	void SetDisplayIcon(bool bShouldDisplayIcon)
	{
		bDisplayIcon = bShouldDisplayIcon;
	}

	/** Gets bDisplayIcon bool */
	UE_DEPRECATED(5.2, "GetDisplayIcon() is deprecated and the icon referenced has been removed")
	bool GetDisplayIcon() const
	{
		return bDisplayIcon;
	}
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Sets a bool for whether the key selector should be enabled or not */
	void SetEnableKeySelector(bool bKeySelectorEnabled);

	
	/** Gets bEnableKeySelector bool */
	bool GetEnableKeySelector() const
	{
		return bEnableKeySelector;
	}
	
	/** Sets default key name - what key selector will default to if it is disabled */
	void SetDefaultKeyName(FString KeyName)
	{
		DefaultKeyName = KeyName;
	}
	
	/** Gets default key name - what key selector will default to if it is disabled */
	FString GetDefaultKeyName() const
	{
		return DefaultKeyName;
	}

	/** Sets tooltip on the KeySelector when it is disabled */
	void SetDisabledKeySelectorToolTip(const FText& InToolTip)
	{
		DisabledKeySelectorToolTip = InToolTip;
	}
	
	/** Gets tooltip on the KeySelector when it is disabled */
	FText GetDisabledKeySelectorToolTip() const
	{
		return DisabledKeySelectorToolTip;
	}

	UE_API void SetKey(const FString& KeyName);
	
public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for Keys.
	 */
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance( );

protected:
	/** Whether the information icon for combo triggers is displayed or not */
	UE_DEPRECATED(5.2, "bDisplayIcon is deprecated and the icon has been removed")
	bool bDisplayIcon = false;
	
	/** Whether the key selector should be enabled or not */
	bool bEnableKeySelector = true;

	/** Tooltip to display on the actual KeySelector */
	FText DisabledKeySelectorToolTip = FText::FromString(TEXT("Key Selector Disabled"));
	
	/** Default Key to assign the KeySelector when it is disabled */
	FString DefaultKeyName = TEXT("None");
	
	/** Gets the current Key being edited. */
	UE_API TOptional<FKey> GetCurrentKey() const;

	/** Updates the property when a new key is selected. */
	UE_API void OnKeyChanged(TSharedPtr<FKey> SelectedKey);

	/** Holds a handle to the property being edited. */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	
	/** shared pointer to the Key Selector. */
    TSharedPtr<SKeySelector> KeySelector = nullptr;
};

#undef UE_API
