// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "IDetailCustomNodeBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "UObject/PerPlatformProperties.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UnrealClient.h"
#include "Widgets/SWidget.h"

#define UE_API DETAILCUSTOMIZATIONS_API

class FDetailWidgetDecl;
class FDetailWidgetRow;
class FMenuBuilder;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class SWidget;

typedef typename TSlateDelegates<FName>::FOnGenerateWidget FOnGenerateWidget;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnPlatformOverrideAction, FName);

struct FPerPlatformPropertyCustomNodeBuilderArgs
{
	/** Callback to generate the name widget. */
	FOnGetContent OnGenerateNameWidget;

	/** List of platforms that can override the default value */
	TAttribute<TArray<FName>> PlatformOverrideNames;

	/** Is overriding per-property enabled */
	TAttribute<bool> IsEnabled = true;

	/** Callback to generate a widget for a specific platform row */
	FOnGenerateWidget OnGenerateWidgetForPlatformRow;

	FOnPlatformOverrideAction OnAddPlatformOverride;

	FOnPlatformOverrideAction OnRemovePlatformOverride;

	/** Used for search box queries */
	FText FilterText;

	/** Used to identify the builder through GetName() */
	FName Name;
};

class FPerPlatformPropertyCustomNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FPerPlatformPropertyCustomNodeBuilder>
{
public:
	FPerPlatformPropertyCustomNodeBuilder(const FPerPlatformPropertyCustomNodeBuilderArgs& InArgs)
		: Args(InArgs)
	{}

	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }

	UE_API virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	UE_API virtual void SetOnToggleExpansion(FOnToggleNodeExpansion InOnRegenerateChildren) override;
	UE_API virtual void GenerateHeaderRowContent(FDetailWidgetRow& HeaderRow);
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual bool InitiallyCollapsed() const override { return true; }
	UE_API virtual FName GetName() const override;
private:
	UE_API void OnAddPlatformOverride(FName PlatformName);
	UE_API bool OnRemovePlatformOverride(FName PlatformName);
	UE_API void AddPlatformToMenu(const FName PlatformName, const FTextFormat Format, FMenuBuilder& AddPlatformMenuBuilder);
private:
	/** Handle to the default value */
	FPerPlatformPropertyCustomNodeBuilderArgs Args;
	FSimpleDelegate OnRebuildChildren;
	FOnToggleNodeExpansion OnToggleExpansion;
};
/**
* Implements a details panel customization for the FPerPlatform structures.
*/
template<typename PerPlatformType>
class FPerPlatformPropertyCustomization : public IPropertyTypeCustomization
{
public:
	FPerPlatformPropertyCustomization()
	{}

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/**
	* Creates a new instance.
	*
	* @return A new customization for FPerPlatform structs.
	*/
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	TSharedRef<SWidget> GetWidget(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder) const;
	TArray<FName> GetPlatformOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const;
	bool AddPlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle);
	bool RemovePlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle);

private:
	/** Cached utils used for resetting customization when layout changes */
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
};

#undef UE_API
