// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SplineMetadataDetailsFactory.generated.h"

class USplineComponent;
class IDetailGroup;

class ISplineMetadataDetails
{
public:
	virtual FName GetName() const = 0;
	virtual FText GetDisplayName() const = 0;
	virtual void Update(USplineComponent* InSplineComponent, const TSet<int32>& InSelectedKeys) = 0;
	virtual void GenerateChildContent(IDetailGroup& InGroup) = 0;
};

UCLASS(MinimalAPI, Abstract)
class USplineMetadataDetailsFactoryBase : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<ISplineMetadataDetails> Create() PURE_VIRTUAL(USplineMetadataDetailsFactoryBase::Create, return nullptr;);
	virtual UClass* GetMetadataClass() const PURE_VIRTUAL(USplineMetadataDetailsFactoryBase::GetMetadataClass, return nullptr;);
};

